// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ApplicationCore : ModuleRules
{
	public ApplicationCore(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core"
			}
		);

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"RHI"
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"InputDevice",
				"Analytics",
				"SynthBenchmark"
			}
		);

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows))
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target,
				"XInput"
			);
			if (Target.bCompileWithAccessibilitySupport && !Target.bIsBuildingConsoleApplication)
			{
				PublicSystemLibraries.Add("uiautomationcore.lib");
				PublicDefinitions.Add("UE_WINDOWS_USING_UIA=1");
			}
			else
			{
				PublicDefinitions.Add("UE_WINDOWS_USING_UIA=0");
			}

			PrivateDependencyModuleNames.Add("WindowsD3D");

			PublicSystemLibraries.Add("shcore.lib");
		}
		else if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target,
				"OpenGL"
			);
			if (Target.bBuildEditor == true)
			{
				string XcodeRoot = Utils.RunLocalProcessAndReturnStdOut("/usr/bin/xcode-select", "--print-path");
				PublicAdditionalLibraries.Add(XcodeRoot + "/Platforms/MacOSX.platform/Developer/SDKs/MacOSX.sdk/System/Library/PrivateFrameworks/MultitouchSupport.framework/Versions/Current/MultitouchSupport.tbd");
			}
			
			PublicFrameworks.Add("GameController");
			PublicFrameworks.Add("CoreHaptics");
		}
		else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target,
				"SDL3"
			);

			// We need FreeType2 and GL for the Splash, but only in the Editor
			if (Target.bCompileAgainstEditor)
			{
				AddEngineThirdPartyPrivateStaticDependencies(Target, "FreeType2");
				AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenGL");
				PrivateIncludePathModuleNames.Add("ImageWrapper");
			}
		}
		else if (Target.IsInPlatformGroup(UnrealPlatformGroup.IOS))
		{
			PublicIncludePaths.AddRange(new string[] {"Runtime/ApplicationCore/Public/IOS"});
			PublicIncludePaths.AddRange(new string[] {"Runtime/ApplicationCore/Private/Apple"});
			AddEngineThirdPartyPrivateStaticDependencies(Target, "SoundSwitch");

			PublicFrameworks.Add("GameController");
			PublicFrameworks.Add("CoreHaptics");

			// export ApplicationCore symbols for embedded Dlls
			ModuleSymbolVisibility = ModuleRules.SymbolVisibility.VisibileForDll;
			
			//Need to add this as BackgroundHTTP files can end up doing work directly from our AppDelegate in iOS and thus we need acccess to correct file locations to save these very early.
			PrivateDependencyModuleNames.Add("BackgroundHTTPFileHash");
			PrivateDependencyModuleNames.Add("CoreUObject");
			PrivateDependencyModuleNames.Add("BuildSettings");
			PrivateDependencyModuleNames.Add("BackgroundHttpIOS");
			PublicIncludePathModuleNames.Add("BackgroundHTTP");
		}
		else if (Target.Platform == UnrealTargetPlatform.Android)
		{
			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Launch",
					"Media",
					"VulkanRHI"
				}
			);
		}

		if (!Target.IsTestTarget && !Target.bCompileAgainstApplicationCore)
		{
			throw new System.Exception("ApplicationCore cannot be used when Target.bCompileAgainstApplicationCore = false.");
		}

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
