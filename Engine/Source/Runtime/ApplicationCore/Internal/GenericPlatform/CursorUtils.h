// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"


namespace UE::Cursor
{
	APPLICATIONCORE_API float CalculateDeltaWithAcceleration(int32 Delta, float Sensitivity);
}
