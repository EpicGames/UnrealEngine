// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Visual C++ redistributable version information.
 */
struct VersionInfo
{
	constexpr VersionInfo() = default;
	
	constexpr VersionInfo(unsigned long InMajor, unsigned long InMinor, unsigned long InBld, unsigned long InRbld)
		: Major(InMajor)
		, Minor(InMinor)
		, Bld(InBld)
		, Rbld(InRbld)
	{}

	constexpr VersionInfo(unsigned long long InVersion)
		: Major((InVersion >> 48) & 0xffff)
		, Minor((InVersion >> 32) & 0xffff)
		, Bld((InVersion >> 16) & 0xffff)
		, Rbld((InVersion >> 0) & 0xffff)
	{ }

	/** Set if any version number is non-zero. */
	inline bool IsSet() const
	{
		return Major || Minor || Bld || Rbld;
	}

	unsigned long Major = 0;
	unsigned long Minor = 0;
	unsigned long Bld = 0;
	unsigned long Rbld = 0;
};

// This minimum should match the version installed by
// Engine/Extras/Redist/en-us/vc_redist.x64.exe
inline constexpr VersionInfo MinRedistVersion = { 14, 50, 35719, 0 };

inline constexpr VersionInfo MinGameInputRedistVersion = { 3, 1, 26100, 6879 };

/** Valid if the specified version is greater or equal than the minimum version. */
inline bool IsVersionValid(const VersionInfo& Version, const VersionInfo& MinVersion)
{
	if (Version.Major > MinVersion.Major) return true;
	if (Version.Major == MinVersion.Major && Version.Minor > MinVersion.Minor) return true;
	if (Version.Major == MinVersion.Major && Version.Minor == MinVersion.Minor && Version.Bld > MinVersion.Bld) return true;
	if (Version.Major == MinVersion.Major && Version.Minor == MinVersion.Minor && Version.Bld == MinVersion.Bld && Version.Rbld >= MinVersion.Rbld) return true;
	return false;
}


