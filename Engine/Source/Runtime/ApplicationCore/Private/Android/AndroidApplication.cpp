// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidApplication.h"
#include "Android/AndroidInputInterface.h"
#include "Android/AndroidWindow.h"
#include "Android/AndroidCursor.h"
#include "IInputDeviceModule.h"
#include "HAL/OutputDevices.h"
#include "Misc/AssertionMacros.h"
#include "Android/AndroidPlatformMisc.h"
#include "Misc/CoreDelegates.h"

DEFINE_LOG_CATEGORY_STATIC(LogAndroidApplication, Log, All);

bool FAndroidApplication::bWindowSizeChanged = false;

FAndroidApplication* FAndroidApplication::_application = nullptr;

TAutoConsoleVariable<bool> CVarAndroidSupportsTimestampQueries(
	TEXT("r.Android.SupportsTimestampQueries"),
	0,
	TEXT("State of Android (GLES and Vulkan) timestamp queries support on an Android device\n")
	TEXT("  0 = unsupported\n")
	TEXT("  1 = supported."),
	ECVF_SetByDeviceProfile);

TAutoConsoleVariable<bool> CVarAndroidSupportsDynamicResolution(
	TEXT("r.Android.SupportsDynamicResolution"),
	0,
	TEXT("State of DynamicResolution (GLES and Vulkan) support on an Android device\n")
	TEXT("  0 = unsupported\n")
	TEXT("  1 = supported."),
	ECVF_SetByDeviceProfile
);

FAndroidApplication* FAndroidApplication::CreateAndroidApplication()
{
	return new FAndroidApplication();
}

FAndroidApplication::FAndroidApplication()
	: GenericApplication(MakeShareable(new FAndroidCursor()))
	, InputInterface( FAndroidInputInterface::Create( MessageHandler, Cursor ) )
	, bHasLoadedInputPlugins(false)
{
	_application = this;
}

FAndroidApplication::FAndroidApplication(TSharedPtr<class FAndroidInputInterface> InInputInterface)
	: GenericApplication((InInputInterface.IsValid() && InInputInterface->GetCursor()) ? InInputInterface->GetCursor() : MakeShareable(new FAndroidCursor()))
	, InputInterface(InInputInterface)
	, bHasLoadedInputPlugins(false)
{
	_application = this;
}

FAndroidApplication::~FAndroidApplication()
{
	IModularFeatures::Get().OnModularFeatureRegistered().RemoveAll(this);
}

void FAndroidApplication::SetMessageHandler( const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler )
{
	GenericApplication::SetMessageHandler(InMessageHandler);
	InputInterface->SetMessageHandler( MessageHandler );
}

void FAndroidApplication::AddExternalInputDevice(TSharedPtr<IInputDevice> InputDevice)
{
	if (InputDevice.IsValid())
	{
		InputInterface->AddExternalInputDevice(MoveTemp(InputDevice));
	}
}

void FAndroidApplication::PollGameDeviceState( const float TimeDelta )
{
	if (!bHasLoadedInputPlugins)
	{
		// Create all externally-implemented input devices from loaded modules.
		TArray<IInputDeviceModule*> PluginImplementations = IModularFeatures::Get().GetModularFeatureImplementations<IInputDeviceModule>(IInputDeviceModule::GetModularFeatureName());
		for (auto InputPluginIt = PluginImplementations.CreateIterator(); InputPluginIt; ++InputPluginIt)
		{
			TSharedPtr<IInputDevice> Device = (*InputPluginIt)->CreateInputDevice(MessageHandler);
			AddExternalInputDevice(MoveTemp(Device));
		}

		// Modules loaded after this point are handled by this callback
		IModularFeatures::Get().OnModularFeatureRegistered().AddRaw(this, &FAndroidApplication::OnInputDeviceModuleRegistered);

		bHasLoadedInputPlugins = true;
	}

	// Poll game device state and send new events
	InputInterface->Tick( TimeDelta );
	InputInterface->SendControllerEvents();
	
	// Note: We're using window size values cached on GT. We dont wait for a valid HW window.
	if (bWindowSizeChanged && 
		Windows.Num() > 0)
	{
		int32 WindowX,WindowY, WindowWidth,WindowHeight;
		Windows[0]->GetFullScreenInfo(WindowX, WindowY, WindowWidth, WindowHeight);

		GenericApplication::GetMessageHandler()->OnSizeChanged(Windows[0],WindowWidth,WindowHeight, false);
		GenericApplication::GetMessageHandler()->OnResizingWindow(Windows[0]);
		
		FDisplayMetrics DisplayMetrics;
		FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);
		BroadcastDisplayMetricsChanged(DisplayMetrics);
		FCoreDelegates::OnSafeFrameChangedEvent.Broadcast();

		bWindowSizeChanged = false;
	}
    
    HandleDeviceOrientation();
}

void FAndroidApplication::HandleDeviceOrientation()
{
	FAndroidMisc::UpdateDeviceOrientation();
	const auto PreviousDeviceOrientation = DeviceOrientation;
	DeviceOrientation = FAndroidMisc::GetDeviceOrientation();

	if (PreviousDeviceOrientation != DeviceOrientation)
	{
		FCoreDelegates::ApplicationReceivedScreenOrientationChangedNotificationDelegate.Broadcast((int32)DeviceOrientation);
	}
}

FPlatformRect FAndroidApplication::GetWorkArea( const FPlatformRect& CurrentWindow ) const
{
	return FAndroidWindow::GetScreenRect();
}

IInputInterface* FAndroidApplication::GetInputInterface()
{
	// NOTE: This does not increase the reference count, so don't cache the result
	return InputInterface.Get();
}

void FAndroidApplication::Tick(const float TimeDelta)
{
	//generate event that will end up calling 'QueryCursor' in slate to support proper reporting of the cursor's type.
	MessageHandler->OnCursorSet();
}

bool FAndroidApplication::IsGamepadAttached() const
{
	FAndroidInputInterface* AndroidInputInterface = (FAndroidInputInterface*)InputInterface.Get();

	if (AndroidInputInterface)
	{
		return AndroidInputInterface->IsGamepadAttached();
	}

	return false;
}

bool FAndroidApplication::GetNativeWindowResolution(int32_t& OutWidth, int32_t& OutHeight) const
{
	if (Windows.IsEmpty()) return false;

	const TSharedPtr<FAndroidWindow>& Window = Windows[0];

	if (!Window.IsValid()) return false;

	return Window->GetNativeWindowResolution(OutWidth, OutHeight);
}

TSharedPtr<FGenericWindow> FAndroidApplication::GetWindowUnderCursor()
{
	return CurrentlyTouchedWindow;
}

void FAndroidApplication::SetCurrentlyTouchedWindow(TSharedPtr<FGenericWindow> InWindow)
{
	CurrentlyTouchedWindow = MoveTemp(InWindow);
}

TSharedPtr<FAndroidWindow> FAndroidApplication::GetPrimaryWindow() const
{
	return Windows.Num() > 0 ? TSharedPtr<FAndroidWindow>(Windows[0]) : TSharedPtr<FAndroidWindow>();
}

void FDisplayMetrics::RebuildDisplayMetrics( FDisplayMetrics& OutDisplayMetrics )
{
	// Get screen rect
	OutDisplayMetrics.PrimaryDisplayWorkAreaRect = FAndroidWindow::GetScreenRect();
	OutDisplayMetrics.VirtualDisplayRect = OutDisplayMetrics.PrimaryDisplayWorkAreaRect;

	// Total screen size of the primary monitor
	OutDisplayMetrics.PrimaryDisplayWidth = OutDisplayMetrics.PrimaryDisplayWorkAreaRect.Right - OutDisplayMetrics.PrimaryDisplayWorkAreaRect.Left;
	OutDisplayMetrics.PrimaryDisplayHeight = OutDisplayMetrics.PrimaryDisplayWorkAreaRect.Bottom - OutDisplayMetrics.PrimaryDisplayWorkAreaRect.Top;

	// Apply the debug safe zones
	OutDisplayMetrics.ApplyDefaultSafeZones();

	bool bIsPortrait = FAndroidWindow::IsPortraitOrientation();
	FVector4 SafeZoneRect = FAndroidWindow::GetSafezone(bIsPortrait);

	// ConfigRules values override values from device
	if (const FString* SafeZoneVar = FPlatformMisc::GetConfigRulesVariable(bIsPortrait ? TEXT("SafeZone_Portrait") : TEXT("SafeZone_Landscape")))
	{
		TArray<FString> ZoneVector;
		ZoneVector.Reserve(4);

		int ZoneParseCount = SafeZoneVar->ParseIntoArray(ZoneVector, TEXT(","), true);
		ensureMsgf(ZoneParseCount == 4, TEXT("SafeZone variable not properly formatted."));

		if (ZoneParseCount == 4)
		{
			SafeZoneRect = FVector4(FCString::Atof(*ZoneVector[0]), FCString::Atof(*ZoneVector[1]), FCString::Atof(*ZoneVector[2]), FCString::Atof(*ZoneVector[3]));
		}
	}

	const double Inset_Left = SafeZoneRect.X > 0 ? SafeZoneRect.X * OutDisplayMetrics.PrimaryDisplayWidth : -1.0f;
	const double Inset_Top = SafeZoneRect.Y > 0 ? SafeZoneRect.Y * OutDisplayMetrics.PrimaryDisplayHeight : -1.0f;
	const double Inset_Right =  SafeZoneRect.Z > 0 ? SafeZoneRect.Z * OutDisplayMetrics.PrimaryDisplayWidth : -1.0f;
	const double Inset_Bottom = SafeZoneRect.W > 0 ? SafeZoneRect.W * OutDisplayMetrics.PrimaryDisplayHeight : -1.0f;

	OutDisplayMetrics.TitleSafePaddingSize.X = (Inset_Left >= 0.0f) ? Inset_Left : OutDisplayMetrics.TitleSafePaddingSize.X;
	OutDisplayMetrics.TitleSafePaddingSize.Y = (Inset_Top >= 0.0f) ? Inset_Top : OutDisplayMetrics.TitleSafePaddingSize.Y;
	OutDisplayMetrics.TitleSafePaddingSize.Z = (Inset_Right >= 0.0f) ? Inset_Right : OutDisplayMetrics.TitleSafePaddingSize.Z;
	OutDisplayMetrics.TitleSafePaddingSize.W = (Inset_Bottom >= 0.0f) ? Inset_Bottom : OutDisplayMetrics.TitleSafePaddingSize.W;
	OutDisplayMetrics.ActionSafePaddingSize = OutDisplayMetrics.TitleSafePaddingSize;
}

TSharedRef< FGenericWindow > FAndroidApplication::MakeWindow()
{
	return FAndroidWindow::Make();
}

void FAndroidApplication::InitializeWindow( const TSharedRef< FGenericWindow >& InWindow, const TSharedRef< FGenericWindowDefinition >& InDefinition, const TSharedPtr< FGenericWindow >& InParent, const bool bShowImmediately )
{
	const TSharedRef< FAndroidWindow > Window = StaticCastSharedRef< FAndroidWindow >( InWindow );
	const TSharedPtr< FAndroidWindow > ParentWindow = StaticCastSharedPtr< FAndroidWindow >( InParent );

	Windows.Add( Window );
	Window->Initialize( this, InDefinition, ParentWindow, bShowImmediately );
}

void FAndroidApplication::OnWindowSizeChanged()
{
	bWindowSizeChanged = true;
}

void FAndroidApplication::OnInputDeviceModuleRegistered(const FName& Type, IModularFeature* ModularFeature)
{
	if (Type == IInputDeviceModule::GetModularFeatureName())
	{
		IInputDeviceModule* Module = static_cast<IInputDeviceModule*>(ModularFeature);

		TSharedPtr<IInputDevice> Device = Module->CreateInputDevice(MessageHandler);
		AddExternalInputDevice(MoveTemp(Device));
	}
}
