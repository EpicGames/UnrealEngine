// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidCursor.h"
#include "Android/AndroidApplication.h"

FAndroidCursor::FAndroidCursor() 
	: CurrentType(EMouseCursor::None)
	, CurrentPosition(FVector2D::ZeroVector)
	, CursorClipRect()
	, bShow(false)
{
}

void FAndroidCursor::SetPosition( const int32 X, const int32 Y )
{
	FVector2D NewPosition(X, Y);

	UpdateCursorClipping(NewPosition);
	CurrentPosition = NewPosition;
}

void FAndroidCursor::SetType( const EMouseCursor::Type InNewCursor )
{
	CurrentType = InNewCursor;
}

void FAndroidCursor::GetSize(int32& Width, int32& Height) const
{
	Width = 32;
	Height = 32;
}

void FAndroidCursor::Show(bool bInShow)
{
	bShow = bInShow;
}

void FAndroidCursor::Lock(const RECT* const Bounds)
{
	if (Bounds == NULL)
	{
		FDisplayMetrics DisplayMetrics;
		FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);

		// The Android cursor should never leave the screen
		CursorClipRect.Min = FIntPoint::ZeroValue;
		CursorClipRect.Max.X = DisplayMetrics.PrimaryDisplayWidth - 1;
		CursorClipRect.Max.Y = DisplayMetrics.PrimaryDisplayHeight - 1;
	}
	else
	{
		CursorClipRect.Min.X = (int32)Bounds->left;
		CursorClipRect.Min.Y = (int32)Bounds->top;
		CursorClipRect.Max.X = (int32)Bounds->right - 1;
		CursorClipRect.Max.Y = (int32)Bounds->bottom - 1;
	}

	FVector2D Position = GetPosition();
	if (UpdateCursorClipping(Position))
	{
//		SetPosition(Position.X, Position.Y);
	}
}

bool FAndroidCursor::UpdateCursorClipping(FVector2D& CursorPosition)
{
	bool bAdjusted = false;

	if (CursorClipRect.Area() > 0)
	{
		if (CursorPosition.X < CursorClipRect.Min.X)
		{
			CursorPosition.X = CursorClipRect.Min.X;
			bAdjusted = true;
		}
		else if (CursorPosition.X > CursorClipRect.Max.X)
		{
			CursorPosition.X = CursorClipRect.Max.X;
			bAdjusted = true;
		}

		if (CursorPosition.Y < CursorClipRect.Min.Y)
		{
			CursorPosition.Y = CursorClipRect.Min.Y;
			bAdjusted = true;
		}
		else if (CursorPosition.Y > CursorClipRect.Max.Y)
		{
			CursorPosition.Y = CursorClipRect.Max.Y;
			bAdjusted = true;
		}
	}

	return bAdjusted;
}
