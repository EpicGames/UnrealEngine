// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidGameControllerMapper.h"
#include "Android/AndroidPlatformMisc.h"
#include "CoreGlobals.h"
#include "Misc/ConfigCacheIni.h"
#include <android/input.h>

extern int32 GAndroidOldXBoxWirelessFirmware;

namespace GameControllerMapping
{
	TSharedPtr<FAndroidGameControllerMapper> FAndroidGameControllerMapper::Instance = nullptr;

	TWeakPtr<FAndroidGameControllerMapper> FAndroidGameControllerMapper::GetInstance()
	{
		if (Instance == nullptr)
		{
			Instance = MakeShareable(new FAndroidGameControllerMapper());
			bool bBlockAndroidKeysOnControllers = false;
			GConfig->GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bBlockAndroidKeysOnControllers"), bBlockAndroidKeysOnControllers, GEngineIni);
			if (!Instance->InitByDataFile())
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidGameControllerMapper - fallback to hardcoded mapping data."));
				Instance->Init();
			}
		}
		return Instance.ToWeakPtr();
	}

	bool FAndroidGameControllerMapper::Init()
	{
		ButtonMappings.Reset();
		DeviceConfigurationMap.Reset();
		DeviceNames.Reset();

		// init ButtonMappings ...

		// Normal - index 0
		KeyCodeToKeyNameMap_SRef CodeToNameMap = MakeShared<KeyCodeToKeyNameMap>();
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_A, 0);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_DPAD_CENTER, 0);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_B, 1);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_X, 2);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_Y, 3);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_L1, 4);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_R1, 5);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_START, 6);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_MENU, 6);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_START, 17);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_MENU, 17);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_SELECT, 7);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BACK, 7);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_SELECT, 16);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BACK, 16);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_THUMBL, 8);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_THUMBR, 9);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_L2, 10);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_R2, 11);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_DPAD_UP, 12);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_DPAD_DOWN, 13);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_DPAD_LEFT, 14);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_DPAD_RIGHT, 15);
		ButtonMappings.Add(MoveTemp(CodeToNameMap));

		// Normal and MapL1R1ToTriggers - index 1
		CodeToNameMap = MakeShared<KeyCodeToKeyNameMap>();
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_A, 0);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_DPAD_CENTER, 0);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_B, 1);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_X, 2);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_Y, 3);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_L1, 4);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_L1, 10);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_R1, 5);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_R1, 11);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_START, 6);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_MENU, 6);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_START, 17);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_MENU, 17);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_SELECT, 7);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BACK, 7);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_SELECT, 16);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BACK, 16);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_THUMBL, 8);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_THUMBR, 9);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_L2, 10);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_R2, 11);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_DPAD_UP, 12);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_DPAD_DOWN, 13);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_DPAD_LEFT, 14);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_DPAD_RIGHT, 15);
		ButtonMappings.Add(MoveTemp(CodeToNameMap));

		// XBox - index 2
		CodeToNameMap = MakeShared<KeyCodeToKeyNameMap>();
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_A, 0);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_B, 1);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_C, 2);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_X, 3);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_Y, 4);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_Z, 5);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_R1, 6);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_R1, 17);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_L1, 7);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_L1, 16);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_L2, 8);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_R2, 9);
		ButtonMappings.Add(MoveTemp(CodeToNameMap));

		// PS4 - index 3
		CodeToNameMap = MakeShared<KeyCodeToKeyNameMap>();
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_B, 0);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_C, 1);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_A, 2);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_X, 3);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_Y, 4);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_Z, 5);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_L2, 6);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_L2, 17);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_MENU, 7);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_MENU, 16);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_SELECT, 8);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_START, 9);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_L1, 10);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_R1, 11);
		ButtonMappings.Add(MoveTemp(CodeToNameMap));

		// PS5 - index 4
		CodeToNameMap = MakeShared<KeyCodeToKeyNameMap>();
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_B, 0);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_C, 1);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_A, 2);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_X, 3);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_Y, 4);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_Z, 5);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_R2, 6);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_R2, 17);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, 3002, 7); // Special Left
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_THUMBL, 7);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_THUMBL, 16);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_SELECT, 8);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_START, 9);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_L1, 10);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_R1, 11);
		ButtonMappings.Add(MoveTemp(CodeToNameMap));

		// PS5 on Android newer than API 30 - index 5
		CodeToNameMap = MakeShared<KeyCodeToKeyNameMap>();
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_A, 0);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_B, 1);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_X, 2);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_Y, 3);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_L1, 4);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_R1, 5);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_THUMBL, 8);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_THUMBR, 9);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_L2, 10);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_R2, 11);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, 3002, 7); // Special Left
		AppendKeyCodeKeyNameMapping(CodeToNameMap, 3002, 16); // Touchpad
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_START, 6);
		AppendKeyCodeKeyNameMapping(CodeToNameMap, AKEYCODE_BUTTON_START, 17);
		ButtonMappings.Add(MoveTemp(CodeToNameMap));

		// init DeviceConfigurationMap ...

		// Default
		DeviceConfig_SRef DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ButtonMapping = ButtonMappings[0];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("Default"));
		RegisterDeviceConfig(-1, -1, DeviceCfg);

		// Amazon Fire Game Controller
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ButtonMapping = ButtonMappings[0];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("Amazon Fire Game Controller"));
		RegisterDeviceConfig(0x1949, 0x0402, DeviceCfg);
		RegisterDeviceConfig(0x1949, 0x0406, DeviceCfg);
		
		// Amazon Luna Game Controller
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ButtonMapping = ButtonMappings[0];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("Amazon Luna Game Controller"));
		RegisterDeviceConfig(0x1949, 0x0419, DeviceCfg);

		// Amazon other controllers
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->SupportsHat = false;
		DeviceCfg->ButtonMapping = ButtonMappings[0];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("Amazon"));
		RegisterDeviceConfig(0x1949, -1, DeviceCfg);

		// NVIDIA Corporation NVIDIA Controller
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ButtonMapping = ButtonMappings[0];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("NVIDIA Corporation NVIDIA Controller"));
		RegisterDeviceConfig(0x0955, 0x7214, DeviceCfg);
		RegisterDeviceConfig(0x0955, 0x7210, DeviceCfg);
		RegisterDeviceConfig(0x0955, 0x7203, DeviceCfg);

		// Samsung Game Pad EI-GP20
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->SupportsHat = true;
		DeviceCfg->MapL1R1ToTriggers = true;
		DeviceCfg->RightStickZRZ = false;
		DeviceCfg->RightStickRXRY = true;
		DeviceCfg->ButtonMapping = ButtonMappings[1];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("Samsung Game Pad EI-GP20"));
		RegisterDeviceConfig(0x04e8, 0xa000, DeviceCfg);

		// Mad Catz C.T.R.L.R
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ButtonMapping = ButtonMappings[0];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("Mad Catz C.T.R.L.R"));
		RegisterDeviceConfig(0x0738, 0x5263, DeviceCfg);
		RegisterDeviceConfig(0x0738, 0x5266, DeviceCfg);

		// Generic X-Box pad - Android API >= 31
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ControllerClass = ControllerClassType::XBoxWired;
		DeviceCfg->SupportsHat = true;
		DeviceCfg->TriggersUseThresholdForClick = true;
		DeviceCfg->ButtonMapping = ButtonMappings[0];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("Generic X-Box pad"));
		// The VID here is just a placeholder, the query will finally fall back to one with the device name
		RegisterDeviceConfig(0x045e, 0x0b12, DeviceCfg);

		// Generic X-Box pad - Android API < 31
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ControllerClass = ControllerClassType::XBoxWired_AndroidAPIBefore31;
		DeviceCfg->SupportsHat = true;
		DeviceCfg->TriggersUseThresholdForClick = true;
		DeviceCfg->RightStickZRZ = false;
		DeviceCfg->RightStickRXRY = true;
		DeviceCfg->MapZRZToTriggers = true;
		DeviceCfg->LTAnalogRangeMinimum = -1.0f;
		DeviceCfg->RTAnalogRangeMinimum = -1.0f;
		DeviceCfg->ButtonMapping = ButtonMappings[0];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("Generic X-Box pad"));
		RegisterDeviceConfig(0x045e, 0x0b12, DeviceCfg); // Add to the special device config map

		// Xbox Wired Controller
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ControllerClass = ControllerClassType::XBoxWired;
		DeviceCfg->SupportsHat = true;
		DeviceCfg->TriggersUseThresholdForClick = true;
		DeviceCfg->ButtonMapping = ButtonMappings[0];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("Xbox Wired Controller"));
		RegisterDeviceConfig(0x045e, 0x0202, DeviceCfg);
		RegisterDeviceConfig(0x045e, 0x0285, DeviceCfg);
		RegisterDeviceConfig(0x045e, 0x0287, DeviceCfg);
		RegisterDeviceConfig(0x045e, 0x0288, DeviceCfg);
		RegisterDeviceConfig(0x045e, 0x0289, DeviceCfg);
		RegisterDeviceConfig(0x045e, 0x028e, DeviceCfg);
		RegisterDeviceConfig(0x045e, 0x02d1, DeviceCfg);
		RegisterDeviceConfig(0x045e, 0x02ea, DeviceCfg);

		// Xbox Wireless Controller
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ControllerClass = ControllerClassType::XBoxWireless;
		DeviceCfg->SupportsHat = true;
		DeviceCfg->TriggersUseThresholdForClick = true;
		DeviceCfg->ButtonMapping = ButtonMappings[0];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("Xbox Wireless Controller"));
		RegisterDeviceConfig(0x045e, 0x02e0, DeviceCfg);
		RegisterDeviceConfig(0x045e, 0x02fd, DeviceCfg);
		RegisterDeviceConfig(0x045e, 0x0b13, DeviceCfg);

		// Xbox Wireless Controller - Older firmware before 3.1.1221.0
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ControllerClass = ControllerClassType::XBoxWireless_OlderFirmware;
		DeviceCfg->SupportsHat = true;
		DeviceCfg->TriggersUseThresholdForClick = true;
		DeviceCfg->MapZRZToTriggers = true;
		DeviceCfg->RightStickZRZ = false;
		DeviceCfg->RightStickRXRY = true;
		DeviceCfg->ButtonMapping = ButtonMappings[2];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("Xbox Wireless Controller"));
		RegisterDeviceConfig(0x045e, 0x02e0, DeviceCfg); // Add to the special device config map
		RegisterDeviceConfig(0x045e, 0x02fd, DeviceCfg); // Add to the special device config map
		RegisterDeviceConfig(0x045e, 0x0b13, DeviceCfg); // Add to the special device config map

		// Xbox Elite Wireless Controller
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ControllerClass = ControllerClassType::XBoxWireless;
		DeviceCfg->SupportsHat = true;
		DeviceCfg->TriggersUseThresholdForClick = true;
		DeviceCfg->ButtonMapping = ButtonMappings[0];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("Xbox Elite Wireless Controller"));
		RegisterDeviceConfig(0x045e, 0x0b00, DeviceCfg);
		RegisterDeviceConfig(0x045e, 0x0b05, DeviceCfg);

		// Xbox Elite Wireless Controller - Older firmware before 3.1.1221.0
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ControllerClass = ControllerClassType::XBoxWireless_OlderFirmware;
		DeviceCfg->SupportsHat = true;
		DeviceCfg->TriggersUseThresholdForClick = true;
		DeviceCfg->MapZRZToTriggers = true;
		DeviceCfg->RightStickZRZ = false;
		DeviceCfg->RightStickRXRY = true;
		DeviceCfg->ButtonMapping = ButtonMappings[2];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("Xbox Elite Wireless Controller"));
		RegisterDeviceConfig(0x045e, 0x0b00, DeviceCfg); // Add to the special device config map
		RegisterDeviceConfig(0x045e, 0x0b05, DeviceCfg);

		// SteelSeries Stratus XL
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->SupportsHat = true;
		DeviceCfg->TriggersUseThresholdForClick = true;
		DeviceCfg->LTAnalogRangeMinimum = 0.5f;
		DeviceCfg->ButtonMapping = ButtonMappings[0];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("SteelSeries Stratus XL"));
		RegisterDeviceConfig(0x0111, 0x1417, DeviceCfg);
		RegisterDeviceConfig(0x0111, 0x1419, DeviceCfg);

		// PS4 Wireless Controller (v2) - Android API >= 10
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ControllerClass = ControllerClassType::PlaystationWireless;
		DeviceCfg->SupportsHat = true;
		DeviceCfg->RightStickZRZ = true;
		DeviceCfg->ButtonMapping = ButtonMappings[0];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("PS4 Wireless Controller (v2)"));
		RegisterDeviceConfig(0x054c, 0x09cc, DeviceCfg);

		// PS4 Wireless Controller (v2) - Android API < 10 and CPU Vendor is not "Sony"
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ControllerClass = ControllerClassType::PS4Wireless_AndroidAPIBefore10_CPUNotSony;
		DeviceCfg->SupportsHat = true;
		DeviceCfg->RightStickZRZ = true;
		DeviceCfg->ButtonMapping = ButtonMappings[3];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("PS4 Wireless Controller (v2)"));
		RegisterDeviceConfig(0x054c, 0x09cc, DeviceCfg); // Add to the special device config map

		// PS4 Wireless Controller
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ControllerClass = ControllerClassType::PlaystationWireless;
		DeviceCfg->SupportsHat = true;
		DeviceCfg->RightStickZRZ = true;
		DeviceCfg->ButtonMapping = ButtonMappings[0];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("PS4 Wireless Controller"));
		RegisterDeviceConfig(0x054c, 0x05c4, DeviceCfg);

		// PS5 Wireless Controller - Android API >= 31
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ControllerClass = ControllerClassType::PlaystationWireless;
		DeviceCfg->SupportsHat = true;
		DeviceCfg->RightStickZRZ = true;
		DeviceCfg->MapRXRYToTriggers = false;
		DeviceCfg->LTAnalogRangeMinimum = -1.0f;
		DeviceCfg->RTAnalogRangeMinimum = -1.0f;
		DeviceCfg->ButtonMapping = ButtonMappings[5];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("PS5 Wireless Controller"));
		RegisterDeviceConfig(0x054c, 0x0ce6, DeviceCfg);

		// PS5 Wireless Controller - Android API < 31
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ControllerClass = ControllerClassType::PS5Wireless_AndroidAPIBefore31;
		DeviceCfg->SupportsHat = true;
		DeviceCfg->RightStickZRZ = true;
		DeviceCfg->MapRXRYToTriggers = true;
		DeviceCfg->ButtonMapping = ButtonMappings[4];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("PS5 Wireless Controller"));
		RegisterDeviceConfig(0x054c, 0x0ce6, DeviceCfg); // Add to the special device config map

		// glap QXPGP001
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ButtonMapping = ButtonMappings[0];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("glap QXPGP001"));
		RegisterDeviceConfig(0x05ac, 0x056a, DeviceCfg);

		// STMicroelectronics Lenovo GamePad
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ButtonMapping = ButtonMappings[0];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("STMicroelectronics Lenovo GamePad"));
		RegisterDeviceConfig(0x0483, 0x5750, DeviceCfg);
		
		// Razer Kishi V2 Pro XBox360 - Android API >= 31
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ControllerClass = ControllerClassType::XBoxWired;
		DeviceCfg->SupportsHat = true;
		DeviceCfg->TriggersUseThresholdForClick = true;
		DeviceCfg->ButtonMapping = ButtonMappings[0];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("Razer Kishi V2 Pro XBox360"));
		RegisterDeviceConfig(0x1532, 0x0037, DeviceCfg);
		RegisterDeviceConfig(0x1532, 0x0718, DeviceCfg);

		// Razer Kishi V2 Pro XBox360 - Android API < 31
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ControllerClass = ControllerClassType::XBoxWired_AndroidAPIBefore31;
		DeviceCfg->SupportsHat = true;
		DeviceCfg->TriggersUseThresholdForClick = true;
		DeviceCfg->RightStickZRZ = false;
		DeviceCfg->RightStickRXRY = true;
		DeviceCfg->MapZRZToTriggers = true;
		DeviceCfg->LTAnalogRangeMinimum = -1.0f;
		DeviceCfg->RTAnalogRangeMinimum = -1.0f;
		DeviceCfg->ButtonMapping = ButtonMappings[0];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("Razer Kishi V2 Pro XBox360"));
		RegisterDeviceConfig(0x1532, 0x0037, DeviceCfg);// Add to the special device config map
		RegisterDeviceConfig(0x1532, 0x0718, DeviceCfg);// Add to the special device config map

		// Razer Kishi V2
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ControllerClass = ControllerClassType::XBoxWired;
		DeviceCfg->SupportsHat = true;
		DeviceCfg->TriggersUseThresholdForClick = true;
		DeviceCfg->ButtonMapping = ButtonMappings[0];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("Razer Kishi V2"));
		RegisterDeviceConfig(0x1532, 0x0712, DeviceCfg);

		// Razer
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->ButtonMapping = ButtonMappings[0];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("Razer"));
		RegisterDeviceConfig(0x1532, -1, DeviceCfg);

		// Luna
		DeviceCfg = MakeShared<DeviceConfig>();
		DeviceCfg->TriggersUseThresholdForClick = true;
		DeviceCfg->ButtonMapping = ButtonMappings[0];
		DeviceCfg->DeviceNameIndex = DeviceNames.Add(TEXT("Luna"));
		RegisterDeviceConfig(0x0171, 0x0419, DeviceCfg);

		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidGameControllerMapper::Init - Layouts: %d, DeviceConfig: %d, SpecialConfig: %d, DeviceNames: %d"), 
			ButtonMappings.Num(), DeviceConfigurationMap.Num(), DeviceConfigurationMap_Special.Num(), DeviceNames.Num());

		return true;
	}

	bool FAndroidGameControllerMapper::InitByDataFile()
	{
		return false;
	}

	const DeviceConfig_WPtr FAndroidGameControllerMapper::GetDeviceConfig(int32 VendorId, int32 ProductId, const FName DeviceName)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidGameControllerMapper::GetDeviceConfig - Vendor: 0x%08x, Product: 0x%08x, Name: %s"), VendorId, ProductId, *(DeviceName.ToString()));

#if USE_ANDROID_JNI
		if (VendorId == 0x054c && ProductId == 0x09cc && FAndroidMisc::GetCPUVendor() != TEXT("Sony") && FAndroidMisc::GetAndroidBuildVersion() < 10)
		{
			const DeviceConfig_SRef* Config = DeviceConfigurationMap_Special.Find(PackDeviceKey(VendorId, ProductId));
			if (Config != nullptr)
			{
				check((*Config)->ControllerClass == ControllerClassType::PS4Wireless_AndroidAPIBefore10_CPUNotSony);
				return Config->ToWeakPtr();
			}
		}
		else if (VendorId == 0x054c && ProductId == 0x0ce6 && FAndroidMisc::GetAndroidBuildVersion() < 31)
		{
			const DeviceConfig_SRef* Config = DeviceConfigurationMap_Special.Find(PackDeviceKey(VendorId, ProductId));
			if (Config != nullptr)
			{
				check((*Config)->ControllerClass == ControllerClassType::PS5Wireless_AndroidAPIBefore31);
				return Config->ToWeakPtr();
			}
		}
		else
#endif
		{
			const DeviceConfig_SRef* Config = DeviceConfigurationMap.Find(PackDeviceKey(VendorId, ProductId));
			if (Config != nullptr)
			{
				return Config->ToWeakPtr();
			}
		}

		// fall back to a query by the device name
		DeviceConfig_SPtr ConfigFoundByName = GetDeviceConfig(DeviceName);
		if (ConfigFoundByName != nullptr)
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Warning: FAndroidGameControllerMapper::GetDeviceConfig - Returns the mapping found by name."));
			return ConfigFoundByName.ToWeakPtr();
		}

		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Warning: FAndroidGameControllerMapper::GetDeviceConfig - Returns default mapping."));
		
		return DeviceConfigurationMap[PackDeviceKey(-1, -1)].ToWeakPtr();
	}

	const DeviceConfig_SPtr FAndroidGameControllerMapper::GetDeviceConfig(const FName InDeviceName)
	{
		FString DeviceName = InDeviceName.ToString();

		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidGameControllerMapper::GetDeviceConfig - Fall back to query by name, Name: %s"), *DeviceName);

		int32 NameIndexFound = -1;
		for (int i = 0; i < DeviceNames.Num(); ++i)
		{
			FString NameInStock = DeviceNames[i].ToString();
			if (DeviceName.StartsWith(*NameInStock))
			{
				NameIndexFound = i;
				break;
			}
		}

		if (NameIndexFound < 0)
		{
			return nullptr;
		}

		TMap<uint64, DeviceConfig_SRef>& ConfigMap_Ref = DeviceConfigurationMap;
		if (GAndroidOldXBoxWirelessFirmware
#if USE_ANDROID_JNI
			|| (FAndroidMisc::GetCPUVendor() != TEXT("Sony") && FAndroidMisc::GetAndroidBuildVersion() < 10) ||
			(FAndroidMisc::GetAndroidBuildVersion() < 31)
#endif
			)
		{
			ConfigMap_Ref = DeviceConfigurationMap_Special;
		}

		uint64 ConfigKey = 0;
		for (const TPair<uint64, DeviceConfig_SRef>& Config_KV : ConfigMap_Ref)
		{
			if (Config_KV.Value.Get().DeviceNameIndex == NameIndexFound)
			{
				ConfigKey = Config_KV.Key;
				break;
			}
		}

		return ConfigKey == 0 ? nullptr : ConfigMap_Ref[ConfigKey].ToSharedPtr();
	}

	void FAndroidGameControllerMapper::RegisterDeviceConfig(int32 VendorId, int32 ProductId, DeviceConfig_SRef& DeviceCfg)
	{
		int64 DeviceKey = PackDeviceKey(VendorId, ProductId);

		TMap<uint64, DeviceConfig_SRef>& ConfigMap_Ref = 
			DeviceCfg->ControllerClass < ControllerClassType::XBoxWired_AndroidAPIBefore31 ? DeviceConfigurationMap : DeviceConfigurationMap_Special;

		if (ConfigMap_Ref.Contains(DeviceKey))
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Warning: FAndroidGameControllerMapper::RegisterDeviceConfig - Existed device config will be replaced. Vendor: 0x%08x, Product: 0x%08x"), VendorId, ProductId);
		}

		ConfigMap_Ref.Add(DeviceKey, DeviceCfg);
	}

	void FAndroidGameControllerMapper::AppendKeyCodeKeyNameMapping(KeyCodeToKeyNameMap_SRef& TargetContainer, int32 KeyCode, int32 KeyNameIndex)
	{
		TArray<int32>* TargetIndexArray = TargetContainer->Find(KeyCode);
		if (TargetIndexArray == nullptr)
		{
			(*TargetContainer).Add(KeyCode, { KeyNameIndex });
		}
		else
		{
			TargetIndexArray->Add(KeyNameIndex);
		}
	}
}
