// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidInputInterface.h"
#include "Android/AndroidJNI.h"
#include "Android/AndroidApplication.h"
#if USE_ANDROID_INPUT
#include "Android/AndroidEventManager.h"
#include "Android/AndroidJniGameActivity.h"
#include "Android/AndroidJniGameControllerManager.h"
//#include "AndroidInputDeviceMappings.h"
#include "Misc/ConfigCacheIni.h"
#include "IInputDevice.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "HAL/ThreadingBase.h"
#include "Misc/CallbackDevice.h"
#include "HAL/PlatformTime.h"
#include "HAL/IConsoleManager.h"
#include "IHapticDevice.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "GenericPlatform/InputDeviceRegistry.h"
#include "Containers/Ticker.h"

#ifndef ANDROID_GAMEPAD_TRIGGER_THRESHOLD
	#define ANDROID_GAMEPAD_TRIGGER_THRESHOLD	0.30f
#endif

TArray<TouchInput> FAndroidInputInterface::TouchInputStack = TArray<TouchInput>();
FCriticalSection FAndroidInputInterface::TouchInputCriticalSection;

TInputDeviceMap<FString> FAndroidInputInterface::InternalDeviceIdMappings;
TMap<int32, FAndroidInputDeviceInfo> FAndroidInputInterface::InputDeviceInfoMap;
TMap<FInputDeviceId, FAndroidGamepadDeviceMapping> FAndroidInputInterface::GameControllerDataMap;
TMap<FInputDeviceId, TWeakPtr<FGenericWindow>> FAndroidInputInterface::InputMappedSurfaces;
TArray<FInputDeviceId> FAndroidInputInterface::FreeSurfaceInputDeviceIds;
TArray<int32> FAndroidInputInterface::GameControllerIdMapping;

FName FAndroidInputInterface::InputClassName_DefaultMobileTouch;
FName FAndroidInputInterface::InputClassName_DefaultGamepad;
FString FAndroidInputInterface::HardwareDeviceIdentifier_DefaultMobileTouch;
FString FAndroidInputInterface::HardwareDeviceIdentifier_DefaultGamepad;

int32 FAndroidInputInterface::CurrentVibeIntensity;
int32 FAndroidInputInterface::MaxVibeTime = 1000;
double FAndroidInputInterface::LastVibeUpdateTime = 0.0;
FForceFeedbackValues FAndroidInputInterface::VibeValues;

bool FAndroidInputInterface::bAllowControllers = true;
bool FAndroidInputInterface::bBlockAndroidKeysOnControllers = false;
bool FAndroidInputInterface::bControllersBlockDeviceFeedback = false;

FGamepadKeyNames::Type FAndroidInputInterface::ButtonMapping[MAX_NUM_CONTROLLER_BUTTONS];
float FAndroidInputInterface::InitialButtonRepeatDelay;
float FAndroidInputInterface::ButtonRepeatDelay;

FDeferredAndroidMessage FAndroidInputInterface::DeferredMessages[MAX_DEFERRED_MESSAGE_QUEUE_SIZE];
int32 FAndroidInputInterface::DeferredMessageQueueLastEntryIndex = 0;
int32 FAndroidInputInterface::DeferredMessageQueueDroppedCount   = 0;

TArray<FAndroidInputInterface::MotionData> FAndroidInputInterface::MotionDataStack
	= TArray<FAndroidInputInterface::MotionData>();

TArray<FAndroidInputInterface::MouseData> FAndroidInputInterface::MouseDataStack
	= TArray<FAndroidInputInterface::MouseData>();

float GAndroidVibrationThreshold = 0.3f;
static FAutoConsoleVariableRef CVarAndroidVibrationThreshold(
	TEXT("Android.VibrationThreshold"),
	GAndroidVibrationThreshold,
	TEXT("If set above 0.0 acts as on/off threshold for device vibrator (Default: 0.3)"),
	ECVF_Default);

int32 GAndroidUseControllerFeedback = 1;
static FAutoConsoleVariableRef CVarAndroidUseControllerFeedback(
	TEXT("Android.UseControllerFeedback"),
	GAndroidUseControllerFeedback,
	TEXT("If set to non-zero, controllers with force feedback support will be active (Default: 1)"),
	ECVF_Default);

int32 GAndroidOldXBoxWirelessFirmware = 0;
static FAutoConsoleVariableRef CVarAndroidOldXBoxWirelessFirmware(
	TEXT("Android.OldXBoxWirelessFirmware"),
	GAndroidOldXBoxWirelessFirmware,
	TEXT("Determines how XBox Wireless controller mapping is handled. 0 assumes new firmware, 1 will use old firmware mapping (Default: 0)"),
	ECVF_Default);

int32 AndroidUnifyMotionSpace = 1;
static FAutoConsoleVariableRef CVarAndroidUnifyMotionSpace(
	TEXT("Android.UnifyMotionSpace"),
	AndroidUnifyMotionSpace,
	TEXT("If set to non-zero, acceleration, gravity, and rotation rate will all be in the same coordinate space. 0 for legacy behaviour. 1 (default as of 5.5) will match Unreal's coordinate space (left-handed, z-up, etc). 2 will be right-handed by swapping x and y. Non-zero also forces rotation rate units to be radians/s and acceleration units to be g."),
	ECVF_Default);

bool AndroidEnableInputDeviceListener = true;
static FAutoConsoleVariableRef CVarAndroidEnableInputDeviceListener(
	TEXT("Android.EnableInputDeviceListener"),
	AndroidEnableInputDeviceListener,
	TEXT("Determines how to detect gamepad connection/disconnection. true for using InputDeviceListener. false for using the gamepad input events (Default: true)"),
	ECVF_Default);

using namespace GameControllerMapping;

bool AndroidThunkCpp_InitGameControllerManager()
{
	using namespace UE::Jni;

	bool bGCMCreated = FGameActivity::createGameControllerManager(FGameActivity::Get());
	if (bGCMCreated)
	{
		FScopedJavaObject<FGameControllerManager*> GameControllerManagerObj = FGameActivity::gameControllerManager.Get(FGameActivity::Get());
		if (GameControllerManagerObj)
		{
			FGameControllerManager::scanDevices(*GameControllerManagerObj);
			return true;
		}
	}

	return false;
}

void JNICALL UE::Jni::FGameControllerManager::nativeOnInputDeviceStateEvent(JNIEnv* env, jobject thiz, jint device_id, jint state, jint type)
{
	InputDeviceStateEvent StateEvent = static_cast<InputDeviceStateEvent>(state);
	InputDeviceType DeviceType = static_cast<InputDeviceType>(type);
	FAndroidInputInterface::HandleInputDeviceStateEvent(device_id, StateEvent, DeviceType);
}

void FAndroidGamepadDeviceMapping::Init(const int32 InVendorId, const int32 InProductId, const FName DeviceName)
{
	TSharedPtr<FAndroidGameControllerMapper> ControllerMapper = FAndroidGameControllerMapper::GetInstance().Pin();
	check(ControllerMapper.IsValid());

	DeviceConfig = ControllerMapper->GetDeviceConfig(InVendorId, InProductId, DeviceName);

	ResetRuntimeData();
}

TSharedRef< FAndroidInputInterface > FAndroidInputInterface::Create(const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler, const TSharedPtr< ICursor >& InCursor)
{
	return MakeShareable(new FAndroidInputInterface(InMessageHandler, InCursor));
}

FAndroidInputInterface::~FAndroidInputInterface()
{
}

namespace AndroidKeyNames
{
	const FGamepadKeyNames::Type Android_Back("Android_Back");
	const FGamepadKeyNames::Type Android_Menu("Android_Menu");
}

FAndroidInputInterface::FAndroidInputInterface(const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler, const TSharedPtr< ICursor >& InCursor)
	: MessageHandler( InMessageHandler )
	, Cursor(InCursor)
{
	GConfig->GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bAllowControllers"), bAllowControllers, GEngineIni);
	GConfig->GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bBlockAndroidKeysOnControllers"), bBlockAndroidKeysOnControllers, GEngineIni);
	GConfig->GetBool(TEXT("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings"), TEXT("bControllersBlockDeviceFeedback"), bControllersBlockDeviceFeedback, GEngineIni);

	ButtonMapping[0] = FGamepadKeyNames::FaceButtonBottom;
	ButtonMapping[1] = FGamepadKeyNames::FaceButtonRight;
	ButtonMapping[2] = FGamepadKeyNames::FaceButtonLeft;
	ButtonMapping[3] = FGamepadKeyNames::FaceButtonTop;
	ButtonMapping[4] = FGamepadKeyNames::LeftShoulder;
	ButtonMapping[5] = FGamepadKeyNames::RightShoulder;
	ButtonMapping[6] = FGamepadKeyNames::SpecialRight;
	ButtonMapping[7] = FGamepadKeyNames::SpecialLeft;
	ButtonMapping[8] = FGamepadKeyNames::LeftThumb;
	ButtonMapping[9] = FGamepadKeyNames::RightThumb;
	ButtonMapping[10] = FGamepadKeyNames::LeftTriggerThreshold;
	ButtonMapping[11] = FGamepadKeyNames::RightTriggerThreshold;
	ButtonMapping[12] = FGamepadKeyNames::DPadUp;
	ButtonMapping[13] = FGamepadKeyNames::DPadDown;
	ButtonMapping[14] = FGamepadKeyNames::DPadLeft;
	ButtonMapping[15] = FGamepadKeyNames::DPadRight;
	ButtonMapping[16] = AndroidKeyNames::Android_Back;  // Technically just an alias for SpecialLeft
	ButtonMapping[17] = AndroidKeyNames::Android_Menu;  // Technically just an alias for SpecialRight

	// Virtual buttons
	ButtonMapping[MAX_NUM_PHYSICAL_CONTROLLER_BUTTONS + 0] = FGamepadKeyNames::LeftStickLeft;
	ButtonMapping[MAX_NUM_PHYSICAL_CONTROLLER_BUTTONS + 1] = FGamepadKeyNames::LeftStickRight;
	ButtonMapping[MAX_NUM_PHYSICAL_CONTROLLER_BUTTONS + 2] = FGamepadKeyNames::LeftStickUp;
	ButtonMapping[MAX_NUM_PHYSICAL_CONTROLLER_BUTTONS + 3] = FGamepadKeyNames::LeftStickDown;
	ButtonMapping[MAX_NUM_PHYSICAL_CONTROLLER_BUTTONS + 4] = FGamepadKeyNames::RightStickLeft;
	ButtonMapping[MAX_NUM_PHYSICAL_CONTROLLER_BUTTONS + 5] = FGamepadKeyNames::RightStickRight;
	ButtonMapping[MAX_NUM_PHYSICAL_CONTROLLER_BUTTONS + 6] = FGamepadKeyNames::RightStickUp;
	ButtonMapping[MAX_NUM_PHYSICAL_CONTROLLER_BUTTONS + 7] = FGamepadKeyNames::RightStickDown;

	InitialButtonRepeatDelay = 0.2f;
	ButtonRepeatDelay = 0.1f;

	GConfig->GetFloat(TEXT("/Script/Engine.InputSettings"), TEXT("InitialButtonRepeatDelay"), InitialButtonRepeatDelay, GInputIni);
	GConfig->GetFloat(TEXT("/Script/Engine.InputSettings"), TEXT("ButtonRepeatDelay"), ButtonRepeatDelay, GInputIni);

	CurrentVibeIntensity = 0;
	FMemory::Memset(VibeValues, 0);
	
	InputDeviceInfoMap.Empty();
	GameControllerDataMap.Empty();
	GameControllerIdMapping.Empty();

	InputClassName_DefaultMobileTouch = TEXT("DefaultMobileTouch");
	InputClassName_DefaultGamepad = TEXT("DefaultGamepad");
	HardwareDeviceIdentifier_DefaultMobileTouch = TEXT("MobileTouch");
	HardwareDeviceIdentifier_DefaultGamepad = TEXT("Gamepad");

	if (AndroidEnableInputDeviceListener)
	{
		extern bool AndroidThunkCpp_InitGameControllerManager();
		if (!AndroidThunkCpp_InitGameControllerManager())
		{
			UE_LOGF(LogAndroid, Error, "GameControllerManager initialization failed!");
		}
	}
}

// all game controller data will be kept for future reconnections
void FAndroidInputInterface::ResetGamepadAssignments()
{
	TArray<int32> GameControllerToRemove;
	for (const TPair<int32, FAndroidInputDeviceInfo>& KVPair : InputDeviceInfoMap)
	{
		const FAndroidInputDeviceInfo& DeviceInfo = KVPair.Value;
		if (DeviceInfo.DeviceType == InputDeviceType::GameController)
		{
			if ((DeviceInfo.DeviceState == MappingState::Valid))
			{
				MapControllerToPlayer(DeviceInfo.Descriptor, EInputDeviceConnectionState::Disconnected);
			}
			GameControllerToRemove.Push(KVPair.Key);
		}
	}

	for (int32 DeviceId : GameControllerToRemove)
	{
		InputDeviceInfoMap.Remove(DeviceId);
	}
}

void FAndroidInputInterface::ResetGamepadAssignmentToController(int32 ControllerId)
{
	FAndroidInputDeviceInfo* DeviceInfo = nullptr;
	if (GetInputDeviceByControllerId(ControllerId, &DeviceInfo))
	{
		if (DeviceInfo->DeviceState == MappingState::Valid)
		{
			MapControllerToPlayer(DeviceInfo->Descriptor, EInputDeviceConnectionState::Disconnected);
		}

		InputDeviceInfoMap.Remove(DeviceInfo->DeviceId);
	}
}

bool FAndroidInputInterface::IsControllerAssignedToGamepad(int32 ControllerId)
{
	FAndroidInputDeviceInfo* DeviceInfo = nullptr;
	if (GetInputDeviceByControllerId(ControllerId, &DeviceInfo))
	{
		MappingState PotentState = AndroidEnableInputDeviceListener ? MappingState::ToValidate : MappingState::Valid;
		return (DeviceInfo->DeviceState >= PotentState);
	}
	return false;
}

const FInputDeviceId FAndroidInputInterface::GetMappedInputDeviceId(int32 ControllerId)
{
	FAndroidInputDeviceInfo* DeviceInfo = nullptr;
	if (GetInputDeviceByControllerId(ControllerId, &DeviceInfo))
	{
		MappingState PotentState = AndroidEnableInputDeviceListener ? MappingState::ToValidate : MappingState::Valid;
		if (DeviceInfo->DeviceState >= PotentState)
		{
			return InternalDeviceIdMappings.FindDeviceId(DeviceInfo->Descriptor);
		}
	}

	return INPUTDEVICEID_NONE;
}

const FName FAndroidInputInterface::GetGamepadControllerName(int32 ControllerId)
{
	FAndroidInputDeviceInfo* DeviceInfo = nullptr;
	if (GetInputDeviceByControllerId(ControllerId, &DeviceInfo))
	{
		MappingState PotentState = AndroidEnableInputDeviceListener ? MappingState::ToValidate : MappingState::Valid;
		if (DeviceInfo->DeviceState >= PotentState)
		{
			return DeviceInfo->Name;
		}
	}
	return InputClassName_DefaultGamepad;
}

void FAndroidInputInterface::SetMessageHandler( const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler )
{
	MessageHandler = InMessageHandler;

	for (auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt)
	{
		(*DeviceIt)->SetMessageHandler(InMessageHandler);
	}
}

void FAndroidInputInterface::AddExternalInputDevice(TSharedPtr<IInputDevice> InputDevice)
{
	if (InputDevice.IsValid())
	{
		ExternalInputDevices.Add(InputDevice);
	}
}

void FAndroidInputInterface::MapControllerToPlayer(const FString& ControllerDescriptor, EInputDeviceConnectionState State)
{
	IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();

	FInputDeviceId DeviceId = InternalDeviceIdMappings.GetOrCreateDeviceId(ControllerDescriptor);
	check(DeviceId != INPUTDEVICEID_NONE);

	FPlatformUserId PlatformUserId = PLATFORMUSERID_NONE;
	if (State == EInputDeviceConnectionState::Connected)
	{
		PlatformUserId = DeviceMapper.GetUserForInputDevice(DeviceId);
		if (PlatformUserId == PLATFORMUSERID_NONE)
		{
			PlatformUserId = DeviceMapper.GetPlatformUserForNewlyConnectedDevice();
		}

		// Find the device name from InputDeviceInfoMap by matching descriptor.
		FName DeviceName = InputClassName_DefaultGamepad;
		for (const TPair<int32, FAndroidInputDeviceInfo>& Pair : InputDeviceInfoMap)
		{
			if (Pair.Value.Descriptor == ControllerDescriptor)
			{
				if (!Pair.Value.Name.IsNone())
				{
					DeviceName = Pair.Value.Name;
				}
				break;
			}
		}

		FInputDeviceDescriptor Desc;
		Desc.InputDeviceName          = DeviceName;
		Desc.HardwareDeviceHandle     = DeviceId;
		Desc.HardwareDeviceIdentifier = FName(*ControllerDescriptor);
		FInputDeviceRegistry::RegisterDevice(Desc);
	}
	else if (State == EInputDeviceConnectionState::Disconnected)
	{
		PlatformUserId = DeviceMapper.GetUserForInputDevice(DeviceId);
	}
	check(PlatformUserId != PLATFORMUSERID_NONE);

	DeviceMapper.Internal_MapInputDeviceToUser(DeviceId, PlatformUserId, State);
}

extern bool AndroidThunkCpp_GetInputDeviceInfo(int32 deviceId, FAndroidInputDeviceInfo &results);

void FAndroidInputInterface::Tick(float DeltaTime)
{
	for (auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt)
	{
		(*DeviceIt)->Tick(DeltaTime);
	}
}

void FAndroidInputInterface::SetLightColor(int32 ControllerId, FColor Color)
{
	for (auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt)
	{
		(*DeviceIt)->SetLightColor(ControllerId, Color);
	}
}

void FAndroidInputInterface::ResetLightColor(int32 ControllerId)
{
	for (auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt)
	{
		(*DeviceIt)->ResetLightColor(ControllerId);
	}
}

void FAndroidInputInterface::SetForceFeedbackChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value)
{
	bool bDidFeedback = false;
	for (auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt)
	{
		if ((*DeviceIt)->SupportsForceFeedback(ControllerId))
		{
			bDidFeedback = true;
			(*DeviceIt)->SetChannelValue(ControllerId, ChannelType, Value);
		}
	}

	// If didn't already assign feedback and active controller has feedback support use it, if enabled
	FAndroidInputDeviceInfo* DeviceInfo = nullptr;
	FAndroidGamepadDeviceMapping* DeviceData = nullptr;
	bool Succeeded = GetInputDeviceByControllerId(ControllerId, &DeviceInfo, &DeviceData);
	if (Succeeded && !bDidFeedback &&
		IsControllerAssignedToGamepad(ControllerId) &&
		GAndroidUseControllerFeedback != 0 &&
		DeviceInfo->FeedbackMotorCount > 0)
	{
		switch (ChannelType)
		{
			case FForceFeedbackChannelType::LEFT_LARGE:
				DeviceData->ControllerVibeState.VibeValues.LeftLarge = Value;
				break;

			case FForceFeedbackChannelType::LEFT_SMALL:
				DeviceData->ControllerVibeState.VibeValues.LeftSmall = Value;
				break;

			case FForceFeedbackChannelType::RIGHT_LARGE:
				DeviceData->ControllerVibeState.VibeValues.RightLarge = Value;
				break;

			case FForceFeedbackChannelType::RIGHT_SMALL:
				DeviceData->ControllerVibeState.VibeValues.RightSmall = Value;
				break;

			default:
				// Unknown channel, so ignore it
			break;
		}
		bDidFeedback = true;
	}

	bDidFeedback |= IsGamepadAttached() && bControllersBlockDeviceFeedback;

	// If controller handled force feedback don't do it on the phone
	if (bDidFeedback)
	{
		VibeValues.LeftLarge = VibeValues.RightLarge = VibeValues.LeftSmall = VibeValues.RightSmall = 0.0f;
		return;
	}

	// Note: only one motor on Android at the moment, but remember all the settings
	// update will look at combination of all values to pick state

	// Save a copy of the value for future comparison
	switch (ChannelType)
	{
		case FForceFeedbackChannelType::LEFT_LARGE:
			VibeValues.LeftLarge = Value;
			break;

		case FForceFeedbackChannelType::LEFT_SMALL:
			VibeValues.LeftSmall = Value;
			break;

		case FForceFeedbackChannelType::RIGHT_LARGE:
			VibeValues.RightLarge = Value;
			break;

		case FForceFeedbackChannelType::RIGHT_SMALL:
			VibeValues.RightSmall = Value;
			break;

		default:
			// Unknown channel, so ignore it
			break;
	}
}

void FAndroidInputInterface::SetForceFeedbackChannelValues(int32 ControllerId, const FForceFeedbackValues &Values)
{
	bool bDidFeedback = false;
	for (auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt)
	{
		if ((*DeviceIt)->SupportsForceFeedback(ControllerId))
		{
			bDidFeedback = true;
			(*DeviceIt)->SetChannelValues(ControllerId, Values);
		}
	}

	// If didn't already assign feedback and active controller has feedback support use it, if enabled
	FAndroidInputDeviceInfo* DeviceInfo = nullptr;
	FAndroidGamepadDeviceMapping* DeviceData = nullptr;
	bool Succeeded = GetInputDeviceByControllerId(ControllerId, &DeviceInfo, &DeviceData);
	if (Succeeded && !bDidFeedback && 
		IsControllerAssignedToGamepad(ControllerId) &&
		GAndroidUseControllerFeedback != 0 &&
		DeviceInfo->FeedbackMotorCount > 0)
	{
		DeviceData->ControllerVibeState.VibeValues = Values;
		bDidFeedback = true;
	}

	bDidFeedback |= IsGamepadAttached() && bControllersBlockDeviceFeedback;

	// If controller handled force feedback don't do it on the phone
	if (bDidFeedback)
	{
		VibeValues.LeftLarge = VibeValues.RightLarge = VibeValues.LeftSmall = VibeValues.RightSmall = 0.0f;
	}
	else
	{
		VibeValues = Values;
	}
}

void FAndroidInputInterface::SetHapticFeedbackValues(int32 ControllerId, int32 Hand, const FHapticFeedbackValues& Values)
{
	for (auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt)
	{
		IHapticDevice* HapticDevice = (*DeviceIt)->GetHapticDevice();
		if (HapticDevice)
		{
			HapticDevice->SetHapticFeedbackValues(ControllerId, Hand, Values);
		}
	}
}

extern bool AndroidThunkCpp_IsGamepadAttached();

bool FAndroidInputInterface::IsGamepadAttached() const
{
	// Check for gamepads that have already been validated
	for (TPair<int32, FAndroidInputDeviceInfo> KVP : InputDeviceInfoMap)
	{
		const FAndroidInputDeviceInfo& CurrentDevice = KVP.Value;

		if ((CurrentDevice.DeviceType == InputDeviceType::GameController) &&
			(CurrentDevice.DeviceState == MappingState::Valid))
		{
			return true;
		}
	}

	for (auto DeviceIt = ExternalInputDevices.CreateConstIterator(); DeviceIt; ++DeviceIt)
	{
		if ((*DeviceIt)->IsGamepadAttached())
		{
			return true;
		}
	}

	//if all of this fails, do a check on the Java side to see if the gamepad is attached
	return AndroidThunkCpp_IsGamepadAttached();
}

static FORCEINLINE int32 ConvertToByte(float Value)
{
	int32 Setting = (int32)(Value * 255.0f);
	return Setting < 0 ? 0 : (Setting < 255 ? Setting : 255);
}

extern void AndroidThunkCpp_Vibrate(int32 Intensity, int32 Duration);
extern bool AndroidThunkCpp_SetInputDeviceVibrators(int32 deviceId, int32 leftIntensity, int32 leftDuration, int32 rightIntensity, int32 rightDuration);

void FAndroidInputInterface::UpdateVibeMotors()
{
	// Turn off vibe if not in focus
	bool bActive = CurrentVibeIntensity > 0;
	if (!FAppEventManager::GetInstance()->IsGameInFocus())
	{
		if (bActive)
		{
			AndroidThunkCpp_Vibrate(0, MaxVibeTime);
			CurrentVibeIntensity = 0;
		}
		return;
	}

	// Use largest vibration state as value
	const float MaxLeft = VibeValues.LeftLarge > VibeValues.LeftSmall ? VibeValues.LeftLarge : VibeValues.LeftSmall;
	const float MaxRight = VibeValues.RightLarge > VibeValues.RightSmall ? VibeValues.RightLarge : VibeValues.RightSmall;
	float Value = MaxLeft > MaxRight ? MaxLeft : MaxRight;

	// apply optional threshold for old behavior
	if (GAndroidVibrationThreshold > 0.0f)
	{
		Value = Value < GAndroidVibrationThreshold ? 0.0f : 1.0f;
	}

	int32 Intensity = ConvertToByte(Value);

	// if previously active and overtime, current state is off
	double CurrentTime = FPlatformTime::Seconds();
	bool bOvertime = 1000 * (CurrentTime - LastVibeUpdateTime) >= MaxVibeTime;
	if (bActive && bOvertime)
	{
		CurrentVibeIntensity = 0;
	}

	// update if not already active at same level
	if (CurrentVibeIntensity != Intensity)
	{
		AndroidThunkCpp_Vibrate(Intensity, MaxVibeTime);
		CurrentVibeIntensity = Intensity;
		LastVibeUpdateTime = CurrentTime;
		//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("VibDevice %f: %d"), (float)LastVibeUpdateTime, Intensity);
	}
}

void FAndroidInputInterface::UpdateControllerVibeMotors(int32 DeviceId, ControllerClassType ControllerClass, FAndroidControllerVibeState& State)
{
	// Turn off vibe if not in focus
	bool bActive = State.LeftIntensity > 0 || State.RightIntensity > 0;
	if (!FAppEventManager::GetInstance()->IsGameInFocus())
	{
		if (bActive)
		{
			AndroidThunkCpp_SetInputDeviceVibrators(DeviceId, 0, MaxVibeTime, 0, MaxVibeTime);
			State.LeftIntensity = 0;
			State.RightIntensity = 0;
		}
		return;
	}

	float MaxLeft;
	float MaxRight;

	// Use largest vibration state as value for controller type
	switch (ControllerClass)
	{
		case ControllerClassType::PlaystationWireless:
//			DS4 maybe should use this?  PS5 seems correct with generic
//			MaxLeft = (State.VibeValues.LeftLarge > State.VibeValues.RightLarge ? State.VibeValues.LeftLarge : State.VibeValues.RightLarge);
//			MaxRight = (State.VibeValues.LeftSmall > State.VibeValues.RightSmall ? State.VibeValues.LeftSmall : State.VibeValues.RightSmall);
//			break;

		case ControllerClassType::Generic:
		case ControllerClassType::XBoxWired:
		case ControllerClassType::XBoxWireless:
		default:
			MaxLeft = (State.VibeValues.LeftLarge > State.VibeValues.LeftSmall ? State.VibeValues.LeftLarge : State.VibeValues.LeftSmall);
			MaxRight = (State.VibeValues.RightLarge > State.VibeValues.RightSmall ? State.VibeValues.RightLarge : State.VibeValues.RightSmall);
			break;
	}

	int32 LeftIntensity = ConvertToByte(MaxLeft);
	int32 RightIntensity = ConvertToByte(MaxRight);

	// if previously active and overtime, current state is off
	double CurrentTime = FPlatformTime::Seconds();
	bool bOvertime = 1000 * (CurrentTime - State.LastVibeUpdateTime) >= MaxVibeTime;
	if (bActive && bOvertime)
	{
		State.LeftIntensity = 0;
		State.RightIntensity = 0;
	}

	// update if not already active at same level
	if (State.LeftIntensity != LeftIntensity || State.RightIntensity != RightIntensity)
	{
		AndroidThunkCpp_SetInputDeviceVibrators(DeviceId, LeftIntensity, MaxVibeTime, RightIntensity, MaxVibeTime);
		State.LeftIntensity = LeftIntensity;
		State.RightIntensity = RightIntensity;
		State.LastVibeUpdateTime = CurrentTime;
		//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("VibController %f: %d, %d"), (float)State.LastVibeUpdateTime, LeftIntensity, RightIntensity);
	}
}

static TCHAR CharMap[] =
{
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	L'0',
	L'1',
	L'2',
	L'3',
	L'4',
	L'5',
	L'6',
	L'7',
	L'8',
	L'9',
	L'*',
	L'#',
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	L'a',
	L'b',
	L'c',
	L'd',
	L'e',
	L'f',
	L'g',
	L'h',
	L'i',
	L'j',
	L'k',
	L'l',
	L'm',
	L'n',
	L'o',
	L'p',
	L'q',
	L'r',
	L's',
	L't',
	L'u',
	L'v',
	L'w',
	L'x',
	L'y',
	L'z',
	L',',
	L'.',
	0,
	0,
	0,
	0,
	L'\t',
	L' ',
	0,
	0,
	0,
	L'\n',
	L'\b',
	L'`',
	L'-',
	L'=',
	L'[',
	L']',
	L'\\',
	L';',
	L'\'',
	L'/',
	L'@',
	0,
	0,
	0,   // *Camera* focus
	L'+',
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	L'0',
	L'1',
	L'2',
	L'3',
	L'4',
	L'5',
	L'6',
	L'7',
	L'8',
	L'9',
	L'/',
	L'*',
	L'-',
	L'+',
	L'.',
	L',',
	L'\n',
	L'=',
	L'(',
	L')',
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0
};

static TCHAR CharMapShift[] =
{
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	L')',
	L'!',
	L'@',
	L'#',
	L'$',
	L'%',
	L'^',
	L'&',
	L'*',
	L'(',
	L'*',
	L'#',
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	L'A',
	L'B',
	L'C',
	L'D',
	L'E',
	L'F',
	L'G',
	L'H',
	L'I',
	L'J',
	L'K',
	L'L',
	L'M',
	L'N',
	L'O',
	L'P',
	L'Q',
	L'R',
	L'S',
	L'T',
	L'U',
	L'V',
	L'W',
	L'X',
	L'Y',
	L'Z',
	L'<',
	L'>',
	0,
	0,
	0,
	0,
	L'\t',
	L' ',
	0,
	0,
	0,
	L'\n',
	L'\b',
	L'~',
	L'_',
	L'+',
	L'{',
	L'}',
	L'|',
	L':',
	L'\"',
	L'?',
	L'@',
	0,
	0,
	0,   // *Camera* focus
	L'+',
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	L'0',
	L'1',
	L'2',
	L'3',
	L'4',
	L'5',
	L'6',
	L'7',
	L'8',
	L'9',
	L'/',
	L'*',
	L'-',
	L'+',
	L'.',
	L',',
	L'\n',
	L'=',
	L'(',
	L')',
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0,
	0
};

extern void AndroidThunkCpp_PushSensorEvents();

void FAndroidInputInterface::SendControllerEvents()
{
	// trigger any motion updates before the lock so they can be queued
	AndroidThunkCpp_PushSensorEvents();

	FScopeLock Lock(&TouchInputCriticalSection);

	// Update device vibe motor with latest values (only one motor so look at combination of all values to pick state)
	UpdateVibeMotors();

	// Check for gamepads needing activating/validation if enabled
	if (bAllowControllers)
	{
		MappingState StateToCheck = AndroidEnableInputDeviceListener ? MappingState::ToActivate : MappingState::ToValidate;
		MappingState StateSwitchTo = AndroidEnableInputDeviceListener ? MappingState::ToValidate : MappingState::Valid;

		TArray<int32> DeviceToRemove;
		for (TPair<int32, FAndroidInputDeviceInfo>& KVPair : InputDeviceInfoMap)
		{
			FAndroidInputDeviceInfo& CurrentDevice = KVPair.Value;
			if (CurrentDevice.DeviceState == StateToCheck)
			{
				// Query for the device type from Java side
				if (AndroidThunkCpp_GetInputDeviceInfo(CurrentDevice.DeviceId, CurrentDevice))
				{
					// Ensure for not recording duplicated devices, even though a previously assigned controller will be removed when disconnected.
					for (const TPair<int32, FAndroidInputDeviceInfo>& Pair : InputDeviceInfoMap)
					{
						if (Pair.Value.DeviceState < StateSwitchTo) // bypass all ToActivate devices
							continue;

						if (Pair.Value.Descriptor.Equals(CurrentDevice.Descriptor))
						{
							DeviceToRemove.Push(Pair.Key);
							UE_LOGF(LogAndroid, Error, "Found input device with same descriptor! DeviceId = %d, DeviceName=%ls, Descriptor=%ls",
								Pair.Key, *(Pair.Value.Name.ToString()), *(Pair.Value.Descriptor));
						}
					}

					CurrentDevice.DeviceState = StateSwitchTo;

					if (CurrentDevice.DeviceType == InputDeviceType::GameController)
					{
						FInputDeviceId InputDeviceId = InternalDeviceIdMappings.FindDeviceId(CurrentDevice.Descriptor);
						if (InputDeviceId == INPUTDEVICEID_NONE)
						{
							FAndroidGamepadDeviceMapping DeviceData(CurrentDevice.VendorId, CurrentDevice.ProductId, CurrentDevice.Name);
							InputDeviceId = InternalDeviceIdMappings.GetOrCreateDeviceId(CurrentDevice.Descriptor);
							GameControllerDataMap.Add(InputDeviceId, DeviceData);
						}
					}

					UE_LOGF(LogAndroid, Log, 
						"New input device recorded: ControllerId = %d, DeviceId=%d, ControllerType=%d, DeviceName=%ls, Descriptor=%ls, FeedbackMotorCount=%d",
						FindControllerId(CurrentDevice.DeviceId), CurrentDevice.DeviceId, CurrentDevice.DeviceType, 
						*(CurrentDevice.Name.ToString()), *CurrentDevice.Descriptor, CurrentDevice.FeedbackMotorCount);
				}
				else
				{
					// Couldn't get the device info from Java side. Discard this device.
					DeviceToRemove.Push(KVPair.Key);
					UE_LOGF(LogAndroid, Error, "Failed to assign gamepad controller %d: DeviceId=%d", FindControllerId(KVPair.Key), CurrentDevice.DeviceId);
				}
			}
		}

		if (DeviceToRemove.Num() > 0)
		{
			for (int32 DeviceId : DeviceToRemove)
			{
				InputDeviceInfoMap.Remove(DeviceId);
			}

			DumpInputDevices();
		}
	}

	for(int i = 0; i < FAndroidInputInterface::TouchInputStack.Num(); ++i)
	{
		// some special inputs has -1 as their device id (e.g. scrcpy inputs)
		// use default touchscreen instead to handle their inputs
		FInputDeviceId TouchInputDeviceId = IPlatformInputDeviceMapper::Get().GetDefaultInputDevice();
		
		FAndroidInputDeviceInfo* TouchDeviceInfo = nullptr;
		TouchInput Touch = FAndroidInputInterface::TouchInputStack[i];
		TSharedPtr<FGenericWindow> TouchWindow;
		if (Touch.InputDeviceId.IsValid())
		{
			TouchInputDeviceId = Touch.InputDeviceId;
			if (TWeakPtr<FGenericWindow>* WindowWeak = InputMappedSurfaces.Find(TouchInputDeviceId))
			{
				TouchWindow = WindowWeak->Pin();
			}
			else if (FAndroidApplication* AndroidApp = FAndroidApplication::Get())
			{
				// Fall back to the primary window
				TouchWindow = AndroidApp->GetPrimaryWindow();
			}
		}
		else if (Touch.DeviceId > 0)
		{
			TouchDeviceInfo = InputDeviceInfoMap.Find(Touch.DeviceId);
			if (TouchDeviceInfo == nullptr)
			{
				continue;
			}

			if (AndroidEnableInputDeviceListener && TouchDeviceInfo->DeviceState == MappingState::ToValidate)
			{
				TouchDeviceInfo->DeviceState = MappingState::Valid;

				if (TouchDeviceInfo->IsExternal == false)
				{
					// It's the built-in touch screen, map it to the default input device id which is already bound to the primary user id.
					FInputDeviceId InputDeviceId = InternalDeviceIdMappings.MapDefaultInputDevice(TouchDeviceInfo->Descriptor);
					check(InputDeviceId != INPUTDEVICEID_NONE);
				}
				else
				{
					MapControllerToPlayer(TouchDeviceInfo->Descriptor, EInputDeviceConnectionState::Connected);
				}

				UE_LOGF(LogAndroid, Log, "Touch Screen state changed to Valid, DeviceId = %d", Touch.DeviceId);

				DumpInputDevices();
			}

			TouchInputDeviceId = InternalDeviceIdMappings.FindDeviceId(TouchDeviceInfo->Descriptor);

			if (TouchInputDeviceId == INPUTDEVICEID_NONE)
			{
				continue;
			}
		}

		///The FInputDeviceScope::HardwareDeviceIdentifier has to be one of values in UInputPlatformSettings::HardwareDevices.
		// This is a temp solution with a hardcoded string which can be mapped to FHardwareDeviceIdentifier::DefaultMobileTouch.
		// TODO: Future improvement is needed to acquire them by values in the Android input device info.
		FName DeviceName = TouchDeviceInfo ? TouchDeviceInfo->Name : InputClassName_DefaultMobileTouch;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FInputDeviceScope InputScope(nullptr, DeviceName, TouchInputDeviceId.GetId(), HardwareDeviceIdentifier_DefaultMobileTouch);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
		FPlatformUserId PlatformUserId = DeviceMapper.GetUserForInputDevice(TouchInputDeviceId);

		// send input to handler
		FAndroidApplication* AndroidApp = FAndroidApplication::Get();
		TSharedPtr<FGenericWindow> PreviousTouchedWindow;
		if (AndroidApp)
		{
			PreviousTouchedWindow = AndroidApp->GetWindowUnderCursor();
			AndroidApp->SetCurrentlyTouchedWindow(TouchWindow);
		}

		switch ( Touch.Type )
		{
		case TouchBegan:
			MessageHandler->OnTouchStarted(TouchWindow, Touch.Position, 1.0f, Touch.Handle, PlatformUserId, TouchInputDeviceId);
			break;
		case TouchEnded:
			MessageHandler->OnTouchEnded(Touch.Position, Touch.Handle, PlatformUserId, TouchInputDeviceId);
			break;
		case TouchMoved:
			MessageHandler->OnTouchMoved(Touch.Position, 1.0f, Touch.Handle, PlatformUserId, TouchInputDeviceId);
			break;
		}

		if (AndroidApp)
		{
			AndroidApp->SetCurrentlyTouchedWindow(MoveTemp(PreviousTouchedWindow));
		}
	}

	// Extract differences in new and old states and send messages
	if (bAllowControllers)
	{
		for (TPair<int32, FAndroidInputDeviceInfo>& KVPair : InputDeviceInfoMap)
		{
			const FAndroidInputDeviceInfo& DeviceInfo = KVPair.Value;
			// Skip unassigned or invalid controllers (treat first one as special case)
			if ((DeviceInfo.DeviceState != MappingState::Valid) || (DeviceInfo.DeviceType != InputDeviceType::GameController))
			{
				continue;
			}
			
			const FInputDeviceId InputDeviceId = InternalDeviceIdMappings.FindDeviceId(DeviceInfo.Descriptor);
			check(InputDeviceId != INPUTDEVICEID_NONE);

			///The FInputDeviceScope::HardwareDeviceIdentifier has to be one of values in UInputPlatformSettings::HardwareDevices.
			// This is a temp solution with a hardcoded string which can be mapped to FHardwareDeviceIdentifier::DefaultGamepad.
			// TODO: Future improvement is needed to acquire them by values in the Android input device info.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FInputDeviceScope InputScope(nullptr, DeviceInfo.Name, InputDeviceId.GetId(), HardwareDeviceIdentifier_DefaultGamepad);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

			FAndroidGamepadDeviceMapping* DeviceData = GameControllerDataMap.Find(InputDeviceId);
			if (DeviceData == nullptr || !DeviceData->DeviceConfig.IsValid())
			{
				// The protection here is for some very rare cases, with only 1 PS5 controller connected, the OS will report
				// two game controllers and one of them is a false game controller without a name and descriptor. It will generate
				// touch inputs and thus be validated as a touch screen and not allocating corresponding device data. 
				// I only met the case when the game was running at a extremely low FPS with a debugger attached.
				continue; 
			}
			FAndroidControllerData& OldControllerState = DeviceData->OldControllerData;
			FAndroidControllerData& NewControllerState = DeviceData->NewControllerData;
			
			IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
			FPlatformUserId UserId = DeviceMapper.GetUserForInputDevice(InputDeviceId);

			// Send controller events any time we have a large enough input threshold similarly to PC/Console (see: XInputInterface.cpp)
			const float RepeatDeadzone = 0.24f;

			if (NewControllerState.LXAnalog != OldControllerState.LXAnalog || FMath::Abs(NewControllerState.LXAnalog) >= RepeatDeadzone)
			{
				MessageHandler->OnControllerAnalog(FGamepadKeyNames::LeftAnalogX, UserId, InputDeviceId, NewControllerState.LXAnalog);
				NewControllerState.ButtonStates[MAX_NUM_PHYSICAL_CONTROLLER_BUTTONS + 1] = NewControllerState.LXAnalog >= RepeatDeadzone;
				NewControllerState.ButtonStates[MAX_NUM_PHYSICAL_CONTROLLER_BUTTONS + 0] = NewControllerState.LXAnalog <= -RepeatDeadzone;
			}
			if (NewControllerState.LYAnalog != OldControllerState.LYAnalog || FMath::Abs(NewControllerState.LYAnalog) >= RepeatDeadzone)
			{
				//LOGD("    Sending updated LeftAnalogY value of %f", NewControllerState.LYAnalog);
				MessageHandler->OnControllerAnalog(FGamepadKeyNames::LeftAnalogY, UserId, InputDeviceId, NewControllerState.LYAnalog);
				NewControllerState.ButtonStates[MAX_NUM_PHYSICAL_CONTROLLER_BUTTONS + 2] = NewControllerState.LYAnalog >= RepeatDeadzone;
				NewControllerState.ButtonStates[MAX_NUM_PHYSICAL_CONTROLLER_BUTTONS + 3] = NewControllerState.LYAnalog <= -RepeatDeadzone;
			}
			if (NewControllerState.RXAnalog != OldControllerState.RXAnalog || FMath::Abs(NewControllerState.RXAnalog) >= RepeatDeadzone)
			{
				//LOGD("    Sending updated RightAnalogX value of %f", NewControllerState.RXAnalog);
				MessageHandler->OnControllerAnalog(FGamepadKeyNames::RightAnalogX, UserId, InputDeviceId, NewControllerState.RXAnalog);
				NewControllerState.ButtonStates[MAX_NUM_PHYSICAL_CONTROLLER_BUTTONS + 5] = NewControllerState.RXAnalog >= RepeatDeadzone;
				NewControllerState.ButtonStates[MAX_NUM_PHYSICAL_CONTROLLER_BUTTONS + 4] = NewControllerState.RXAnalog <= -RepeatDeadzone;
			}
			if (NewControllerState.RYAnalog != OldControllerState.RYAnalog || FMath::Abs(NewControllerState.RYAnalog) >= RepeatDeadzone)
			{
				//LOGD("    Sending updated RightAnalogY value of %f", NewControllerState.RYAnalog);
				MessageHandler->OnControllerAnalog(FGamepadKeyNames::RightAnalogY, UserId, InputDeviceId, NewControllerState.RYAnalog);
				NewControllerState.ButtonStates[MAX_NUM_PHYSICAL_CONTROLLER_BUTTONS + 6] = NewControllerState.RYAnalog >= RepeatDeadzone;
				NewControllerState.ButtonStates[MAX_NUM_PHYSICAL_CONTROLLER_BUTTONS + 7] = NewControllerState.RYAnalog <= -RepeatDeadzone;
			}
			
			DeviceConfig_SPtr DeviceConfigPtr = DeviceData->DeviceConfig.Pin();
			check(DeviceConfigPtr.IsValid());
			const bool UseTriggerThresholdForClick = DeviceConfigPtr->TriggersUseThresholdForClick;
			if (NewControllerState.LTAnalog != OldControllerState.LTAnalog)
			{
				//LOGD("    Sending updated LeftTriggerAnalog value of %f", NewControllerState.LTAnalog);
				MessageHandler->OnControllerAnalog(FGamepadKeyNames::LeftTriggerAnalog, UserId, InputDeviceId, NewControllerState.LTAnalog);

				if (UseTriggerThresholdForClick)
				{
					// Handle the trigger theshold "virtual" button state
					//check(IndexToKeyNameMap[10] == FGamepadKeyNames::LeftTriggerThreshold);
					NewControllerState.ButtonStates[10] = NewControllerState.LTAnalog >= ANDROID_GAMEPAD_TRIGGER_THRESHOLD;
				}
			}
			if (NewControllerState.RTAnalog != OldControllerState.RTAnalog)
			{
				//LOGD("    Sending updated RightTriggerAnalog value of %f", NewControllerState.RTAnalog);
				MessageHandler->OnControllerAnalog(FGamepadKeyNames::RightTriggerAnalog, UserId, InputDeviceId, NewControllerState.RTAnalog);

				if (UseTriggerThresholdForClick)
				{
					// Handle the trigger theshold "virtual" button state
					//check(IndexToKeyNameMap[11] == FGamepadKeyNames::RightTriggerThreshold);
					NewControllerState.ButtonStates[11] = NewControllerState.RTAnalog >= ANDROID_GAMEPAD_TRIGGER_THRESHOLD;
				}
			}

			const double CurrentTime = FPlatformTime::Seconds();

			// For each button check against the previous state and send the correct message if any
			for (int32 ButtonIndex = 0; ButtonIndex < MAX_NUM_CONTROLLER_BUTTONS; ButtonIndex++)
			{
				if (NewControllerState.ButtonStates[ButtonIndex] != OldControllerState.ButtonStates[ButtonIndex])
				{
					if (NewControllerState.ButtonStates[ButtonIndex])
					{
						//LOGD("    Sending joystick button down %d (first)", ButtonMapping[ButtonIndex]);
						MessageHandler->OnControllerButtonPressed(ButtonMapping[ButtonIndex], UserId, InputDeviceId, false);
					}
					else
					{
						//LOGD("    Sending joystick button up %d", ButtonMapping[ButtonIndex]);
						MessageHandler->OnControllerButtonReleased(ButtonMapping[ButtonIndex], UserId, InputDeviceId, false);
					}

					if (NewControllerState.ButtonStates[ButtonIndex])
					{
						// This button was pressed - set the button's NextRepeatTime to the InitialButtonRepeatDelay
						NewControllerState.NextRepeatTime[ButtonIndex] = CurrentTime + InitialButtonRepeatDelay;
					}
				}
				else if (NewControllerState.ButtonStates[ButtonIndex] && NewControllerState.NextRepeatTime[ButtonIndex] <= CurrentTime)
				{
					// Send button repeat events
					MessageHandler->OnControllerButtonPressed(ButtonMapping[ButtonIndex], UserId, InputDeviceId, true);

					// Set the button's NextRepeatTime to the ButtonRepeatDelay
					NewControllerState.NextRepeatTime[ButtonIndex] = CurrentTime + ButtonRepeatDelay;
				}
			}

			// send controller force feedback updates if enabled
			if (GAndroidUseControllerFeedback != 0)
			{
				if (DeviceInfo.FeedbackMotorCount > 0)
				{
					UpdateControllerVibeMotors(DeviceInfo.DeviceId, DeviceConfigPtr->ControllerClass, DeviceData->ControllerVibeState);
				}
			}

			// Update the state for next time
			OldControllerState = NewControllerState;
		}
	}

	for (int i = 0; i < FAndroidInputInterface::MotionDataStack.Num(); ++i)
	{
		MotionData motion_data = FAndroidInputInterface::MotionDataStack[i];

		MessageHandler->OnMotionDetected(
			motion_data.Tilt, motion_data.RotationRate,
			motion_data.Gravity, motion_data.Acceleration,
			0);
	}

	for (int i = 0; i < FAndroidInputInterface::MouseDataStack.Num(); ++i)
	{
		MouseData mouse_data = FAndroidInputInterface::MouseDataStack[i];

		switch (mouse_data.EventType)
		{
			case MouseEventType::MouseMove:
				if (Cursor.IsValid())
				{
					Cursor->SetPosition(mouse_data.AbsoluteX, mouse_data.AbsoluteY);
					MessageHandler->OnMouseMove();
				}
				MessageHandler->OnRawMouseMove(mouse_data.DeltaX, mouse_data.DeltaY);
				break;

			case MouseEventType::MouseWheel:
				MessageHandler->OnMouseWheel(mouse_data.WheelDelta);
				break;

			case MouseEventType::MouseButtonDown:
				MessageHandler->OnMouseDown(nullptr, mouse_data.Button);
				break;

			case MouseEventType::MouseButtonUp:
				MessageHandler->OnMouseUp(mouse_data.Button);
				break;
		}
	}

	for (int32 MessageIndex = 0; MessageIndex < FMath::Min(DeferredMessageQueueLastEntryIndex, MAX_DEFERRED_MESSAGE_QUEUE_SIZE); ++MessageIndex)
	{
		const FDeferredAndroidMessage& DeferredMessage = DeferredMessages[MessageIndex];
		const TCHAR Char = DeferredMessage.KeyEventData.modifier & AMETA_SHIFT_ON ? CharMapShift[DeferredMessage.KeyEventData.keyId] : CharMap[DeferredMessage.KeyEventData.keyId];
		
		switch (DeferredMessage.messageType)
		{

			case MessageType_KeyDown:

				MessageHandler->OnKeyDown(DeferredMessage.KeyEventData.keyId, Char, DeferredMessage.KeyEventData.isRepeat);
				MessageHandler->OnKeyChar(Char,  DeferredMessage.KeyEventData.isRepeat);
				break;

			case MessageType_KeyUp:

				MessageHandler->OnKeyUp(DeferredMessage.KeyEventData.keyId, Char, false);
				break;
		} 
	}

	if (DeferredMessageQueueDroppedCount)
	{
		//should warn that messages got dropped, which message queue?
		DeferredMessageQueueDroppedCount = 0;
	}

	DeferredMessageQueueLastEntryIndex = 0;

	FAndroidInputInterface::TouchInputStack.Empty(0);

	FAndroidInputInterface::MotionDataStack.Empty();

	FAndroidInputInterface::MouseDataStack.Empty();

	for (auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt)
	{
		(*DeviceIt)->SendControllerEvents();
	}
}

void FAndroidInputInterface::QueueTouchInput(const TArray<TouchInput>& InTouchEvents)
{
	FScopeLock Lock(&TouchInputCriticalSection);

	FAndroidInputInterface::TouchInputStack.Append(InTouchEvents);
}

FInputDeviceId FAndroidInputInterface::RegisterSurfaceForTouch(const TSharedRef<FAndroidWindow>& Window)
{
	FScopeLock Lock(&TouchInputCriticalSection);

	// Recycle a previously-freed FInputDeviceId when one is available; otherwise allocate a fresh one from the platform mapper.
	FInputDeviceId DeviceId;
	if (FreeSurfaceInputDeviceIds.Num() > 0)
	{
		DeviceId = FreeSurfaceInputDeviceIds.Pop(EAllowShrinking::No);
	}
	else
	{
		DeviceId = IPlatformInputDeviceMapper::Get().AllocateNewInputDeviceId();
	}
	InputMappedSurfaces.Emplace(DeviceId, TWeakPtr<FGenericWindow>(Window));

	FInputDeviceDescriptor RegistryDescriptor;
	RegistryDescriptor.HardwareDeviceHandle = DeviceId;
	RegistryDescriptor.InputDeviceName = InputClassName_DefaultMobileTouch;
	RegistryDescriptor.HardwareDeviceIdentifier = FName(*HardwareDeviceIdentifier_DefaultMobileTouch);
	FInputDeviceRegistry::RegisterDevice(RegistryDescriptor);

	return DeviceId;
}

void FAndroidInputInterface::UnregisterSurface(FInputDeviceId DeviceId)
{
	FScopeLock Lock(&TouchInputCriticalSection);

	InputMappedSurfaces.Remove(DeviceId);
	FInputDeviceRegistry::RemoveDevice(DeviceId);
	FreeSurfaceInputDeviceIds.Add(DeviceId);
}

int32 FAndroidInputInterface::FindControllerId(int32 DeviceId)
{
	if (!bAllowControllers)
	{
		return INDEX_NONE;
	}
	
	// Treat non-positive devices ids special
	if (DeviceId < 1)
		return INDEX_NONE;

	return GameControllerIdMapping.Find(DeviceId);
}

ControllerClassType FAndroidInputInterface::GetControllerClass(int32 ControllerId) const
{
	ControllerClassType Result = ControllerClassType::Generic;
	
	FAndroidGamepadDeviceMapping* OutDeviceData;
	if (GetInputDeviceByControllerId(ControllerId, nullptr, &OutDeviceData))
	{
		if (OutDeviceData->DeviceConfig.IsValid())
		{
			return OutDeviceData->DeviceConfig.Pin()->ControllerClass;
		}
	}
	
	return Result;
}

// This function can be used to check if a device ID is valid when passing both OutDeviceInfo and OutDeviceData with nullptr
bool FAndroidInputInterface::GetInputDeviceByDeviceId(int32 DeviceId, FAndroidInputDeviceInfo** OutDeviceInfo, FAndroidGamepadDeviceMapping** OutDeviceData)
{
	FAndroidInputDeviceInfo* DeviceInfo = InputDeviceInfoMap.Find(DeviceId);
	if (DeviceInfo == nullptr)
	{
		return false;
	}

	if (OutDeviceInfo != nullptr)
	{
		*OutDeviceInfo = DeviceInfo;
	}

	if ((OutDeviceData != nullptr) && (DeviceInfo->DeviceType == InputDeviceType::GameController))
	{
		FInputDeviceId InputDeviceId = InternalDeviceIdMappings.FindDeviceId(DeviceInfo->Descriptor);
		if (InputDeviceId == INPUTDEVICEID_NONE)
		{
			return false;
		}
		*OutDeviceData = GameControllerDataMap.Find(InputDeviceId);
		if (*OutDeviceData == nullptr)
		{
			return false;
		}
	}

	return true;
}

// This function can be used to check if a controller ID is valid when passing both OutDeviceInfo and OutDeviceData with nullptr
bool FAndroidInputInterface::GetInputDeviceByControllerId(int32 ControllerId, FAndroidInputDeviceInfo** OutDeviceInfo, FAndroidGamepadDeviceMapping** OutDeviceData)
{
	if (GameControllerIdMapping.IsValidIndex(ControllerId))
	{
		return GetInputDeviceByDeviceId(GameControllerIdMapping[ControllerId], OutDeviceInfo, OutDeviceData);
	}
	return false;
}

void FAndroidInputInterface::AddPendingInputDevice(int32 DeviceId, InputDeviceType DeviceType)
{
	if (InputDeviceInfoMap.Contains(DeviceId))
	{
		return; // a changed device will return here, nothing needs to be updated at the native side.
	}

	bool IsGameController = DeviceType == InputDeviceType::GameController;
	// newly connected device is saved for now and will be validated in the next frame
	FAndroidInputDeviceInfo& AddedDeviceInfo = InputDeviceInfoMap.Add(DeviceId);
	AddedDeviceInfo.DeviceState = (AndroidEnableInputDeviceListener && IsGameController) ? MappingState::ToActivate : MappingState::ToValidate;
	AddedDeviceInfo.DeviceType = DeviceType;
	AddedDeviceInfo.DeviceId = DeviceId;

	if (IsGameController)
	{
		int32 UnassignedId = -1;
		for (int32 ControllerId = 0; ControllerId < GameControllerIdMapping.Num(); ++ControllerId)
		{
			int32 MappedDeviceId = GameControllerIdMapping[ControllerId];
			if (MappedDeviceId == DeviceId)
			{
				return;
			}
			if ((UnassignedId == -1) && (!InputDeviceInfoMap.Contains(MappedDeviceId)))
			{
				UnassignedId = ControllerId;
			}
		}

		if (UnassignedId == -1)
		{
			GameControllerIdMapping.Push(DeviceId);
		}
		else
		{
			GameControllerIdMapping[UnassignedId] = DeviceId;
		}
	}
}

void FAndroidInputInterface::RemoveInputDevice(int32 DeviceId)
{
	FAndroidInputDeviceInfo* DeviceInfo = InputDeviceInfoMap.Find(DeviceId);
	if (DeviceInfo == nullptr)
	{
		UE_LOGF(LogAndroid, Error, "Nonexistent input device removed, DeviceId = %d", DeviceId);
		return;
	}

	if (DeviceInfo->DeviceState == MappingState::Valid)
	{
		ExecuteOnGameThread(TEXT("Broadcast the EInputDeviceConnectionState::Disconnected status"), 
			[Descriptor = DeviceInfo->Descriptor]()
			{
				MapControllerToPlayer(Descriptor, EInputDeviceConnectionState::Disconnected);
			});
	}

	if (DeviceInfo->DeviceType == InputDeviceType::GameController)
	{
		FInputDeviceId InputDeviceId = InternalDeviceIdMappings.FindDeviceId(DeviceInfo->Descriptor);
		if (InputDeviceId != INPUTDEVICEID_NONE)
		{
			// keep the game controller data for future reconnections since the FInputDeviceId won't change
			FAndroidGamepadDeviceMapping* GameControllerData = GameControllerDataMap.Find(InputDeviceId);
			if (GameControllerData != nullptr)
			{
				GameControllerData->ResetRuntimeData();
			}
		}
	}

	InputDeviceInfoMap.Remove(DeviceId);
}

void FAndroidInputInterface::DumpInputDevices()
{
#if UE_BUILD_DEVELOPMENT
	UE_LOGF(LogAndroid, Log, "===== Dump Input Devices =====");

	for (TPair<int32, FAndroidInputDeviceInfo> KVP : InputDeviceInfoMap)
	{
		const FAndroidInputDeviceInfo& Info = KVP.Value;
		FInputDeviceId InputDeviceId = InternalDeviceIdMappings.FindDeviceId(Info.Descriptor);
		IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
		FPlatformUserId UserId = DeviceMapper.GetUserForInputDevice(InputDeviceId);
		UE_LOGF(LogAndroid, Log, "DeviceInfo, DeviceId = %03d, Type = %d, State = %d, IsExternal = %d, InputDeviceId = %02d, UserId = %02d, Name = %ls",
			Info.DeviceId, Info.DeviceType, Info.DeviceState, Info.IsExternal, InputDeviceId.GetId(), UserId.GetInternalId(), *(Info.Name.ToString()));
	}
	
	UE_LOGF(LogAndroid, Log, "---");

	for (TPair<FInputDeviceId, FAndroidGamepadDeviceMapping> KVP : GameControllerDataMap)
	{
		FInputDeviceId InputDeviceId = KVP.Key;
		const FAndroidGamepadDeviceMapping& Data = KVP.Value;
		if (Data.DeviceConfig.IsValid())
		{
			UE_LOGF(LogAndroid, Log, "DeviceData, InputDeviceId = %02d, ControllerClass = %d",
				InputDeviceId.GetId(), Data.DeviceConfig.Pin()->ControllerClass);
		}
	}

	UE_LOGF(LogAndroid, Log, "---");

	for (int32 DeviceId : GameControllerIdMapping)
	{
		UE_LOGF(LogAndroid, Log, "ControllerId, DeviceId = %03d", DeviceId);
	}

	UE_LOGF(LogAndroid, Log, "==============================");
#endif //UE_BUILD_DEVELOPMENT
}

void FAndroidInputInterface::HandleInputDeviceStateEvent(int32 DeviceId, InputDeviceStateEvent StateEvent, InputDeviceType DeviceType)
{
	if (!AndroidEnableInputDeviceListener)
	{
		return;
	}

	FScopeLock Lock(&TouchInputCriticalSection);

	switch (StateEvent)
	{
	case InputDeviceStateEvent::Added:
		// falls through
	case InputDeviceStateEvent::Changed:
		AddPendingInputDevice(DeviceId, DeviceType);
		break;
	case InputDeviceStateEvent::Removed:
		RemoveInputDevice(DeviceId);
		DumpInputDevices();
		break;
	}
}

void FAndroidInputInterface::JoystickAxisEvent(int32 DeviceId, int32 AxisId, float AxisValue)
{
	FScopeLock Lock(&TouchInputCriticalSection);

	FAndroidInputDeviceInfo* DeviceInfo = nullptr;
	FAndroidGamepadDeviceMapping* DeviceData = nullptr;
	if (!GetInputDeviceByDeviceId(DeviceId, &DeviceInfo, &DeviceData))
	{
		AddPendingInputDevice(DeviceId, InputDeviceType::GameController);
		return;
	}

	if (AndroidEnableInputDeviceListener && DeviceInfo->DeviceState == MappingState::ToValidate)
	{
		MapControllerToPlayer(DeviceInfo->Descriptor, EInputDeviceConnectionState::Connected);
		DeviceInfo->DeviceState = MappingState::Valid;
		UE_LOGF(LogAndroid, Log, "Gamepad state changed to Valid, DeviceId = %d", DeviceId);
	}

	if (DeviceData == nullptr || !DeviceData->DeviceConfig.IsValid())
	{
		return; // bypass inputs from the device not identified as a game controller
	}

	auto RemapTriggerFunction = [](const float Minimum, const float Value)
	{
		if(Minimum != 0.0f)
		{
			const float AdjustMin = Minimum;
			const float AdjustMax = 1.0f - AdjustMin;
			return FMath::Clamp(Value - AdjustMin, 0.0f, AdjustMax) / AdjustMax;
		}
		return Value;
	};

	DeviceConfig_SPtr DeviceConfigPtr = DeviceData->DeviceConfig.Pin();
	check(DeviceConfigPtr.IsValid());

	// Deal with left stick and triggers (generic)
	switch (AxisId)
	{
		case AMOTION_EVENT_AXIS_X:			DeviceData->NewControllerData.LXAnalog =  AxisValue; return;
		case AMOTION_EVENT_AXIS_Y:			DeviceData->NewControllerData.LYAnalog = -AxisValue; return;
		case AMOTION_EVENT_AXIS_LTRIGGER:
			{
				if (!(DeviceConfigPtr->MapZRZToTriggers || DeviceConfigPtr->MapRXRYToTriggers))
				{
					DeviceData->NewControllerData.LTAnalog = RemapTriggerFunction(DeviceConfigPtr->LTAnalogRangeMinimum, AxisValue);
					return;
				}
			}
		case AMOTION_EVENT_AXIS_RTRIGGER:
			{
				if (!(DeviceConfigPtr->MapZRZToTriggers || DeviceConfigPtr->MapRXRYToTriggers))
				{
					DeviceData->NewControllerData.RTAnalog = RemapTriggerFunction(DeviceConfigPtr->RTAnalogRangeMinimum, AxisValue);
					return;
				}
			}
	}

	// Deal with right stick Z/RZ events
	if (DeviceConfigPtr->RightStickZRZ)
	{
		switch (AxisId)
		{
			case AMOTION_EVENT_AXIS_Z:		DeviceData->NewControllerData.RXAnalog =  AxisValue; return;
			case AMOTION_EVENT_AXIS_RZ:		DeviceData->NewControllerData.RYAnalog = -AxisValue; return;
		}
	}

	// Deal with right stick RX/RY events
	if (DeviceConfigPtr->RightStickRXRY)
	{
		switch (AxisId)
		{
			case AMOTION_EVENT_AXIS_RX:		DeviceData->NewControllerData.RXAnalog =  AxisValue; return;
			case AMOTION_EVENT_AXIS_RY:		DeviceData->NewControllerData.RYAnalog = -AxisValue; return;
		}
	}

	// Deal with Z/RZ mapping to triggers
	if (DeviceConfigPtr->MapZRZToTriggers)
	{
		switch (AxisId)
		{
			case AMOTION_EVENT_AXIS_Z:		DeviceData->NewControllerData.LTAnalog = RemapTriggerFunction(DeviceConfigPtr->LTAnalogRangeMinimum, AxisValue); return;
			case AMOTION_EVENT_AXIS_RZ:		DeviceData->NewControllerData.RTAnalog = RemapTriggerFunction(DeviceConfigPtr->RTAnalogRangeMinimum, AxisValue); return;
		}
	}

	if (DeviceConfigPtr->MapRXRYToTriggers)
	{
		switch (AxisId)
		{
			case AMOTION_EVENT_AXIS_RX:		DeviceData->NewControllerData.LTAnalog = RemapTriggerFunction(DeviceConfigPtr->LTAnalogRangeMinimum, AxisValue); return;
			case AMOTION_EVENT_AXIS_RY:		DeviceData->NewControllerData.RTAnalog = RemapTriggerFunction(DeviceConfigPtr->RTAnalogRangeMinimum, AxisValue); return;
		}
	}

	// Deal with hat (convert to DPAD buttons)
	if (DeviceConfigPtr->SupportsHat)
	{
		// Apply a small dead zone to hats
		const float DeadZone = 0.2f;

		switch (AxisId)
		{
			case AMOTION_EVENT_AXIS_HAT_X:
				// AMOTION_EVENT_AXIS_HAT_X translates to KEYCODE_DPAD_LEFT and AKEYCODE_DPAD_RIGHT
				if (AxisValue > DeadZone)
				{
					DeviceData->NewControllerData.ButtonStates[14] = false;	// DPAD_LEFT released
					DeviceData->NewControllerData.ButtonStates[15] = true;	// DPAD_RIGHT pressed
				}
				else if (AxisValue < -DeadZone)
				{
					DeviceData->NewControllerData.ButtonStates[14] = true;	// DPAD_LEFT pressed
					DeviceData->NewControllerData.ButtonStates[15] = false;	// DPAD_RIGHT released
				}
				else
				{
					DeviceData->NewControllerData.ButtonStates[14] = false;	// DPAD_LEFT released
					DeviceData->NewControllerData.ButtonStates[15] = false;	// DPAD_RIGHT released
				}
				return;
			case AMOTION_EVENT_AXIS_HAT_Y:
				// AMOTION_EVENT_AXIS_HAT_Y translates to KEYCODE_DPAD_UP and AKEYCODE_DPAD_DOWN
				if (AxisValue > DeadZone)
				{
					DeviceData->NewControllerData.ButtonStates[12] = false;	// DPAD_UP released
					DeviceData->NewControllerData.ButtonStates[13] = true;	// DPAD_DOWN pressed
				}
				else if (AxisValue < -DeadZone)
				{
					DeviceData->NewControllerData.ButtonStates[12] = true;	// DPAD_UP pressed
					DeviceData->NewControllerData.ButtonStates[13] = false;	// DPAD_DOWN released
				}
				else
				{
					DeviceData->NewControllerData.ButtonStates[12] = false;	// DPAD_UP released
					DeviceData->NewControllerData.ButtonStates[13] = false;	// DPAD_DOWN released
				}
				return;
		}
	}
}

void FAndroidInputInterface::JoystickButtonEvent(int32 deviceId, int32 buttonId, bool buttonDown)
{
	FScopeLock Lock(&TouchInputCriticalSection);

	FAndroidInputDeviceInfo* DeviceInfo = nullptr;
	FAndroidGamepadDeviceMapping* DeviceData = nullptr;
	if (!GetInputDeviceByDeviceId(deviceId, &DeviceInfo, &DeviceData))
	{
		AddPendingInputDevice(deviceId, InputDeviceType::GameController);
		return;
	}

	if (AndroidEnableInputDeviceListener && DeviceInfo->DeviceState == MappingState::ToValidate)
	{
		MapControllerToPlayer(DeviceInfo->Descriptor, EInputDeviceConnectionState::Connected);
		DeviceInfo->DeviceState = MappingState::Valid;
		UE_LOGF(LogAndroid, Log, "Gamepad state changed to Valid, DeviceId = %d", deviceId);
		DumpInputDevices();
	}

	if (DeviceData == nullptr || !DeviceData->DeviceConfig.IsValid())
	{
		return; // bypass inputs from a device not identified as a game controller or without a valid device configuration data
	}

	// Deal with button remapping
	DeviceConfig_SPtr DeviceConfigPtr = DeviceData->DeviceConfig.Pin();
	check(DeviceConfigPtr.IsValid());
	KeyCodeToKeyNameMap_SPtr ButtonMappingPtr = DeviceConfigPtr->ButtonMapping.Pin();
	check(ButtonMappingPtr.IsValid());
	TArray<int32>* KeyNameIndices = ButtonMappingPtr->Find(buttonId);
	if (KeyNameIndices != nullptr)
	{
		for (int32 KeyNameIndex : *KeyNameIndices)
		{
			if ((KeyNameIndex == 16 || KeyNameIndex == 17) && bBlockAndroidKeysOnControllers)
			{
				continue;
			}

			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("JoystickButtonEvent[%d]: %d"), (int)KeyNameIndex, buttonId);
			DeviceData->NewControllerData.ButtonStates[KeyNameIndex] = buttonDown;
		}
	}
	else
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Found KeyCode without mapping: %d"), buttonId);
	}
}


int32 FAndroidInputInterface::GetAlternateKeyEventForMouse(int32 deviceId, int32 buttonId)
{
	FScopeLock Lock(&TouchInputCriticalSection);

	if (buttonId == 0)
	{
		FAndroidInputDeviceInfo* Info = nullptr;
		FAndroidGamepadDeviceMapping* data = nullptr;
		if (GetInputDeviceByDeviceId(deviceId, &Info, &data))
		{
			ensure(Info); // must always be set if GetInputDeviceByDeviceId returned true
			if (Info->DeviceState == MappingState::Valid &&
				data != nullptr && data->DeviceConfig.IsValid())
			{
				DeviceConfig_SPtr DeviceConfigPtr = data->DeviceConfig.Pin();
				check(DeviceConfigPtr.IsValid());
				if (DeviceConfigPtr->ControllerClass == ControllerClassType::PlaystationWireless)
				{
					return 3002;
				}
			}
		}
	}
	return 0;
}

void FAndroidInputInterface::MouseMoveEvent(int32 deviceId, float absoluteX, float absoluteY, float deltaX, float deltaY)
{
	// for now only deal with one mouse
	FScopeLock Lock(&TouchInputCriticalSection);

	FAndroidInputInterface::MouseDataStack.Push(
		MouseData{ MouseEventType::MouseMove, EMouseButtons::Invalid, (int32)absoluteX, (int32)absoluteY, (int32)deltaX, (int32)deltaY, 0.0f });
}

void FAndroidInputInterface::MouseWheelEvent(int32 deviceId, float wheelDelta)
{
	// for now only deal with one mouse
	FScopeLock Lock(&TouchInputCriticalSection);

	FAndroidInputInterface::MouseDataStack.Push(
		MouseData{ MouseEventType::MouseWheel, EMouseButtons::Invalid, 0, 0, 0, 0, wheelDelta });
}

void FAndroidInputInterface::MouseButtonEvent(int32 deviceId, int32 buttonId, bool buttonDown)
{
	// for now only deal with one mouse
	FScopeLock Lock(&TouchInputCriticalSection);

	MouseEventType EventType = buttonDown ? MouseEventType::MouseButtonDown : MouseEventType::MouseButtonUp;
	EMouseButtons::Type UnrealButton = (buttonId == 0) ? EMouseButtons::Left : (buttonId == 1) ? EMouseButtons::Right : EMouseButtons::Middle;
	FAndroidInputInterface::MouseDataStack.Push(
		MouseData{ EventType, UnrealButton, 0, 0, 0, 0, 0.0f });
}

void FAndroidInputInterface::DeferMessage(const FDeferredAndroidMessage& DeferredMessage)
{
	FScopeLock Lock(&TouchInputCriticalSection);
	// Get the index we should be writing to
	int32 Index = DeferredMessageQueueLastEntryIndex++;

	if (Index >= MAX_DEFERRED_MESSAGE_QUEUE_SIZE)
	{
		// Actually, if the queue is full, drop the message and increment a counter of drops
		DeferredMessageQueueDroppedCount++;
		return;
	}
	DeferredMessages[Index] = DeferredMessage;
}

void FAndroidInputInterface::QueueMotionData(const FVector& Tilt, const FVector& RotationRate, const FVector& Gravity, const FVector& Acceleration)
{
	FScopeLock Lock(&TouchInputCriticalSection);
	EDeviceScreenOrientation ScreenOrientation = FPlatformMisc::GetDeviceOrientation();
	FVector TempRotationRate = RotationRate;

	if (AndroidUnifyMotionSpace != 0)
	{
		FVector TempTilt = Tilt;
		FVector TempGravity = Gravity;
		FVector TempAcceleration = Acceleration;

		auto ReorientLandscapeLeft = [](FVector InValue)
		{
			return AndroidUnifyMotionSpace == 1 ? FVector(-InValue.Z, -InValue.Y, InValue.X) : FVector(-InValue.Y, -InValue.Z, InValue.X);
		};

		auto ReorientLandscapeRight = [](FVector InValue)
		{
			return AndroidUnifyMotionSpace == 1 ? FVector(-InValue.Z, InValue.Y, -InValue.X) : FVector(InValue.Y, -InValue.Z, -InValue.X);
		};

		auto ReorientPortrait = [](FVector InValue)
		{
			return AndroidUnifyMotionSpace == 1 ? FVector(-InValue.Z, InValue.X, InValue.Y) : FVector(InValue.X, -InValue.Z, InValue.Y);
		};

		const float ToG = 1.f / 9.8f;

		switch (ScreenOrientation)
		{
			// the x tilt is inverted in LandscapeLeft.
		case EDeviceScreenOrientation::LandscapeLeft:
			TempTilt = -ReorientLandscapeLeft(TempTilt);
			TempRotationRate = -ReorientLandscapeLeft(TempRotationRate);
			TempGravity = ReorientLandscapeLeft(TempGravity) * ToG;
			TempAcceleration = ReorientLandscapeLeft(TempAcceleration) * ToG;
			break;
			// the y tilt is inverted in LandscapeRight.
		case EDeviceScreenOrientation::LandscapeRight:
			TempTilt = -ReorientLandscapeRight(TempTilt);
			TempRotationRate = -ReorientLandscapeRight(TempRotationRate);
			TempGravity = ReorientLandscapeRight(TempGravity) * ToG;
			TempAcceleration = ReorientLandscapeRight(TempAcceleration) * ToG;
			break;
		case EDeviceScreenOrientation::Portrait:
			TempTilt = -ReorientPortrait(TempTilt);
			TempRotationRate = -ReorientPortrait(TempRotationRate);
			TempGravity = ReorientPortrait(TempGravity) * ToG;
			TempAcceleration = ReorientPortrait(TempAcceleration) * ToG;
			break;
		}

		if (AndroidUnifyMotionSpace == 2)
		{
			TempRotationRate = -TempRotationRate;
		}

		FAndroidInputInterface::MotionDataStack.Push(
			MotionData{ TempTilt, TempRotationRate, TempGravity, TempAcceleration });
	}
	else
	{
		switch (ScreenOrientation)
		{
			// the x tilt is inverted in LandscapeLeft.
		case EDeviceScreenOrientation::LandscapeLeft:
			TempRotationRate.X *= -1.0f;
			break;
			// the y tilt is inverted in LandscapeRight.
		case EDeviceScreenOrientation::LandscapeRight:
			TempRotationRate.Y *= -1.0f;
			break;
		}

		FAndroidInputInterface::MotionDataStack.Push(
			MotionData{ Tilt, TempRotationRate, Gravity, Acceleration });
	}
}

#endif
