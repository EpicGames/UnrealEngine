// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidJavaMediaFrameData.h"
#include "Android/AndroidApplication.h"

#include "GlobalShader.h"
#include "MediaShaders.h"
#include "ColorManagement/ColorSpace.h"
#include "PipelineStateCache.h"
#include "ShaderParameterUtils.h"
#include "IVulkanDynamicRHI.h"
#include "IOpenGLDynamicRHI.h"
#include "RHIStaticStates.h"

#include <android/hardware_buffer.h>
#include <android/hardware_buffer_jni.h>

#if USE_ANDROID_JNI

jfieldID FAndroidJavaMediaFrameData::FrameData_HardwareBufferHandle = nullptr;
jfieldID FAndroidJavaMediaFrameData::FrameData_UScale = nullptr;
jfieldID FAndroidJavaMediaFrameData::FrameData_UOffset = nullptr;
jfieldID FAndroidJavaMediaFrameData::FrameData_VScale = nullptr;
jfieldID FAndroidJavaMediaFrameData::FrameData_VOffset = nullptr;
jmethodID FAndroidJavaMediaFrameData::FrameData_ReleaseFN = nullptr;

FAndroidJavaMediaFrameData::FAndroidJavaMediaFrameData()
	: FrameDataGlobalRef(nullptr)
{
	if (FrameData_HardwareBufferHandle == nullptr)
	{
		JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
		jclass FrameDataClass = FAndroidApplication::FindJavaClass("com/epicgames/unreal/BitmapRenderer$FrameData");
		FrameData_HardwareBufferHandle = JEnv->GetFieldID(FrameDataClass, "hardwareBuffer", "Landroid/hardware/HardwareBuffer;");
		FrameData_UScale = JEnv->GetFieldID(FrameDataClass, "UScale", "F");
		FrameData_UOffset = JEnv->GetFieldID(FrameDataClass, "UOffset", "F");
		FrameData_VScale = JEnv->GetFieldID(FrameDataClass, "VScale", "F");
		FrameData_VOffset = JEnv->GetFieldID(FrameDataClass, "VOffset", "F");
		FrameData_ReleaseFN = JEnv->GetMethodID(FrameDataClass, "release", "()V");
	}
}

FAndroidJavaMediaFrameData::~FAndroidJavaMediaFrameData()
{
	CleanUp();
}

FAndroidJavaMediaFrameData& FAndroidJavaMediaFrameData::operator=(FAndroidJavaMediaFrameData& Other)
{
	Swap(FrameDataGlobalRef, Other.FrameDataGlobalRef);
	return *this;
}

bool FAndroidJavaMediaFrameData::Set(jobject InFrameData)
{
	CleanUp();

	if (InFrameData != nullptr)
	{
		JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();

		jobjectRefType RefType = JEnv->GetObjectRefType(InFrameData);
		if (RefType == jobjectRefType::JNILocalRefType)
		{
			FrameDataGlobalRef = JEnv->NewGlobalRef(InFrameData);
		}
		else if (RefType == jobjectRefType::JNIGlobalRefType)
		{
			FrameDataGlobalRef = InFrameData;
		}
	}

	return (FrameDataGlobalRef != nullptr);
}

jobject FAndroidJavaMediaFrameData::Extract()
{
	if (FrameDataGlobalRef == nullptr)
	{
		return nullptr;
	}

	JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
	jobject HardwareBufferObj = JEnv->GetObjectField(FrameDataGlobalRef, FrameData_HardwareBufferHandle);
	check(HardwareBufferObj != nullptr);

	return HardwareBufferObj;
}

void FAndroidJavaMediaFrameData::CleanUp()
{
	Fence = nullptr;
	Texture = nullptr;

	if (FrameDataGlobalRef != nullptr)
	{
		JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
		JEnv->CallVoidMethod(FrameDataGlobalRef, FrameData_ReleaseFN);
		JEnv->DeleteGlobalRef(FrameDataGlobalRef);
		FrameDataGlobalRef = nullptr;
	}
}

bool FAndroidJavaMediaFrameData::ExtractToTextureVulkan(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, IMediaTextureSample* TextureSample)
{
	check(IsInRenderingThread());

	if (GDynamicRHI->RHIIsRenderingSuspended() || FrameDataGlobalRef == nullptr)
	{
		return false;
	}

	bool Converted = false;

	jobject HardwareBufferObj = Extract();
	AHardwareBuffer* HardwareBuffer = AHardwareBuffer_fromHardwareBuffer(FAndroidApplication::GetJavaEnv(), HardwareBufferObj);
	if (HardwareBuffer != nullptr)
	{
		AHardwareBuffer_acquire(HardwareBuffer);

		IVulkanDynamicRHI* RHI = GetIVulkanDynamicRHI();
		Texture = RHI->RHICreateTexture2DFromAndroidHardwareBuffer(HardwareBuffer);
		if (Texture.IsValid())
		{
			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			FRHITexture* RenderTarget = InDstTexture;

			Fence = RHICreateGPUFence(TEXT("MediaFrameDataToTextureVulkan"));

			RHICmdList.Transition(FRHITransitionInfo(InDstTexture, ERHIAccess::Unknown, ERHIAccess::RTV));

			FRHIRenderPassInfo RPInfo(RenderTarget, ERenderTargetActions::DontLoad_Store);
			RHICmdList.BeginRenderPass(RPInfo, TEXT("ConvertMedia_HardwareBuffer"));
			{
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

				FIntPoint Dim = TextureSample->GetOutputDim();
				RHICmdList.SetViewport(0, 0, 0.0f, (float)Dim.X, (float)Dim.Y, 1.0f);

				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
				GraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<CW_RGBA, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE>::GetRHI();
				GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;

				// configure media shaders
				FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
				TShaderMapRef<FMediaShadersVS> VertexShader(ShaderMap);

				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GMediaVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();

				FMatrix44f YUVMtx = TextureSample->GetSampleToRGBMatrix();
				FMatrix44f ColorSpaceMtx;
				//------------------------------------------------------------------------
				//GetColorSpaceConversionMatrixForSample(this, ColorSpaceMtx);
				{
					const UE::Color::FColorSpace& Working = UE::Color::FColorSpace::GetWorking();

					if (TextureSample->GetMediaTextureSampleColorConverter())
					{
						ColorSpaceMtx = FMatrix44f::Identity;
					}
					else
					{
						ColorSpaceMtx = UE::Color::Transpose<float>(UE::Color::FColorSpaceTransform(TextureSample->GetSourceColorSpace(), Working));
					}

					float NF = TextureSample->GetHDRNitsNormalizationFactor();
					if (NF != 1.0f)
					{
						ColorSpaceMtx = ColorSpaceMtx.ApplyScale(NF);
					}
				}
				//------------------------------------------------------------------------

				TShaderMapRef<FVYUConvertPS> ConvertShader(ShaderMap);
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
				SetShaderParametersLegacyPS(RHICmdList, ConvertShader, Texture, Dim, YUVMtx, TextureSample->GetEncodingType(), ColorSpaceMtx,
					MediaShaders::EToneMapMethod::None, TextureSample->GetScaleRotation(), TextureSample->GetOffset());

				// draw full size quad into render target
				FBufferRHIRef VertexBuffer = CreateTempMediaVertexBuffer(RHICmdList);
				RHICmdList.SetStreamSource(0, VertexBuffer, 0);
				// set viewport to RT size
				RHICmdList.SetViewport(0, 0, 0.0f, (float)Dim.X, (float)Dim.Y, 1.0f);

				RHICmdList.DrawPrimitive(0, 2, 1);
			}
			RHICmdList.EndRenderPass();

			RHICmdList.Transition(FRHITransitionInfo(RenderTarget, ERHIAccess::RTV, ERHIAccess::SRVGraphics));

			RHICmdList.WriteGPUFence(Fence);

			Converted = true;
		}

		AHardwareBuffer_release(HardwareBuffer);
	}

	FAndroidApplication::GetJavaEnv()->DeleteLocalRef(HardwareBufferObj);
	return Converted;
}

bool FAndroidJavaMediaFrameData::ExtractToTextureOES(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, IMediaTextureSample* TextureSample)
{
	if (!FrameDataGlobalRef)
	{
		return false;
	}

	check(IsInRenderingThread());

	bool Converted = false;

	jobject HardwareBufferObj = Extract();
	AHardwareBuffer* HardwareBuffer = AHardwareBuffer_fromHardwareBuffer(FAndroidApplication::GetJavaEnv(), HardwareBufferObj);
	if (HardwareBuffer != nullptr)
	{
		AHardwareBuffer_acquire(HardwareBuffer);

		IOpenGLDynamicRHI* RHI = GetIOpenGLDynamicRHI();
		Texture = RHI->RHICreateTexture2DFromAndroidHardwareBuffer(RHICmdList, HardwareBuffer);
		if (Texture.IsValid())
		{
			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			FRHITexture* RenderTarget = InDstTexture.GetReference();

			RHICmdList.Transition(FRHITransitionInfo(InDstTexture, ERHIAccess::Unknown, ERHIAccess::RTV));

			FRHIRenderPassInfo RPInfo(RenderTarget, ERenderTargetActions::DontLoad_Store);
			RHICmdList.BeginRenderPass(RPInfo, TEXT("ConvertMedia_HardwareBuffer"));
			{
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

				FIntPoint Dim = TextureSample->GetOutputDim();
				RHICmdList.SetViewport(0, 0, 0.0f, (float)Dim.X, (float)Dim.Y, 1.0f);

				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
				GraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<CW_RGBA, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE>::GetRHI();
				GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;

				// configure media shaders
				auto ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
				TShaderMapRef<FMediaShadersVS> VertexShader(ShaderMap);

				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GMediaVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();

				FSamplerStateInitializerRHI InSamplerStateInitializer(SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp);
				FSamplerStateRHIRef InSamplerState = RHICreateSamplerState(InSamplerStateInitializer);

				TShaderMapRef<FReadTextureExternalPS> CopyShader(ShaderMap);
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = CopyShader.GetPixelShader();
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
				SetShaderParametersLegacyPS(RHICmdList, CopyShader, Texture, InSamplerState, TextureSample->GetScaleRotation(), TextureSample->GetOffset());

				// draw full size quad into render target
				FBufferRHIRef VertexBuffer = CreateTempMediaVertexBuffer(RHICmdList);
				RHICmdList.SetStreamSource(0, VertexBuffer, 0);
				// set viewport to RT size
				RHICmdList.SetViewport(0, 0, 0.0f, (float)Dim.X, (float)Dim.Y, 1.0f);

				RHICmdList.DrawPrimitive(0, 2, 1);
			}
			RHICmdList.EndRenderPass();

			RHICmdList.Transition(FRHITransitionInfo(InDstTexture, ERHIAccess::RTV, ERHIAccess::SRVMask));

			Converted = true;
		}

		AHardwareBuffer_release(HardwareBuffer);
	}

	FAndroidApplication::GetJavaEnv()->DeleteLocalRef(HardwareBufferObj);
	return Converted;
}

#endif //USE_ANDROID_JNI