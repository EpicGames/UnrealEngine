// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidJavaMediaPlayer.h"
#include "Android/AndroidApplication.h"

#if USE_ANDROID_JNI

#if UE_BUILD_SHIPPING
// always clear any exceptions in SHipping
#define CHECK_JNI_RESULT(Id) if (Id == 0) { JEnv->ExceptionClear(); }
#else
#define CHECK_JNI_RESULT(Id) \
if (Id == 0) \
{ \
	if (!bIsOptional) { JEnv->ExceptionDescribe(); checkf(Id != 0, TEXT("Failed to find " #Id)); } \
	JEnv->ExceptionClear(); \
}
#endif

static jfieldID FindField(JNIEnv* JEnv, jclass Class, const ANSICHAR* FieldName, const ANSICHAR* FieldType, bool bIsOptional)
{
	jfieldID Field = Class == NULL ? NULL : JEnv->GetFieldID(Class, FieldName, FieldType);
	CHECK_JNI_RESULT(Field);
	return Field;
}

/* MediaDataSource handling
*****************************************************************************/

FCriticalSection FJavaAndroidMediaPlayer::MediaDataSourcesCS;
TMap<int64, TWeakPtr<FJavaAndroidMediaDataSource, ESPMode::ThreadSafe>> FJavaAndroidMediaPlayer::AllMediaDataSources;

TSharedPtr<FJavaAndroidMediaDataSource, ESPMode::ThreadSafe> FJavaAndroidMediaPlayer::GetMediaDataSourcePtr(int64 Identifier)
{
	FScopeLock ScopeLock(&MediaDataSourcesCS);

	//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("GetMediaDataSourcePtr: %llu"), Identifier);

	TWeakPtr<FJavaAndroidMediaDataSource, ESPMode::ThreadSafe> MediaDataSource = AllMediaDataSources.FindRef(Identifier);
	return (MediaDataSource.IsValid()) ? MediaDataSource.Pin() : TSharedPtr<FJavaAndroidMediaDataSource, ESPMode::ThreadSafe>();
}

void FJavaAndroidMediaPlayer::AddMediaDataSourcePtr(int64 Identifier, TSharedPtr<FJavaAndroidMediaDataSource, ESPMode::ThreadSafe> MediaDataSource)
{
	FScopeLock ScopeLock(&MediaDataSourcesCS);

	//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("AddMediaDataSourcePtr: %llu"), Identifier);

	AllMediaDataSources.Add(Identifier, MediaDataSource);
}

jint JNICALL UE::Jni::FMediaPlayer14::nativeReadAt(JNIEnv* jenv, jobject thiz, jlong Identifier, jlong Position, Nio::FByteBuffer* Buffer, jint Offset, jint Count)
{
	//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("MediaPlayer14_ReadAt(%llu, %llu, , %d, %d)"), Identifier, Position, Offset, Count);

	// look up the media data source and attempt to read
	TSharedPtr<FJavaAndroidMediaDataSource, ESPMode::ThreadSafe> MediaDataSource = FJavaAndroidMediaPlayer::GetMediaDataSourcePtr(Identifier);
	if (MediaDataSource.IsValid())
	{
		uint8* OutBuffer = (uint8*)jenv->GetDirectBufferAddress(Buffer);
		//FPlatformMisc::LowLevelOutputDebugStringf(TEXT("MediaPlayer14_ReadAt buffer=%p"), OutBuffer);
		return MediaDataSource->ReadAt((int64)Position, OutBuffer + (int32)Offset, (int32)Count);
	}
	return -1;
}

FJavaAndroidMediaDataSource::FJavaAndroidMediaDataSource(const TSharedRef<FArchive, ESPMode::ThreadSafe>& InArchive)
	: Archive(InArchive)
{ }

FJavaAndroidMediaDataSource::~FJavaAndroidMediaDataSource()
{ }

int64 FJavaAndroidMediaDataSource::GetSize()
{
	FScopeLock ScopeLock(&CriticalSection);
	return Archive->TotalSize();
}

int64 FJavaAndroidMediaDataSource::GetCurrentPosition()
{
	FScopeLock ScopeLock(&CriticalSection);
	return Archive->Tell();
}

int32 FJavaAndroidMediaDataSource::ReadAt(int64 Position, uint8* Buffer, int32 Count)
{
	FScopeLock ScopeLock(&CriticalSection);

	int64 Size = Archive->TotalSize();
	if (Position >= Size)
	{
		// signal EOF
		return -1;
	}

	// Seek if not already in proper location
	int64 CurrentPosition = Archive->Tell();
	if (CurrentPosition != Position)
	{
		Archive->Seek(Position);
	}

	int32 BytesToRead = Count;

	if (Position + BytesToRead > Size)
	{
		BytesToRead = (int32)(Size - Position);
	}

	if (BytesToRead > 0)
	{
		Archive->Serialize(Buffer, BytesToRead);
	}

	return BytesToRead;
}

/* JavaAndroidMediaPlayer
*****************************************************************************/

bool AndroidEnableMediaPlayerBRIR = true;
static FAutoConsoleVariableRef CVarAndroidEnableMediaPlayerBRIR(
	TEXT("Android.EnableMediaPlayerBRIR"),
	AndroidEnableMediaPlayerBRIR,
	TEXT("Determines how the Android Media Player handles video frame data. True for using ImageReader/HardwareBuffer. False for copying data through the CPU memory. (Default: true)"),
	ECVF_Default);

FJavaAndroidMediaPlayer::FJavaAndroidMediaPlayer(bool swizzlePixels, bool vulkanRenderer, bool needTrackInfo, bool forceBitmapRenderLegacy)
	: TClassObject{swizzlePixels, vulkanRenderer, needTrackInfo}
	, GetDurationMethod(GetClassMethod("getDuration", "()I"))
	, ResetMethod(GetClassMethod("reset", "()V"))
	, GetCurrentPositionMethod(GetClassMethod("getCurrentPosition", "()I"))
	, DidCompleteMethod(GetClassMethod("didComplete", "()Z"))
	, IsLoopingMethod(GetClassMethod("isLooping", "()Z"))
	, IsPlayingMethod(GetClassMethod("isPlaying", "()Z"))
	, IsPreparedMethod(GetClassMethod("isPrepared", "()Z"))
	, SetDataSourceURLMethod(GetClassMethod("setDataSourceURL", "(Ljava/lang/String;)Z"))
	, SetDataSourceArchiveMethod(GetClassMethod("setDataSourceArchive", "(JJ)Z"))
	, SetDataSourceFileMethod(GetClassMethod("setDataSource", "(Ljava/lang/String;JJ)Z"))
	, SetDataSourceAssetMethod(GetClassMethod("setDataSource", "(Landroid/content/res/AssetManager;Ljava/lang/String;JJ)Z"))
	, PrepareMethod(GetClassMethod("prepare", "()V"))
	, PrepareAsyncMethod(GetClassMethod("prepareAsync", "()V"))
	, SeekToMethod(GetClassMethod("seekTo", "(I)V"))
	, SetLoopingMethod(GetClassMethod("setLooping", "(Z)V"))
	, ReleaseMethod(GetClassMethod("release", "()V"))
	, GetVideoHeightMethod(GetClassMethod("getVideoHeight", "()I"))
	, GetVideoWidthMethod(GetClassMethod("getVideoWidth", "()I"))
	, SetVideoEnabledMethod(GetClassMethod("setVideoEnabled", "(Z)V"))
	, SetAudioEnabledMethod(GetClassMethod("setAudioEnabled", "(Z)V"))
	, SetAudioVolumeMethod(GetClassMethod("setAudioVolume", "(F)V"))
	, GetVideoLastFrameDataMethod(GetClassMethod("getVideoLastFrameData", "()Ljava/nio/Buffer;"))
	, StartMethod(GetClassMethod("start", "()V"))
	, PauseMethod(GetClassMethod("pause", "()V"))
	, StopMethod(GetClassMethod("stop", "()V"))
	, GetVideoLastFrameMethod(GetClassMethod("getVideoLastFrame", "(I)Z"))
	, GetAudioTracksMethod(GetClassMethod("GetAudioTracks", "()[Lcom/epicgames/unreal/MediaPlayer14$AudioTrackInfo;"))
	, GetCaptionTracksMethod(GetClassMethod("GetCaptionTracks", "()[Lcom/epicgames/unreal/MediaPlayer14$CaptionTrackInfo;"))
	, GetVideoTracksMethod(GetClassMethod("GetVideoTracks", "()[Lcom/epicgames/unreal/MediaPlayer14$VideoTrackInfo;"))
	, DidResolutionChangeMethod(GetClassMethod("didResolutionChange", "()Z"))
	, GetExternalTextureIdMethod(GetClassMethod("getExternalTextureId", "()I"))
	, UpdateVideoFrameMethod(GetClassMethod("updateVideoFrame", "(I)Lcom/epicgames/unreal/MediaPlayer14$FrameUpdateInfo;"))
	, SetUsingBitmapRendererImageReaderMethod(GetClassMethod("setUsingBitmapRendererImageReader", "(Z)V"))
	, GetVideoLastMediaFrameDataMethod(GetClassMethod("getVideoLastMediaFrameData", "()Lcom/epicgames/unreal/BitmapRenderer$FrameData;"))
{
	VideoTexture = nullptr;
	bVideoTextureValid = false;
	MediaDataSource = nullptr;

	UScale = VScale = 1.0f;
	UOffset = VOffset = 0.0f;

	bTrackInfoSupported = FAndroidMisc::GetAndroidBuildVersion() >= 16;

	if (bTrackInfoSupported)
	{
		SelectTrackMethod = GetClassMethod("selectTrack", "(I)V");
	}

	BRSelection = forceBitmapRenderLegacy ? BitmapRendererSelection::Legacy : BitmapRendererSelection::Unknown;
	UsingVulkanRenderer = vulkanRenderer && FAndroidMisc::ShouldUseVulkan();

	JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();

	// get field IDs for FrameUpdateInfo class members
	FrameUpdateInfoClass = FAndroidApplication::FindJavaClassGlobalRef("com/epicgames/unreal/MediaPlayer14$FrameUpdateInfo");
	FrameUpdateInfo_CurrentPosition = FindField(JEnv, FrameUpdateInfoClass, "CurrentPosition", "I", false);
	FrameUpdateInfo_FrameReady = FindField(JEnv, FrameUpdateInfoClass, "FrameReady", "Z", false);
	FrameUpdateInfo_RegionChanged = FindField(JEnv, FrameUpdateInfoClass, "RegionChanged", "Z", false);
	FrameUpdateInfo_UScale = FindField(JEnv, FrameUpdateInfoClass, "UScale", "F", false);
	FrameUpdateInfo_UOffset = FindField(JEnv, FrameUpdateInfoClass, "UOffset", "F", false);
	FrameUpdateInfo_VScale = FindField(JEnv, FrameUpdateInfoClass, "VScale", "F", false);
	FrameUpdateInfo_VOffset = FindField(JEnv, FrameUpdateInfoClass, "VOffset", "F", false);

	// get field IDs for AudioTrackInfo class members
	AudioTrackInfoClass = FAndroidApplication::FindJavaClassGlobalRef("com/epicgames/unreal/MediaPlayer14$AudioTrackInfo");
	AudioTrackInfo_Index = FindField(JEnv, AudioTrackInfoClass, "Index", "I", false);
	AudioTrackInfo_MimeType = FindField(JEnv, AudioTrackInfoClass, "MimeType", "Ljava/lang/String;", false);
	AudioTrackInfo_DisplayName = FindField(JEnv, AudioTrackInfoClass, "DisplayName", "Ljava/lang/String;", false);
	AudioTrackInfo_Language = FindField(JEnv, AudioTrackInfoClass, "Language", "Ljava/lang/String;", false);
	AudioTrackInfo_Channels = FindField(JEnv, AudioTrackInfoClass, "Channels", "I", false);
	AudioTrackInfo_SampleRate = FindField(JEnv, AudioTrackInfoClass, "SampleRate", "I", false);

	// get field IDs for CaptionTrackInfo class members
	CaptionTrackInfoClass = FAndroidApplication::FindJavaClassGlobalRef("com/epicgames/unreal/MediaPlayer14$CaptionTrackInfo");
	CaptionTrackInfo_Index = FindField(JEnv, CaptionTrackInfoClass, "Index", "I", false);
	CaptionTrackInfo_MimeType = FindField(JEnv, CaptionTrackInfoClass, "MimeType", "Ljava/lang/String;", false);
	CaptionTrackInfo_DisplayName = FindField(JEnv, CaptionTrackInfoClass, "DisplayName", "Ljava/lang/String;", false);
	CaptionTrackInfo_Language = FindField(JEnv, CaptionTrackInfoClass, "Language", "Ljava/lang/String;", false);

	// get field IDs for VideoTrackInfo class members
	VideoTrackInfoClass = FAndroidApplication::FindJavaClassGlobalRef("com/epicgames/unreal/MediaPlayer14$VideoTrackInfo");
	VideoTrackInfo_Index = FindField(JEnv, VideoTrackInfoClass, "Index", "I", false);
	VideoTrackInfo_MimeType = FindField(JEnv, VideoTrackInfoClass, "MimeType", "Ljava/lang/String;", false);
	VideoTrackInfo_DisplayName = FindField(JEnv, VideoTrackInfoClass, "DisplayName", "Ljava/lang/String;", false);
	VideoTrackInfo_Language = FindField(JEnv, VideoTrackInfoClass, "Language", "Ljava/lang/String;", false);
	VideoTrackInfo_BitRate = FindField(JEnv, VideoTrackInfoClass, "BitRate", "I", false);
	VideoTrackInfo_Width = FindField(JEnv, VideoTrackInfoClass, "Width", "I", false);
	VideoTrackInfo_Height = FindField(JEnv, VideoTrackInfoClass, "Height", "I", false);
	VideoTrackInfo_FrameRate = FindField(JEnv, VideoTrackInfoClass, "FrameRate", "F", false);
}

FJavaAndroidMediaPlayer::~FJavaAndroidMediaPlayer()
{
	if (auto Env = FAndroidApplication::GetJavaEnv())
	{
		Env->DeleteGlobalRef(FrameUpdateInfoClass);
		Env->DeleteGlobalRef(AudioTrackInfoClass);
		Env->DeleteGlobalRef(CaptionTrackInfoClass);
		Env->DeleteGlobalRef(VideoTrackInfoClass);
	}
}

int32 FJavaAndroidMediaPlayer::GetDuration()
{
	return CallMethod<int32>(GetDurationMethod);
}

void FJavaAndroidMediaPlayer::Reset()
{
	UScale = VScale = 1.0f;
	UOffset = VOffset = 0.0f;
	CallMethod<void>(ResetMethod);

	// Release media data source if active
	if (MediaDataSource.IsValid())
	{
		FScopeLock ScopeLock(&MediaDataSourcesCS);
		AllMediaDataSources.Remove(reinterpret_cast<int64>(this));
		MediaDataSource.Reset();
	}
}

void FJavaAndroidMediaPlayer::Stop()
{
	CallMethod<void>(StopMethod);
}

int32 FJavaAndroidMediaPlayer::GetCurrentPosition()
{
	int32 position = CallMethod<int32>(GetCurrentPositionMethod);
	return position;
}

bool FJavaAndroidMediaPlayer::IsLooping()
{
	return CallMethod<bool>(IsLoopingMethod);
}

bool FJavaAndroidMediaPlayer::IsPlaying()
{
	return CallMethod<bool>(IsPlayingMethod);
}

bool FJavaAndroidMediaPlayer::IsPrepared()
{
	return CallMethod<bool>(IsPreparedMethod);
}

bool FJavaAndroidMediaPlayer::DidComplete()
{
	return CallMethod<bool>(DidCompleteMethod);
}

bool FJavaAndroidMediaPlayer::SetDataSource(const FString & Url)
{
	UScale = VScale = 1.0f;
	UOffset = VOffset = 0.0f;
	return CallMethod<bool>(SetDataSourceURLMethod, *GetJString(Url));
}

bool FJavaAndroidMediaPlayer::SetDataSource(const TSharedRef<FArchive, ESPMode::ThreadSafe>& Archive)
{
	int64 FileSize = Archive->TotalSize();

	MediaDataSource = MakeShared<FJavaAndroidMediaDataSource, ESPMode::ThreadSafe>(Archive);
	int64 Identifier = reinterpret_cast<int64>(this);
	AddMediaDataSourcePtr(Identifier, MediaDataSource);

	return CallMethod<bool>(SetDataSourceArchiveMethod, Identifier, FileSize);
}

bool FJavaAndroidMediaPlayer::SetDataSource(const FString& MoviePathOnDevice, int64 offset, int64 size)
{
	UScale = VScale = 1.0f;
	UOffset = VOffset = 0.0f;
	return CallMethod<bool>(SetDataSourceFileMethod, *GetJString(MoviePathOnDevice), offset, size);
}

bool FJavaAndroidMediaPlayer::SetDataSource(jobject AssetMgr, const FString& AssetPath, int64 offset, int64 size)
{
	UScale = VScale = 1.0f;
	UOffset = VOffset = 0.0f;
	return CallMethod<bool>(SetDataSourceAssetMethod, AssetMgr, *GetJString(AssetPath), offset, size);
}

bool FJavaAndroidMediaPlayer::Prepare()
{
	// This can return an exception in some cases (URL without internet, for example)
	JNIEnv*	JEnv = FAndroidApplication::GetJavaEnv();
	JEnv->CallVoidMethod(Object, PrepareMethod.Method);
	if (JEnv->ExceptionCheck())
	{
		JEnv->ExceptionDescribe();
		JEnv->ExceptionClear();
		return false;
	}
	return true;
}

bool FJavaAndroidMediaPlayer::PrepareAsync()
{
	// This can return an exception in some cases (URL without internet, for example)
	JNIEnv*	JEnv = FAndroidApplication::GetJavaEnv();
	JEnv->CallVoidMethod(Object, PrepareAsyncMethod.Method);
	if (JEnv->ExceptionCheck())
	{
		JEnv->ExceptionDescribe();
		JEnv->ExceptionClear();
		return false;
	}
	return true;
}

void FJavaAndroidMediaPlayer::SeekTo(int32 Milliseconds)
{
	CallMethod<void>(SeekToMethod, Milliseconds);
}

void FJavaAndroidMediaPlayer::SetLooping(bool Looping)
{
	CallMethod<void>(SetLoopingMethod, Looping);
}

void FJavaAndroidMediaPlayer::Release()
{
	CallMethod<void>(ReleaseMethod);
}

int32 FJavaAndroidMediaPlayer::GetVideoHeight()
{
	return CallMethod<int32>(GetVideoHeightMethod);
}

int32 FJavaAndroidMediaPlayer::GetVideoWidth()
{
	return CallMethod<int32>(GetVideoWidthMethod);
}

void FJavaAndroidMediaPlayer::SetVideoEnabled(bool enabled /*= true*/)
{
	CallMethod<void>(SetVideoEnabledMethod, enabled);
}

void FJavaAndroidMediaPlayer::SetAudioEnabled(bool enabled /*= true*/)
{
	CallMethod<void>(SetAudioEnabledMethod, enabled);
}

void FJavaAndroidMediaPlayer::SetAudioVolume(float Volume)
{
	CallMethod<void>(SetAudioVolumeMethod, Volume);
}

/** This function has to be called before calling any GetVideoLastXXXXXXX() functions */
void FJavaAndroidMediaPlayer::SelectBitmapRenderer()
{
	if (BRSelection == BitmapRendererSelection::Unknown)
	{
		if (UsingVulkanRenderer)
		{
			if (AndroidEnableMediaPlayerBRIR && FAndroidMisc::IsExternalMemoryAndroidHardwareBufferExtensionLoaded())
			{
				BRSelection = BitmapRendererSelection::ImageReader;
			}
			else
			{
				BRSelection = BitmapRendererSelection::Legacy;
			}
		}
		else
		{
			if (AndroidEnableMediaPlayerBRIR)
			{
				BRSelection = BitmapRendererSelection::ImageReader;
			}
			else
			{
				BRSelection = BitmapRendererSelection::Legacy;
			}
		}

		JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
		JEnv->CallVoidMethod(Object, SetUsingBitmapRendererImageReaderMethod.Method, (BRSelection == BitmapRendererSelection::ImageReader));
	}
}

bool FJavaAndroidMediaPlayer::IsUsingBitmapRendererImageReader()
{
	SelectBitmapRenderer();
	return (BRSelection == BitmapRendererSelection::ImageReader);
}

jobject FJavaAndroidMediaPlayer::GetVideoLastMediaFrameData()
{
	if (!IsUsingBitmapRendererImageReader())
	{
		return nullptr;
	}

	JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
	jobject localFrameData = JEnv->CallObjectMethod(Object, GetVideoLastMediaFrameDataMethod.Method);
	if (JEnv->ExceptionCheck())
	{
		JEnv->ExceptionDescribe();
		JEnv->ExceptionClear();
		return nullptr;
	}

	if (localFrameData == nullptr)
	{
		return nullptr;
	}

	jobject globalFrameData = FAndroidApplication::GetJavaEnv()->NewGlobalRef(localFrameData);

	JEnv->DeleteLocalRef(localFrameData);

	return globalFrameData;
}

bool FJavaAndroidMediaPlayer::GetVideoLastFrameData(void* & outPixels, int64 & outCount)
{
	if (IsUsingBitmapRendererImageReader())
	{
		return false;
	}

	// This can return an exception in some cases
	JNIEnv*	JEnv = FAndroidApplication::GetJavaEnv();
	auto buffer = NewScopedJavaObject(JEnv, JEnv->CallObjectMethod(Object, GetVideoLastFrameDataMethod.Method));
	if (JEnv->ExceptionCheck())
	{
		JEnv->ExceptionDescribe();
		JEnv->ExceptionClear();
		return false;
	}
	
	if (buffer)
	{
		outPixels = JEnv->GetDirectBufferAddress(*buffer);
		outCount = JEnv->GetDirectBufferCapacity(*buffer);
	}
	
	if (!buffer || nullptr == outPixels || 0 == outCount)
	{
		return false;
	}
	return true;
}

void FJavaAndroidMediaPlayer::Start()
{
	CallMethod<void>(StartMethod);
}

void FJavaAndroidMediaPlayer::Pause()
{
	CallMethod<void>(PauseMethod);
}

bool FJavaAndroidMediaPlayer::DidResolutionChange()
{
	return CallMethod<bool>(DidResolutionChangeMethod);
}

int32 FJavaAndroidMediaPlayer::GetExternalTextureId()
{
	return CallMethod<int32>(GetExternalTextureIdMethod);
}

bool FJavaAndroidMediaPlayer::UpdateVideoFrame(int32 ExternalTextureId, int32 *CurrentPosition, bool *bRegionChanged)
{
	// This can return an exception in some cases
	JNIEnv*	JEnv = FAndroidApplication::GetJavaEnv();
	auto Result = NewScopedJavaObject(JEnv, JEnv->CallObjectMethod(Object, UpdateVideoFrameMethod.Method, ExternalTextureId));
	if (JEnv->ExceptionCheck())
	{
		JEnv->ExceptionDescribe();
		JEnv->ExceptionClear();
		*CurrentPosition = -1;
		*bRegionChanged = false;
		return false;
	}

	if (!Result)
	{
		*CurrentPosition = -1;
		*bRegionChanged = false;
		return false;
	}

	*CurrentPosition = (int32)JEnv->GetIntField(*Result, FrameUpdateInfo_CurrentPosition);
	bool bFrameReady = (bool)JEnv->GetBooleanField(*Result, FrameUpdateInfo_FrameReady);
	*bRegionChanged = (bool)JEnv->GetBooleanField(*Result, FrameUpdateInfo_RegionChanged);
	UScale = (float)JEnv->GetFloatField(*Result, FrameUpdateInfo_UScale);
	UOffset = (float)JEnv->GetFloatField(*Result, FrameUpdateInfo_UOffset);
	VScale = (float)JEnv->GetFloatField(*Result, FrameUpdateInfo_VScale);
	VOffset = (float)JEnv->GetFloatField(*Result, FrameUpdateInfo_VOffset);
	
	return bFrameReady;
}

bool FJavaAndroidMediaPlayer::GetVideoLastFrame(int32 destTexture)
{
	if (IsUsingBitmapRendererImageReader())
	{
		return false;
	}

	// This can return an exception in some cases
	JNIEnv*	JEnv = FAndroidApplication::GetJavaEnv();
	bool Result = JEnv->CallBooleanMethod(Object, GetVideoLastFrameMethod.Method, destTexture);
	if (JEnv->ExceptionCheck())
	{
		JEnv->ExceptionDescribe();
		JEnv->ExceptionClear();
		return false;
	}
	return Result;
}

FName FJavaAndroidMediaPlayer::GetClassName()
{
	if (FAndroidMisc::GetAndroidBuildVersion() >= 14)
	{
		return FName("com/epicgames/unreal/MediaPlayer14");
	}
	else
	{
		return FName();
	}
}

bool FJavaAndroidMediaPlayer::SelectTrack(int32 index)
{
	if (!bTrackInfoSupported)
	{
		// Just assume it worked
		return true;
	}

	// This can return an exception in some cases
	JNIEnv*	JEnv = FAndroidApplication::GetJavaEnv();
	JEnv->CallVoidMethod(Object, SelectTrackMethod.Method, index);
	if (JEnv->ExceptionCheck())
	{
		JEnv->ExceptionDescribe();
		JEnv->ExceptionClear();
		return false;
	}
	return true;
}

bool FJavaAndroidMediaPlayer::GetAudioTracks(TArray<FAudioTrack>& AudioTracks)
{
	AudioTracks.Empty();

	jobjectArray TrackArray = CallMethod<jobjectArray>(GetAudioTracksMethod);
	if (nullptr != TrackArray)
	{
		bool bIsOptional = false;
		JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
		jsize ElementCount = JEnv->GetArrayLength(TrackArray);

		for (int Index = 0; Index < ElementCount; ++Index)
		{
			auto Track = NewScopedJavaObject(JEnv, JEnv->GetObjectArrayElement(TrackArray, Index));

			int32 AudioTrackIndex = AudioTracks.AddDefaulted();
			FAudioTrack& AudioTrack = AudioTracks[AudioTrackIndex];

			AudioTrack.Index = (int32)JEnv->GetIntField(*Track, AudioTrackInfo_Index);

			AudioTrack.MimeType = FJavaHelper::FStringFromLocalRef(JEnv, (jstring)JEnv->GetObjectField(*Track, AudioTrackInfo_MimeType));
			AudioTrack.DisplayName = FJavaHelper::FStringFromLocalRef(JEnv, (jstring)JEnv->GetObjectField(*Track, AudioTrackInfo_DisplayName));
			AudioTrack.Language = FJavaHelper::FStringFromLocalRef(JEnv, (jstring)JEnv->GetObjectField(*Track, AudioTrackInfo_Language));
			
			AudioTrack.Channels = (int32)JEnv->GetIntField(*Track, AudioTrackInfo_Channels);
			AudioTrack.SampleRate = (int32)JEnv->GetIntField(*Track, AudioTrackInfo_SampleRate);
		}
		JEnv->DeleteGlobalRef(TrackArray);

		return true;
	}
	return false;
}

bool FJavaAndroidMediaPlayer::GetCaptionTracks(TArray<FCaptionTrack>& CaptionTracks)
{
	CaptionTracks.Empty();

	jobjectArray TrackArray = CallMethod<jobjectArray>(GetCaptionTracksMethod);
	if (nullptr != TrackArray)
	{
		bool bIsOptional = false;
		JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
		jsize ElementCount = JEnv->GetArrayLength(TrackArray);

		for (int Index = 0; Index < ElementCount; ++Index)
		{
			auto Track = NewScopedJavaObject(JEnv, JEnv->GetObjectArrayElement(TrackArray, Index));

			int32 CaptionTrackIndex = CaptionTracks.AddDefaulted();
			FCaptionTrack& CaptionTrack = CaptionTracks[CaptionTrackIndex];

			CaptionTrack.Index = (int32)JEnv->GetIntField(*Track, CaptionTrackInfo_Index);

			CaptionTrack.MimeType = FJavaHelper::FStringFromLocalRef(JEnv, (jstring)JEnv->GetObjectField(*Track, CaptionTrackInfo_MimeType));
			CaptionTrack.DisplayName = FJavaHelper::FStringFromLocalRef(JEnv, (jstring)JEnv->GetObjectField(*Track, CaptionTrackInfo_DisplayName));
			CaptionTrack.Language = FJavaHelper::FStringFromLocalRef(JEnv, (jstring)JEnv->GetObjectField(*Track, CaptionTrackInfo_Language));
		}
		JEnv->DeleteGlobalRef(TrackArray);

		return true;
	}
	return false;
}

bool FJavaAndroidMediaPlayer::GetVideoTracks(TArray<FVideoTrack>& VideoTracks)
{
	VideoTracks.Empty();

	jobjectArray TrackArray = CallMethod<jobjectArray>(GetVideoTracksMethod);
	if (nullptr != TrackArray)
	{
		bool bIsOptional = false;
		JNIEnv* JEnv = FAndroidApplication::GetJavaEnv();
		jsize ElementCount = JEnv->GetArrayLength(TrackArray);

		for (int Index = 0; Index < ElementCount; ++Index)
		{
			auto Track = NewScopedJavaObject(JEnv, JEnv->GetObjectArrayElement(TrackArray, Index));

			int32 VideoTrackIndex = VideoTracks.AddDefaulted();
			FVideoTrack& VideoTrack = VideoTracks[VideoTrackIndex];

			VideoTrack.Index = (int32)JEnv->GetIntField(*Track, VideoTrackInfo_Index);

			VideoTrack.MimeType = FJavaHelper::FStringFromLocalRef(JEnv, (jstring)JEnv->GetObjectField(*Track, VideoTrackInfo_MimeType));
			VideoTrack.DisplayName = FJavaHelper::FStringFromLocalRef(JEnv, (jstring)JEnv->GetObjectField(*Track, VideoTrackInfo_DisplayName));
			VideoTrack.Language = FJavaHelper::FStringFromLocalRef(JEnv, (jstring)JEnv->GetObjectField(*Track, VideoTrackInfo_Language));

			VideoTrack.BitRate = (int32)JEnv->GetIntField(*Track, VideoTrackInfo_BitRate);
			VideoTrack.Dimensions = FIntPoint((int32)JEnv->GetIntField(*Track, VideoTrackInfo_Width), (int32)JEnv->GetIntField(*Track, VideoTrackInfo_Height));
			VideoTrack.FrameRate = JEnv->GetFloatField(*Track, VideoTrackInfo_FrameRate);
		}
		JEnv->DeleteGlobalRef(TrackArray);

		return true;
	}
	return false;
}

#endif
