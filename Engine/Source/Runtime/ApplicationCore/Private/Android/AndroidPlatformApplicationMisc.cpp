// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidPlatformApplicationMisc.h"

#include "Android/AndroidApplication.h"
#include "Android/AndroidFeedbackContext.h"
#include "Android/AndroidErrorOutputDevice.h"
#include "Android/AndroidInputInterface.h"
#include "HAL/PlatformMisc.h"
#include "Misc/ConfigCacheIni.h"
#include "Internationalization/Regex.h"
#include "Modules/ModuleManager.h"

void FAndroidApplicationMisc::LoadPreInitModules()
{
	FModuleManager::Get().LoadModule(TEXT("OpenGLDrv"));
#if USE_ANDROID_AUDIO
	FModuleManager::Get().LoadModule(TEXT("AndroidAudio"));
	FModuleManager::Get().LoadModule(TEXT("AudioMixerAndroid"));
#endif
}

class FFeedbackContext* FAndroidApplicationMisc::GetFeedbackContext()
{
	static FAndroidFeedbackContext Singleton;
	return &Singleton;
}

class FOutputDeviceError* FAndroidApplicationMisc::GetErrorOutputDevice()
{
	static FAndroidErrorOutputDevice Singleton;
	return &Singleton;
}


GenericApplication* FAndroidApplicationMisc::CreateApplication()
{
	return FAndroidApplication::CreateAndroidApplication();
}

void FAndroidApplicationMisc::RequestMinimize()
{
#if USE_ANDROID_JNI
	extern void AndroidThunkCpp_Minimize();
	AndroidThunkCpp_Minimize();
#endif
}

bool FAndroidApplicationMisc::IsScreensaverEnabled()
{
#if USE_ANDROID_JNI
	extern bool AndroidThunkCpp_IsScreensaverEnabled();
	return AndroidThunkCpp_IsScreensaverEnabled();
#else
	return false;
#endif
}

bool FAndroidApplicationMisc::ControlScreensaver(EScreenSaverAction Action)
{
#if USE_ANDROID_JNI
	extern void AndroidThunkCpp_KeepScreenOn(bool Enable);
	switch (Action)
	{
		case EScreenSaverAction::Disable:
			// Prevent display sleep.
			AndroidThunkCpp_KeepScreenOn(true);
			break;

		case EScreenSaverAction::Enable:
			// Stop preventing display sleep now that we are done.
			AndroidThunkCpp_KeepScreenOn(false);
			break;
	}
	return true;
#else
	return false;
#endif
}

void FAndroidApplicationMisc::SetGamepadsAllowed(bool bAllowed)
{
	if (FAndroidInputInterface* InputInterface = (FAndroidInputInterface*)FAndroidApplication::Get()->GetInputInterface())
	{
		InputInterface->SetGamepadsAllowed(bAllowed);
	}
}


void FAndroidApplicationMisc::SetGamepadsBlockDeviceFeedback(bool bBlock)
{
	if (FAndroidInputInterface* InputInterface = (FAndroidInputInterface*)FAndroidApplication::Get()->GetInputInterface())
	{
		InputInterface->SetGamepadsBlockDeviceFeedback(bBlock);
	}
}

void FAndroidApplicationMisc::ResetGamepadAssignments()
{
	FAndroidInputInterface::ResetGamepadAssignments();
}

void FAndroidApplicationMisc::ResetGamepadAssignmentToController(int32 ControllerId)
{
	FAndroidInputInterface::ResetGamepadAssignmentToController(ControllerId);
}

bool FAndroidApplicationMisc::IsControllerAssignedToGamepad(int32 ControllerId)
{
	return FAndroidInputInterface::IsControllerAssignedToGamepad(ControllerId);
}

FString FAndroidApplicationMisc::GetGamepadControllerName(int32 ControllerId)
{
	return (FAndroidInputInterface::GetGamepadControllerName(ControllerId).ToString());
}

extern void AndroidThunkCpp_ClipboardCopy(const FString& Str);
void FAndroidApplicationMisc::ClipboardCopy(const TCHAR* Str)
{
#if USE_ANDROID_JNI
	AndroidThunkCpp_ClipboardCopy(Str);
#endif
}

extern FString AndroidThunkCpp_ClipboardPaste();
void FAndroidApplicationMisc::ClipboardPaste(class FString& Result)
{
#if USE_ANDROID_JNI
	Result = AndroidThunkCpp_ClipboardPaste();
#endif
}


void FAndroidApplicationMisc::EnableMotionData(bool bEnable)
{
#if USE_ANDROID_JNI
	extern void AndroidThunkCpp_EnableMotion(bool bEnable);
	AndroidThunkCpp_EnableMotion(bEnable);
#endif
}

struct FScreenDensity
{
	FString Model;
	bool IsRegex;
	int32 Density;
	
	FScreenDensity()
		: Model()
		, IsRegex(false)
		, Density(0)
	{
	}

	bool InitFromString(const FString& InSourceString)
	{
		Model = TEXT("");
		Density = 0;
		IsRegex = false;

		// The initialization is only successful if the Model and Density values can all be parsed from the string
		const bool bSuccessful = FParse::Value(*InSourceString, TEXT("Model="), Model) && FParse::Value(*InSourceString, TEXT("Density="), Density);

		// Regex= is optional, it lets us know if this model requires regular expression matching, which is much more expensive.
		FParse::Bool(*InSourceString, TEXT("IsRegex="), IsRegex);

		return bSuccessful;
	}

	bool IsMatch(const FString& InDeviceModel) const
	{
		if ( IsRegex )
		{
			const FRegexPattern RegexPattern(Model);
			FRegexMatcher RegexMatcher(RegexPattern, InDeviceModel);

			return RegexMatcher.FindNext();
		}
		else
		{
			return Model == InDeviceModel;
		}
	}
};

static float GetWindowUpscaleFactor()
{
	// Determine the difference between the native resolution of the device, and the size of our window,
	// and return that scalar.

	int32_t SurfaceWidth, SurfaceHeight;
	FAndroidWindow::CalculateSurfaceSize(SurfaceWidth, SurfaceHeight);

	FPlatformRect ScreenRect = FAndroidWindow::GetScreenRect();

	const float CalculatedScaleFactor = (float)(FVector2D(ScreenRect.Right - ScreenRect.Left, ScreenRect.Bottom - ScreenRect.Top).Size() / FVector2D(SurfaceWidth, SurfaceHeight).Size());

	return CalculatedScaleFactor;
}

EScreenPhysicalAccuracy FAndroidApplicationMisc::ComputePhysicalScreenDensity(int32& OutScreenDensity)
{
	FString MyDeviceModel = FPlatformMisc::GetDeviceModel();

	TArray<FString> DeviceStrings;
	GConfig->GetArray(TEXT("DeviceScreenDensity"), TEXT("Devices"), DeviceStrings, GEngineIni);

	TArray<FScreenDensity> Devices;
	for ( const FString& DeviceString : DeviceStrings )
	{
		FScreenDensity DensityEntry;
		if ( DensityEntry.InitFromString(DeviceString) )
		{
			Devices.Add(DensityEntry);
		}
	}

	for ( const FScreenDensity& Device : Devices )
	{
		if ( Device.IsMatch(MyDeviceModel) )
		{
			OutScreenDensity = (int32)((float)Device.Density * GetWindowUpscaleFactor());
			return EScreenPhysicalAccuracy::Truth;
		}
	}

#if USE_ANDROID_JNI
	const EScreenPhysicalAccuracy Accuracy  = GetNativePhysicalScreenDensity(OutScreenDensity);
	OutScreenDensity = (int32)((float)OutScreenDensity * GetWindowUpscaleFactor());
	return Accuracy;
#else
	return EScreenPhysicalAccuracy::Unknown;
#endif
}

EScreenPhysicalAccuracy FAndroidApplicationMisc::ComputePhysicalScreenDimensions(FVector2f& OutScreenDimensions, float& OutScreenSize)
{
	FIntVector2 NativeScreenResolution;
	GetNativePhysicalScreenResolution(NativeScreenResolution);

	int32 ScreenDensity;
	const EScreenPhysicalAccuracy Accuracy = GetNativePhysicalScreenDensity(ScreenDensity);

	if (ScreenDensity <= 0 || Accuracy == EScreenPhysicalAccuracy::Unknown)
	{
		UE_LOGF(LogAndroid, Warning, "ComputePhysicalScreenDimensions : Incorrect ScreenDensity or Unknown ScreenPhysicalAccuracy. ScreenDensity: %d, Accuracy: %d",
			ScreenDensity,
			EnumToUnderlyingType(Accuracy));

		return Accuracy;
	}

	OutScreenDimensions = FVector2f(NativeScreenResolution) / (float)ScreenDensity;
	OutScreenSize = OutScreenDimensions.Size();

	UE_LOGF(LogAndroid, Display, "ComputePhysicalScreenDimensions : NativeScreenResolution: %ls, ScreenDensity: %d, OutScreenDimensions (inches): %ls, OutScreenSize (inches): %f",
		*NativeScreenResolution.ToString(),
		ScreenDensity,
		*OutScreenDimensions.ToString(),
		OutScreenSize);

	return Accuracy;
}

EScreenPhysicalAccuracy FAndroidApplicationMisc::GetNativePhysicalScreenDensity(int32& OutScreenDensity)
{
#if USE_ANDROID_JNI
	extern FString AndroidThunkCpp_GetMetaDataString(const FString& Key);
	FString DPIStrings = AndroidThunkCpp_GetMetaDataString(TEXT("unreal.displaymetrics.dpi"));
	TArray<FString> DPIValues;
	DPIStrings.ParseIntoArray(DPIValues, TEXT(","));

	float xdpi, ydpi;
	LexFromString(xdpi, *DPIValues[0]);
	LexFromString(ydpi, *DPIValues[1]);

	OutScreenDensity = (int32)(( xdpi + ydpi ) / 2.0f);

	if ( OutScreenDensity <= 0 || OutScreenDensity > 2000 )
	{
		return EScreenPhysicalAccuracy::Unknown;
	}

	return EScreenPhysicalAccuracy::Approximation;
#else
	return EScreenPhysicalAccuracy::Unknown;
#endif
}

void FAndroidApplicationMisc::GetNativePhysicalScreenResolution(FIntVector2& OutScreenResolution)
{
#if USE_ANDROID_JNI
	extern FString AndroidThunkCpp_GetMetaDataString(const FString& Key);
	FString DisplayString = AndroidThunkCpp_GetMetaDataString(FString("unreal.display"));
	TArray<FString> DisplayValues;
	DisplayString.ParseIntoArray(DisplayValues, TEXT(";"));

	FString WidthString = DisplayValues[DisplayValues.IndexOfByKey(TEXT("realWidth")) + 1];
	FString HeightString = DisplayValues[DisplayValues.IndexOfByKey(TEXT("realHeight")) + 1];

	int Width = FCString::Atoi(*WidthString);
	int Height = FCString::Atoi(*HeightString);

	OutScreenResolution = FIntVector2(Width, Height);
#endif
}
