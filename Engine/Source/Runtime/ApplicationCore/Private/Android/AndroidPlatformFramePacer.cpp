// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidPlatformFramePacer.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "RHI.h"
#include "RHIUtilities.h"

/*******************************************************************
 * FAndroidPlatformRHIFramePacer implementation
 *******************************************************************/

TAutoConsoleVariable<int32> FAndroidPlatformRHIFramePacer::CVarUseSwappyForFramePacing(
	TEXT("a.UseSwappyForFramePacing"),
#if USE_ANDROID_STANDALONE
	// swappy crashes when using single instance and texture views.
	0,
#else
	1,
#endif
	TEXT("True to use Swappy for frame pacing."));

TAutoConsoleVariable<int32> FAndroidPlatformRHIFramePacer::CVarSupportNonVSyncMultipleFrameRates(
	TEXT("a.SupportNonVSyncMultipleFrameRates"),
	0,
	TEXT("Set to True to support frame rates we cannot vsync at. Requires a.UseSwappyForFramePacing=1 for OpenGL ES."));

TAutoConsoleVariable<int32> FAndroidPlatformRHIFramePacer::CVarAllowFrameTimestamps(
	TEXT("a.AllowFrameTimestamps"),
#if USE_ANDROID_STANDALONE
	// frame time stamping when using texture views and single instance in android cannot be used as the frame numbering in android driver won't match up to updates.
	0,
#else
	1,
#endif
	TEXT("True to allow the use use eglGetFrameTimestampsANDROID et al for frame pacing or spew."));

TAutoConsoleVariable<int32> FAndroidPlatformRHIFramePacer::CVarTimeStampErrorRetryCount(
	TEXT("a.TimeStampErrorRetryCount"),
	100,
	TEXT("Number of consecutive frames eglGetFrameTimestampsANDROID can fail before reverting to the naive frame pacer.\n")
	TEXT("Retry count is reset after any successful eglGetFrameTimestampsANDROID calls or a change to sync interval."));


// Legacy pacer stuff
TAutoConsoleVariable<int32> FAndroidPlatformRHIFramePacer::CVarUseGetFrameTimestamps(
	TEXT("a.UseFrameTimeStampsForPacing"),
	0,
	TEXT("True to use eglGetFrameTimestampsANDROID for frame pacing on android (if supported). Only active if a.UseChoreographer is false or the various things needed to use that are not available."));

TAutoConsoleVariable<int32> FAndroidPlatformRHIFramePacer::CVarSpewGetFrameTimestamps(
	TEXT("a.SpewFrameTimeStamps"),
	0,
	TEXT("True to information about frame pacing to the log (if supported). Setting this to 2 results in more detail."));

TAutoConsoleVariable<float> FAndroidPlatformRHIFramePacer::CVarStallSwap(
	TEXT("CriticalPathStall.Swap"),
	0.0f,
	TEXT("Sleep for the given time after the swap (android only for now). Time is given in ms. This is a debug option used for critical path analysis and forcing a change in the critical path."));

TAutoConsoleVariable<int32> FAndroidPlatformRHIFramePacer::CVarDisableOpenGLGPUSync(
	TEXT("r.Android.DisableOpenGLGPUSync"),
	1,
	TEXT("When true, android OpenGL will not prevent the GPU from running more than one frame behind. This will allow higher performance on some devices but increase input latency."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> FAndroidPlatformRHIFramePacer::CVarSwappyGPUFrameTimeFence(
	TEXT("r.Android.SwappyGPUFrameTimeFence"),
	200,
	TEXT("Android Swappy GPU Frame Time Fence in milliseconds. Defaulted to 200 ms"),
	ECVF_Default);

IAndroidFramePacer* FAndroidPlatformRHIFramePacer::FramePacer = nullptr;
int32 FAndroidPlatformRHIFramePacer::InternalFramePace = 0;

void FAndroidPlatformRHIFramePacer::Init(IAndroidFramePacer* InFramePacer)
{
	TArray<int32> RefreshRates = FAndroidMisc::GetSupportedNativeDisplayRefreshRates();
	FString RefreshRatesString;
	for (int32 Rate : RefreshRates)
	{
		RefreshRatesString += FString::Printf(TEXT("%d "), Rate);
	}
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Device supports the following refresh rates %s\n"), *RefreshRatesString);
	FramePacer = InFramePacer;
	FramePacer->Init();
}

bool FAndroidPlatformRHIFramePacer::IsEnabled()
{  
	return FramePacer != nullptr;
}

int32 FAndroidPlatformRHIFramePacer::GetFramePace()
{
	if (InternalFramePace == 0)
	{
		// Internal frame pacing FPS is not set, fall back to generic framepacer based on RHI.SyncInterval
		int32 SyncIntervalFramePace = FGenericPlatformRHIFramePacer::GetFramePaceFromSyncInterval();
		InternalFramePace = SyncIntervalFramePace;
	}

	return InternalFramePace;
}

int32 FAndroidPlatformRHIFramePacer::SetFramePace(int32 InFramePace)
{
	int32 NewFramePace = SupportsFramePace(InFramePace) ? InFramePace : 0;

	// Call generic framepacer to update rhi.SyncInterval where possible
	FGenericPlatformRHIFramePacer::SetFramePaceToSyncInterval(InFramePace);

	/* Update cvar if necessary */
	if (InternalFramePace != NewFramePace)
	{
		InternalFramePace = NewFramePace;
	}

	return NewFramePace;
}

int32 FAndroidPlatformRHIFramePacer::GetLegacySyncInterval()
{
	return RHIGetSyncInterval();
}

void FAndroidPlatformRHIFramePacer::Destroy()
{
	if (FramePacer != nullptr)
	{
		delete FramePacer;
		FramePacer = nullptr;
	}
}
