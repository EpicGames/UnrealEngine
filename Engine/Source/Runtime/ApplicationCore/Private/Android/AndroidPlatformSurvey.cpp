// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AndroidSurvey.cpp: HardwareSurvey implementation
=============================================================================*/

#include "Android/AndroidPlatformSurvey.h"
#include "Android/AndroidWindow.h"

bool FAndroidPlatformSurvey::GetSurveyResults(FHardwareSurveyResults& OutResults, bool bWait)
{
	OutResults.OSBits = 64;

	FCString::Strcpy(OutResults.Platform, *(FAndroidMisc::GetDeviceMake() + TEXT("-") + FAndroidMisc::GetDeviceModel()));
	FCString::Strcpy(OutResults.OSVersion, *FAndroidMisc::GetAndroidVersion());
	FCString::Strcpy(OutResults.OSLanguage, *FPlatformMisc::GetDefaultLocale());
	// @todo vulkan: Get Vulkan version somehow
	FCString::Strcpy(OutResults.RenderingAPI, FAndroidMisc::ShouldUseVulkan() ? TEXT("Vulkan") : *FAndroidMisc::GetGLVersion());
	FCString::Strcpy(OutResults.RHIAdapter.AdapterName, *FAndroidMisc::GetGPUFamily());

	// display 0 is max size
	int32 ScreenWidth, ScreenHeight;
	FAndroidWindow::CalculateSurfaceSize(ScreenWidth, ScreenHeight);

	OutResults.Displays[0].CurrentModeWidth = ScreenWidth;
	OutResults.Displays[0].CurrentModeHeight = ScreenHeight;

	OutResults.CPUCount = FPlatformMisc::NumberOfCores();


	// display 1 is current size
	FPlatformRect ViewSize = FAndroidWindow::GetScreenRect();
	OutResults.Displays[1].CurrentModeWidth = ViewSize.Right - ViewSize.Left;
	OutResults.Displays[1].CurrentModeHeight = ViewSize.Bottom - ViewSize.Top;
	return true;
}
