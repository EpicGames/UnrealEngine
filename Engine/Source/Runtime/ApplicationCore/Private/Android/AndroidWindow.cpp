// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidWindow.h"
#include "Android/AndroidWindowUtils.h"
#include "Android/AndroidEventManager.h"
#include "Android/AndroidDynamicRHI.h"
#include "Android/AndroidJniGameActivity.h"

#if USE_ANDROID_JNI
#include <android/native_window.h>
#include <android/native_window_jni.h>
#endif
#include "HAL/OutputDevices.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CommandLine.h"
#include "HAL/PlatformStackWalk.h"


int32 GAndroidWindowDPI = 0;
static FAutoConsoleVariableRef CVarAndroidWindowDPI(
	TEXT("Android.WindowDPI"),
	GAndroidWindowDPI,
	TEXT("Values > 0 will set the system window resolution (i.e. swap buffer) to achieve the DPI requested.\n")
	TEXT("default: 0"),
	ECVF_ReadOnly);

int32 GAndroidWindowDPIQueryMethod = 0;
static FAutoConsoleVariableRef CVarAndroidWindowDPIQueryMethod(
	TEXT("Android.DPIQueryMethod"),
	GAndroidWindowDPIQueryMethod,
	TEXT("The method used to determine the native screen DPI when calculating the scale factor required to achieve the requested Android.WindowDPI.\n")
	TEXT("0: Use displaymetrics xdpi/ydpi (default)\n")
	TEXT("1: Use displaymetrics densityDpi"),
	ECVF_ReadOnly);

int32 GAndroid3DSceneMaxDesiredPixelCount = 0;
static FAutoConsoleVariableRef CVarAndroid3DSceneMaxDesiredPixelCount(
	TEXT("Android.3DSceneMaxDesiredPixelCount"),
	GAndroid3DSceneMaxDesiredPixelCount,
	TEXT("Works in conjunction with Android.WindowDPI, this specifies a maximum pixel count for the 3D scene.\n")
	TEXT("Values >0 will be used to scale down the 3D scene (equivalent to setting r.screenpercentage)\n")
	TEXT("such that the 3d scene will be no more than 3DSceneMaxDesiredPixelCount pixels.\n")
	TEXT("This is only applied if the 3d scene pixel at the 'Android.WindowDPI' DPI goes above the specified value.\n")
	TEXT("default: 0"),
	ECVF_ReadOnly);

int32 GAndroid3DSceneMinDPI = 0;
static FAutoConsoleVariableRef CVarAndroid3DSceneMinDPI(
	TEXT("Android.3DSceneMinDPI"),
	GAndroid3DSceneMinDPI,
	TEXT("Works in conjunction with Android.3DSceneMaxDesiredPixelCount, and specifies a minimum DPI level for the 3D scene.\n")
	TEXT("Values >0 specify an absolute minimum 3D scene DPI, if Android.3DSceneMaxDesiredPixelCount causes the 3d resolution to be below\n")
	TEXT("Android.3DSceneMinDPI then the scale factor will be set to achieve the minimum dpi.\n")
	TEXT("Useful to achieve a minimum quality level on low DPI devices at the expense of GPU performance.\n")
	TEXT("default: 0"),
	ECVF_ReadOnly);

int GAndroidPropagateAlpha = 0;

struct FAndroidCachedWindowRectParams
{
	int32 WindowWidth = -1;
	int32 WindowHeight = -1;

	float ContentScaleFactor = -1.0f;
	int32 MobileResX = -1;
	int32 MobileResY = -1;

	int32 WindowDPI = -1;
	int32 SceneMinDPI = -1;
	int32 SceneMaxDesiredPixelCount = -1;

	ANativeWindow* Window_EventThread = nullptr;

	bool operator ==(const FAndroidCachedWindowRectParams& Rhs) const { return FMemory::Memcmp(this, &Rhs, sizeof(Rhs)) == 0; }
	bool operator !=(const FAndroidCachedWindowRectParams& Rhs) const { return FMemory::Memcmp(this, &Rhs, sizeof(Rhs)) != 0; }
};


// Cached calculated screen resolution
static FAndroidCachedWindowRectParams CachedWindowRect;
static FAndroidCachedWindowRectParams CachedWindowRect_EventThread;

const FAndroidCachedWindowRectParams& GetCachedRect(bool bUseEventThreadWindow)
{
	return bUseEventThreadWindow ? CachedWindowRect_EventThread : CachedWindowRect;
}

static void ClearCachedWindowRects()
{
	CachedWindowRect = FAndroidCachedWindowRectParams();
	CachedWindowRect_EventThread = FAndroidCachedWindowRectParams();
}

int32 GSurfaceViewWidth = -1;
int32 GSurfaceViewHeight = -1;

void* GAndroidWindowOverride = nullptr;
static ANativeWindow* GAcquiredWindow = nullptr;

void* FAndroidWindow::NativeWindow = NULL;

FAndroidWindow::~FAndroidWindow()
{
	//       Use NativeWindow_Destroy() instead.
}

TSharedRef<FAndroidWindow> FAndroidWindow::Make()
{
	TSharedRef<FAndroidWindow> Window = MakeShareable( new FAndroidWindow() );
	int Num = FAndroidWindowManager::Get().AddWindow(Window);

	if( Num == 1)  // special case for native activity, the first window gets any pre-existing ANative window. if there is not yet a pending window then the event thread will set it later.
	{
		// lock both the FAndroidWindow and the pending window to transfer the pending window across
		FNativeAccessor MyANativeAccessor = Window->GetANativeAccessor(true);
		FAndroidWindowManager::FPendingWindowAccessor PendingAccessor = FAndroidWindowManager::GetPendingWindowAccessor();

		MyANativeAccessor.SetANativeWindow(PendingAccessor.GetPreviousANativeWindow()); // first set previous handle, this is retained when the current handle is set next
		MyANativeAccessor.SetANativeWindow(PendingAccessor.GetANativeWindow());
		PendingAccessor.SetANativeWindow(nullptr);
	}

	return Window;
}

FAndroidWindow::FAndroidWindow()
{
}

void FAndroidWindow::Destroy()
{
	FAndroidWindowManager::Get().RemoveWindow(SharedThis(this));
}

void FAndroidWindow::Initialize( class FAndroidApplication* const Application, const TSharedRef< FGenericWindowDefinition >& InDefinition, const TSharedPtr< FAndroidWindow >& InParent, const bool bShowImmediately )
{
	OwningApplication = Application;
	Definition = InDefinition;
}

bool FAndroidWindow::GetFullScreenInfo( int32& X, int32& Y, int32& Width, int32& Height ) const
{
	FPlatformRect ScreenRect = GetScreenRect();

	X = ScreenRect.Left;
	Y = ScreenRect.Top;
	Width = ScreenRect.Right - ScreenRect.Left;
	Height = ScreenRect.Bottom - ScreenRect.Top;

	return true;
}


void FAndroidWindow::SetOSWindowHandle(void* InWindow)
{
	// not expecting a window to be supplied on android.
	check(InWindow == nullptr);
}


//This function is declared in the Java-defined class, GameActivity.java: "public native void nativeSetObbInfo(String PackageName, int Version, int PatchVersion);"
static bool GAndroidIsPortrait = false;
static EDeviceScreenOrientation GDeviceScreenOrientation = EDeviceScreenOrientation::Unknown;
static int GAndroidDepthBufferPreference = 0;
static FVector4 GAndroidPortraitSafezone(-1.0f, -1.0f, -1.0f, -1.0f), GAndroidLandscapeSafezone(-1.0f, -1.0f, -1.0f, -1.0f), *GAndroidLastSafezone = nullptr;
#if USE_ANDROID_JNI
void JNICALL UE::Jni::FGameActivity::nativeSetWindowInfo(JNIEnv* jenv, jobject thiz, jboolean bIsPortrait, jint DepthBufferPreference, jint PropagateAlpha)
{
	ClearCachedWindowRects();
	GAndroidIsPortrait = bIsPortrait == JNI_TRUE;
	GAndroidDepthBufferPreference = DepthBufferPreference;
	GAndroidPropagateAlpha = PropagateAlpha;
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("App is running in %s\n"), GAndroidIsPortrait ? TEXT("Portrait") : TEXT("Landscape"));
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("AndroidPropagateAlpha =  %d\n"), GAndroidPropagateAlpha);
}

void JNICALL UE::Jni::FGameActivity::nativeSetSurfaceViewInfo(JNIEnv* jenv, jobject thiz, jint width, jint height)
{
	STANDALONE_DEBUG_LOG( TEXT("nativeSetSurfaceViewInfo prev[width=%d, height=%d] new[width=%d, height=%d]"), GSurfaceViewWidth, GSurfaceViewHeight, width, height);

	if (GAndroidWindowOverride != nullptr && (width != GSurfaceViewWidth || height != GSurfaceViewHeight))	
	{
		GSurfaceViewWidth = width;
		GSurfaceViewHeight = height;
		FAppEventManager::GetInstance()->EnqueueAppEvent(APP_EVENT_STATE_WINDOW_RESIZED, FAppEventData((ANativeWindow*)GAndroidWindowOverride));
		STANDALONE_DEBUG_LOG(TEXT("nativeSetSurfaceViewInfo width=%d and height=%d"), GSurfaceViewWidth, GSurfaceViewHeight);
	}
}

void JNICALL UE::Jni::FGameActivity::nativeSetSafezoneInfo(JNIEnv* jenv, jobject thiz, jboolean bIsPortrait, jfloat left, jfloat top, jfloat right, jfloat bottom)
{
	FVector4 NewSafezone(left, top, right, bottom);
	FVector4& TargetSafezone = bIsPortrait ? GAndroidPortraitSafezone : GAndroidLandscapeSafezone;
	bool SafezoneUpdated = TargetSafezone != NewSafezone || GAndroidLastSafezone != &TargetSafezone;
	GAndroidLastSafezone = &TargetSafezone;
	TargetSafezone = NewSafezone;
	
#if USE_ANDROID_EVENTS
	if( SafezoneUpdated )
	{
		FAppEventManager::GetInstance()->EnqueueAppEvent(APP_EVENT_STATE_SAFE_ZONE_UPDATED);
	}
#endif
	UE_LOGF(LogAndroid, Log, "nativeSetSafezoneInfo bIsPortrait=%d, left=%f, top=%f, right=%f, bottom=%f", bIsPortrait ? 1 : 0, left, top, right, bottom);
}
#endif

bool FAndroidWindow::bAreCachedNativeDimensionsValid = false;
int32 FAndroidWindow::CachedNativeWindowWidth = -1;
int32 FAndroidWindow::CachedNativeWindowHeight = -1;

bool FAndroidWindow::IsPortraitOrientation()
{
	return GAndroidIsPortrait;
}

FVector4 FAndroidWindow::GetSafezone(bool bPortrait)
{
	return bPortrait ? GAndroidPortraitSafezone : GAndroidLandscapeSafezone;
}

int32 FAndroidWindow::GetDepthBufferPreference()
{
	return GAndroidDepthBufferPreference;
}

void FAndroidWindow::InvalidateCachedScreenRect()
{
	ClearCachedWindowRects();
}

void FAndroidWindow::AcquireWindowRef(ANativeWindow* InWindow)
{
#if USE_ANDROID_JNI
#if USE_ANDROID_STANDALONE
	STANDALONE_DEBUG_LOG(TEXT("AcquireWindowRef USE_ANDROID_JNI is enabled: InWindow=%p, GAcquiredWindow=%p, GAndroidWindowOverride=%p"), InWindow, GAcquiredWindow, GAndroidWindowOverride);

	if (InWindow == nullptr)
	{
		UE_LOGF(LogAndroid, Log, "FAndroidWindow::AcquireWindowRef skipped because InWindow is null.");
		return;
	}

	if (GAcquiredWindow == InWindow)
	{
		STANDALONE_DEBUG_LOG(TEXT("AcquireWindowRef USE_ANDROID_JNI is enabled: %p and GAcquiredWindow == InWindow"), InWindow);
		return;
	}

	if (GAcquiredWindow != nullptr)
	{
		STANDALONE_DEBUG_LOG(TEXT("AcquireWindowRef USE_ANDROID_JNI is enabled: %p and GAcquiredWindow != nullptr"), InWindow);

		ReleaseWindowRef(GAcquiredWindow);
	}

	check(GAcquiredWindow == NULL);
	STANDALONE_DEBUG_LOG(TEXT("AcquireWindowRef USE_ANDROID_JNI is enabled: %p and ANativeWindow_acquire"), InWindow);

	// Added logic to store the Acquired window and when calling ReleaseWindowRef, check if the window is the same and only release if it matches
	// This logic is to deal with the fact Android lifecycles for activities can overlap. ideally we would create a context based container to manage this
	// but for now this is a useful protection.
	GAcquiredWindow = InWindow;
	STANDALONE_DEBUG_LOG(TEXT("FAndroidWindow::AcquireWindowRef GAcquiredWindow=%p, GAndroidWindowOverride=%p"), GAcquiredWindow, GAndroidWindowOverride);
#endif //USE_ANDROID_STANDALONE

	ANativeWindow_acquire(InWindow);

#endif
}

void FAndroidWindow::ReleaseWindowRef(ANativeWindow* InWindow)
{
#if USE_ANDROID_JNI
#if USE_ANDROID_STANDALONE
	if (GAcquiredWindow == nullptr && InWindow == nullptr)
	{
		STANDALONE_DEBUG_LOG(TEXT("ReleaseWindowRef skipped because GAcquiredWindow is null.  Window %p reference will not be released."), InWindow);
		return;
	}

	ANativeWindow* ReleaseWindow = GAcquiredWindow;
	if (InWindow == nullptr || GAcquiredWindow == InWindow)
	{
		InWindow = GAcquiredWindow;
		GAcquiredWindow = nullptr;
	}

	STANDALONE_DEBUG_LOG(TEXT("ReleaseWindowRef using window: %p"), InWindow);
#endif //USE_ANDROID_STANDALONE
	ANativeWindow_release(InWindow);
#endif
}

void FAndroidWindow::SetHardwareWindow_EventThread(void* InWindow)
{
#if USE_ANDROID_EVENTS
	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("SetHardwareWindow_EventThread(USE_ANDROID_EVENTS) -> InWindow(%p), GAndroidWindowOverride(%p), IsInAndroidEventThread()=%d"), InWindow, GAndroidWindowOverride, IsInAndroidEventThread());

	check(IsInAndroidEventThread());
#endif

#if USE_ANDROID_STANDALONE
	if (GAndroidWindowOverride && InWindow != GAndroidWindowOverride)
	{
		STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("SetHardwareWindow_EventThread(USE_ANDROID_STANDALONE) -> InWindow(%p) is not current GAndroidWindowOverride(%p)"), InWindow, GAndroidWindowOverride);
	}
#endif

	//using raw native window handle for now. Could be changed to use AndroidWindow later if needed
	NativeWindow = InWindow;
}

void* FAndroidWindow::GetHardwareWindow_EventThread()
{
#if USE_ANDROID_STANDALONE
	void* result = GAndroidWindowOverride != nullptr ? GAndroidWindowOverride : NativeWindow;
	if (result != nullptr)
	{
		if (GAndroidWindowOverride != NativeWindow)
		{
			STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("GetHardwareWindow_EventThread GAndroidWindowOverride=%p, NativeWindow=%p"), GAndroidWindowOverride, NativeWindow);
		}
		return result;
	}
	else
	{
		//STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("ERROR: GetHardwareWindow_EventThread has invalid window!!! GAndroidWindowOverride=%p, NativeWindow=%p"), GAndroidWindowOverride, NativeWindow);
		return result;
	}
#endif

	return NativeWindow;
}

bool FAndroidWindow::WaitForWindowDimensions()
{
	while (!bAreCachedNativeDimensionsValid)
	{
		if (IsEngineExitRequested()
#if USE_ANDROID_EVENTS
		|| FAppEventManager::GetInstance()->WaitForEventInQueue(EAppEventState::APP_EVENT_STATE_ON_DESTROY, 0.0f)
#endif
			)
		{
			// Application is shutting down
			return false;
		}
		FPlatformProcess::Sleep(0.001f);
	}
	return true;
}

// To be called during initialization from the event thread.
// once set the dimensions are 'valid' and further changes are updated via FAppEventManager::Tick 
void FAndroidWindow::SetWindowDimensions_EventThread(ANativeWindow* DimensionWindow)
{
	if (bAreCachedNativeDimensionsValid == false)
	{
#if USE_ANDROID_JNI
		CachedNativeWindowWidth = ANativeWindow_getWidth(DimensionWindow);
		CachedNativeWindowHeight = ANativeWindow_getHeight(DimensionWindow);
#else
		int32 ResWidth, ResHeight;
		if (FPlatformMisc::GetOverrideResolution(ResWidth, ResHeight))
		{
			CachedNativeWindowWidth = ResWidth;
			CachedNativeWindowHeight = ResHeight;
		}
#endif
		FPlatformMisc::MemoryBarrier();
		bAreCachedNativeDimensionsValid = true;
	}
}

// Update the dimensions from FAppEventManager::Tick based on messages posted from the event thread.
void FAndroidWindow::EventManagerUpdateWindowDimensions(int32 Width, int32 Height)
{
	check(bAreCachedNativeDimensionsValid);
	check(Width >= 0 && Height >= 0);

#if USE_ANDROID_STANDALONE
	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("FAndroidWindow::EventManagerUpdateWindowDimensions GAndroidWindowOverride=%p, Width=%d, Height=%d, GSurfaceViewWidth=%d, GSurfaceViewHeight=%d, CachedNativeWindowWidth=%d, CachedNativeWindowHeight=%d"),
		GAndroidWindowOverride, Width, Height, GSurfaceViewWidth, GSurfaceViewHeight, CachedNativeWindowWidth, CachedNativeWindowHeight);

	if (GAndroidWindowOverride && GSurfaceViewWidth > 0)
	{
		Width = GSurfaceViewWidth;
		Height = GSurfaceViewHeight;
	}
#endif

	bool bChanged = CachedNativeWindowWidth != Width || CachedNativeWindowHeight != Height;

	CachedNativeWindowWidth = Width;
	CachedNativeWindowHeight = Height;

	if (bChanged)
	{
		InvalidateCachedScreenRect();
	}
}

bool FAndroidWindow::GetNativeWindowResolution(int32_t& OutWidth, int32_t& OutHeight) const
{
#if USE_ANDROID_JNI

	if (NativeWindow == nullptr) return false;

	ANativeWindow* AndroidWindow = static_cast<ANativeWindow*>(NativeWindow);

	OutWidth = ANativeWindow_getWidth(AndroidWindow);
	OutHeight = ANativeWindow_getHeight(AndroidWindow);

	return true;

#else

	// Android without JNI?
	// Just making the function work for unexpected platforms
	return FPlatformMisc::GetOverrideResolution(OutWidth, OutHeight);

#endif
}

void* FAndroidWindow::WaitForHardwareWindow()
{
	// Sleep if the hardware window isn't currently available.
	// The Window may not exist if the activity is pausing/resuming, in which case we make this thread wait
	// This case will come up frequently as a result of the DON flow in Gvr.
	// Until the app is fully resumed. It would be nicer if this code respected the lifecycle events
	// of an android app instead, but all of those events are handled on a separate thread and it would require
	// significant re-architecturing to do.

	// Before sleeping, we peek into the event manager queue to see if it contains an ON_DESTROY event, 
	// in which case, we exit the loop to allow the application to exit before a window has been created.
	// It is not sufficient to check the IsEngineExitRequested() global function, as the handler reacting to the APP_EVENT_STATE_ON_DESTROY
	// may be running in the same thread as this method and therefore lead to a deadlock.

	void* WindowEventThread = GetHardwareWindow_EventThread();
	while (WindowEventThread == nullptr)
	{
#if USE_ANDROID_EVENTS
		if (IsEngineExitRequested() || FAppEventManager::GetInstance()->WaitForEventInQueue(EAppEventState::APP_EVENT_STATE_ON_DESTROY, 0.0f))
		{
			// Application is shutting down soon, abort the wait and return nullptr
			return nullptr;
		}
#endif
		FPlatformProcess::Sleep(0.001f);
		WindowEventThread = GetHardwareWindow_EventThread();
	}
	return WindowEventThread;
}

#if USE_ANDROID_JNI
extern bool AndroidThunkCpp_IsXRImmersiveHMDApplication();
#endif

static bool IsCachedRectValid(bool bUseEventThreadWindow, const FAndroidCachedWindowRectParams& TestRect)
{
	// window must be valid when bUseEventThreadWindow and null when !bUseEventThreadWindow.
	check((TestRect.Window_EventThread != nullptr) == bUseEventThreadWindow);

	const FAndroidCachedWindowRectParams& CachedRect = GetCachedRect(bUseEventThreadWindow);

	bool bValidCache = true;

	if (CachedRect.ContentScaleFactor != TestRect.ContentScaleFactor)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("***** RequestedContentScaleFactor different %f != %f, not using res cache (%d)"), TestRect.ContentScaleFactor, CachedWindowRect.ContentScaleFactor, (int32)bUseEventThreadWindow);
		bValidCache = false;
	}

	if (CachedRect.MobileResX != TestRect.MobileResX)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("***** RequestedMobileResX different %d != %d, not using res cache (%d)"), TestRect.MobileResX, CachedWindowRect.MobileResX, (int32)bUseEventThreadWindow);
		bValidCache = false;
	}

	if (CachedRect.MobileResY != TestRect.MobileResY)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("***** RequestedMobileResY different %d != %d, not using res cache (%d)"), TestRect.MobileResY, CachedWindowRect.MobileResY, (int32)bUseEventThreadWindow);
		bValidCache = false;
	}

	if (CachedRect.Window_EventThread != TestRect.Window_EventThread)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("***** Window different, not using res cache (%d)"), (int32)bUseEventThreadWindow);
		bValidCache = false;
	}

	if (CachedRect.WindowDPI != TestRect.WindowDPI)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("***** WindowDPI is %d, not using res cache (%d)"), TestRect.WindowDPI, (int32)bUseEventThreadWindow);
		bValidCache = false;
	}

	if (CachedRect.SceneMaxDesiredPixelCount != TestRect.SceneMaxDesiredPixelCount)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("***** SceneMaxDesiredPixelCount is %d, not using res cache (%d)"), TestRect.SceneMaxDesiredPixelCount, (int32)bUseEventThreadWindow);
		bValidCache = false;
	}

	if (CachedRect.SceneMinDPI != TestRect.SceneMinDPI)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("***** SceneMinDPI is %d, not using res cache (%d)"), TestRect.SceneMinDPI, (int32)bUseEventThreadWindow);
		bValidCache = false;
	}

	if (CachedRect.WindowWidth <= 8)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("***** WindowWidth is %d, not using res cache (%d)"), CachedRect.WindowWidth, (int32)bUseEventThreadWindow);
		bValidCache = false;
	}

	return bValidCache;
}

void CacheRect(bool bUseEventThreadWindow, const FAndroidCachedWindowRectParams& NewValues)
{
	check(NewValues.Window_EventThread != nullptr || !bUseEventThreadWindow);

	(bUseEventThreadWindow ? CachedWindowRect_EventThread : CachedWindowRect) = NewValues;

	UE_LOGF(LogAndroid, Log, "***** Cached WindowRect %d, %d (%d)", NewValues.WindowWidth, NewValues.WindowHeight, (int32)bUseEventThreadWindow);
}

struct FAndroidDisplayInfo
{
	FIntVector2 WindowDims;
	float SceneScaleFactor;
};

static FAndroidDisplayInfo GetAndroidDisplayInfoFromDPITargets(int32 TargetDPI, int32 SceneMaxDesiredPixelCount, int32 LowerLimit3DDPI);

FPlatformRect FAndroidWindow::GetScreenRect(bool bUseEventThreadWindow)
{
	int32 OverrideResX, OverrideResY;
	// allow a subplatform to dictate resolution - we can't easily subclass FAndroidWindow the way its used
	if (FPlatformMisc::GetOverrideResolution(OverrideResX, OverrideResY))
	{
		FPlatformRect Rect;
		Rect.Left = Rect.Top = 0;
		Rect.Right = OverrideResX;
		Rect.Bottom = OverrideResY;

		return Rect;
	}

	// too much of the following code needs JNI things, just assume override
#if !USE_ANDROID_JNI

	UE_LOGF(LogAndroid, Fatal, "FAndroidWindow::CalculateSurfaceSize currently expects non-JNI platforms to override resolution");
	return FPlatformRect();
#else

	static const bool bIsXRImmersiveHMDApp = AndroidThunkCpp_IsXRImmersiveHMDApplication();

	// CSF is a multiplier to 1280x720
	static IConsoleVariable* CVarScale = IConsoleManager::Get().FindConsoleVariable(TEXT("r.MobileContentScaleFactor"));
	// If the app is for XR Mobile then always use 0 as ScaleFactor (to match window size).
	float RequestedContentScaleFactor = bIsXRImmersiveHMDApp ? 0.0f : CVarScale->GetFloat();

	FString CmdLineCSF;
	if (FParse::Value(FCommandLine::Get(), TEXT("mcsf="), CmdLineCSF, false))
	{
		RequestedContentScaleFactor = FCString::Atof(*CmdLineCSF);
	}

	static IConsoleVariable* CVarResX = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.DesiredResX"));
	static IConsoleVariable* CVarResY = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.DesiredResY"));
	int32 RequestedResX = bIsXRImmersiveHMDApp ? 0 : CVarResX->GetInt();
	int32 RequestedResY = bIsXRImmersiveHMDApp ? 0 : CVarResY->GetInt();

	FString CmdLineMDRes;
	if (FParse::Value(FCommandLine::Get(), TEXT("mobileresx="), CmdLineMDRes, false))
	{
		RequestedResX = FCString::Atoi(*CmdLineMDRes);
	}
	if (FParse::Value(FCommandLine::Get(), TEXT("mobileresy="), CmdLineMDRes, false))
	{
		RequestedResY = FCString::Atoi(*CmdLineMDRes);
	}

	// since orientation won't change on Android, use cached results if still valid. Different cache is maintained for event_thread flavor.
	ANativeWindow* Window = bUseEventThreadWindow ? (ANativeWindow*)FAndroidWindow::GetHardwareWindow_EventThread() : nullptr;
	FAndroidCachedWindowRectParams CurrentParams;
	CurrentParams.ContentScaleFactor = RequestedContentScaleFactor;
	CurrentParams.MobileResX = RequestedResX;
	CurrentParams.MobileResY = RequestedResY;
	CurrentParams.Window_EventThread = Window;
	CurrentParams.WindowDPI = GAndroidWindowDPI;
	CurrentParams.SceneMinDPI = FMath::Min(GAndroid3DSceneMinDPI, GAndroidWindowDPI);
	CurrentParams.SceneMaxDesiredPixelCount = GAndroid3DSceneMaxDesiredPixelCount;

	bool bComputeRect = !IsCachedRectValid(bUseEventThreadWindow, CurrentParams);
	if (bComputeRect)
	{
		// currently hardcoding resolution

		// get the aspect ratio of the physical screen
		int32 ScreenWidth, ScreenHeight;
		CalculateSurfaceSize(ScreenWidth, ScreenHeight, bUseEventThreadWindow);

		static auto* MobileHDRCvar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.MobileHDR"));
		const bool bMobileHDR = (MobileHDRCvar && MobileHDRCvar->GetValueOnAnyThread() == 1);
		UE_LOGF(LogAndroid, Log, "Mobile HDR: %ls", bMobileHDR ? TEXT("YES") : TEXT("no"));

		if (!bIsXRImmersiveHMDApp && (GAndroidWindowOverride == nullptr))
		{
			if (CurrentParams.WindowDPI && RequestedResX == 0 && RequestedResY == 0)
			{
				FAndroidDisplayInfo Info = GetAndroidDisplayInfoFromDPITargets(CurrentParams.WindowDPI, CurrentParams.SceneMaxDesiredPixelCount, CurrentParams.SceneMinDPI);
				ScreenWidth = Info.WindowDims.X;
				ScreenHeight = Info.WindowDims.Y;
				if (IsInGameThread())
				{
					static IConsoleVariable* CVarSSP = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SecondaryScreenPercentage.GameViewport"));
					CVarSSP->Set((float)Info.SceneScaleFactor * 100.0f);
				}
			}
			else
			{
				AndroidWindowUtils::ApplyContentScaleFactor(ScreenWidth, ScreenHeight);
			}
		}

		// save for future calls
		CurrentParams.WindowWidth = ScreenWidth;
		CurrentParams.WindowHeight = ScreenHeight;
		CacheRect(bUseEventThreadWindow, CurrentParams);
	}

	const FAndroidCachedWindowRectParams& CachedRect = GetCachedRect(bUseEventThreadWindow);

	// create rect and return
	FPlatformRect ScreenRect;
	ScreenRect.Left = 0;
	ScreenRect.Top = 0;
	ScreenRect.Right = CachedRect.WindowWidth;
	ScreenRect.Bottom = CachedRect.WindowHeight;

	return ScreenRect;
#endif
}

void FAndroidWindow::CalculateSurfaceSize(int32_t& SurfaceWidth, int32_t& SurfaceHeight, bool bUseEventThreadWindow)
{
	// allow a subplatform to dictate resolution - we can't easily subclass FAndroidWindow the way its used
	if (FPlatformMisc::GetOverrideResolution(SurfaceWidth, SurfaceHeight))
	{
		return;
	}

	// too much of the following code needs JNI things, just assume override
#if !USE_ANDROID_JNI

	UE_LOGF(LogAndroid, Fatal, "FAndroidWindow::CalculateSurfaceSize currently expects non-JNI platforms to override resolution");

#else
	STANDALONE_DEBUG_LOGf(LogAndroid, TEXT("::CalculateSurfaceSize(USE_ANDROID_JNI) -> bUseEventThreadWindow=%d, GAndroidWindowOverride(%p), IsInAndroidEventThread()=%d"), bUseEventThreadWindow, GAndroidWindowOverride, IsInAndroidEventThread());

	if (bUseEventThreadWindow)
	{
		check(IsInAndroidEventThread());
		ANativeWindow* WindowEventThread = (ANativeWindow*)GetHardwareWindow_EventThread();
		check(WindowEventThread);

		SurfaceWidth = (GSurfaceViewWidth > 0) ? GSurfaceViewWidth : ANativeWindow_getWidth(WindowEventThread);
		SurfaceHeight = (GSurfaceViewHeight > 0) ? GSurfaceViewHeight : ANativeWindow_getHeight(WindowEventThread);
	}
	else
	{
#if !USE_ANDROID_STANDALONE
		FAndroidWindow::WaitForWindowDimensions();
#endif
		SurfaceWidth = (GSurfaceViewWidth > 0) ? GSurfaceViewWidth : CachedNativeWindowWidth;
		SurfaceHeight = (GSurfaceViewHeight > 0) ? GSurfaceViewHeight : CachedNativeWindowHeight;
	}

#if !USE_ANDROID_STANDALONE
	// some phones gave it the other way (so, if swap if the app is landscape, but width < height)
	if ((GAndroidIsPortrait && SurfaceWidth > SurfaceHeight) ||
		(!GAndroidIsPortrait && SurfaceWidth < SurfaceHeight))
	{
		Swap(SurfaceWidth, SurfaceHeight);
	}
#endif

	// ensure the size is divisible by a specified amount
	// do not convert to a surface size that is larger than native resolution
	
#if USE_ANDROID_STANDALONE
	const int DividableBy = 1;	// don't change size of external window 
#else
	// Mobile VR doesn't need buffer quantization as Unreal never renders directly to the buffer in VR mode. 
	static const bool bIsXRImmersiveHMDApp = AndroidThunkCpp_IsXRImmersiveHMDApplication();
	const int DividableBy = bIsXRImmersiveHMDApp ? 1 : 8;
#endif

	SurfaceWidth = (SurfaceWidth / DividableBy) * DividableBy;
	SurfaceHeight = (SurfaceHeight / DividableBy) * DividableBy;
#endif
}

bool FAndroidWindow::OnWindowOrientationChanged(EDeviceScreenOrientation DeviceScreenOrientation)
{
	if (GDeviceScreenOrientation != DeviceScreenOrientation)
	{
		GDeviceScreenOrientation = DeviceScreenOrientation;
		bool bIsPortrait = GDeviceScreenOrientation == EDeviceScreenOrientation::Portrait || GDeviceScreenOrientation == EDeviceScreenOrientation::PortraitUpsideDown;
		UE_LOGF(LogAndroid, Log, "Window orientation changed: %ls, GDeviceScreenOrientation=%d", bIsPortrait ? TEXT("Portrait") : TEXT("Landscape"), EnumToUnderlyingType(GDeviceScreenOrientation));
		GAndroidIsPortrait = bIsPortrait;
		return true;
	}
	return false;
}

extern FString AndroidThunkCpp_GetMetaDataString(const FString& Key);

// Sets the Android screen size to 
static FAndroidDisplayInfo GetAndroidDisplayInfoFromDPITargets(int32 TargetDPI, int32 SceneMaxDesiredPixelCount, int32 LowerLimit3DDPI)
{
	auto StringToMap = [](const FString& str)
	{
		TMap<FString, FString> mapret;
		TArray<FString> OutArray;
		str.ParseIntoArray(OutArray, TEXT(";"));
		if (OutArray.Num() % 2 == 0)
		{
			for (int i = 0; i < OutArray.Num(); i += 2)
			{
				mapret.Add(OutArray[i], OutArray[i + 1]);
			}
		}
		return mapret;
	};

	FString JNIMetrics = AndroidThunkCpp_GetMetaDataString(FString("unreal.displaymetrics.metrics"));
	FString JNIDisplay = AndroidThunkCpp_GetMetaDataString(FString("unreal.display"));

	TMap<FString, FString> metricsParams = StringToMap(JNIMetrics);
	TMap<FString, FString> displayParams = StringToMap(JNIDisplay);
	static const FString DensityDpiKey("densityDpi");
	static const FString xDpiKey("xdpi");
	static const FString yDpiKey("ydpi");
	static const FString WidthPixelsKey("realWidth");
	static const FString HeightPixelsKey("realHeight");

	int32 DensityDPI = metricsParams.Contains(TEXT("densityDpi")) ? FCString::Atoi(*metricsParams.FindChecked(DensityDpiKey)) : 0;
	int32 xdpi = metricsParams.Contains(TEXT("xdpi")) ? FCString::Atoi(*metricsParams.FindChecked(xDpiKey)) : 0;
	int32 ydpi = metricsParams.Contains(TEXT("ydpi")) ? FCString::Atoi(*metricsParams.FindChecked(yDpiKey)) : 0;
	int32 avgdpi = (xdpi + ydpi) / 2;

	UE_LOGF(LogAndroid, Display, "AndroidDisplayInfoFromDPITargets : DPI info: DensityDPI %d, dpi %d, xdpi %d, ydpi %d", DensityDPI, avgdpi, xdpi, ydpi);

	int32 NativeScreenDensityDPI;
	switch (GAndroidWindowDPIQueryMethod)
	{
		case 1:
		{
			NativeScreenDensityDPI = DensityDPI;
			break;
		}
		case 0:
		default:
		{
			NativeScreenDensityDPI = avgdpi;
			break;
		}
	}

	FIntVector2 NativeScreenPixelDims(
		displayParams.Contains(WidthPixelsKey) ? FCString::Atoi(*displayParams.FindChecked(WidthPixelsKey)) : 0
		, displayParams.Contains(HeightPixelsKey) ? FCString::Atoi(*displayParams.FindChecked(HeightPixelsKey)) : 0
	);

	FAndroidDisplayInfo Info;
	Info.SceneScaleFactor = 1.0f;

	// NativeScreenDensityDPI is approx.
	const float ApproxSystemToDesiredDPIScale = FMath::Min((float)TargetDPI / (float)NativeScreenDensityDPI, 1.0f);
	Info.WindowDims = FIntVector2((int32)FMath::RoundFromZero((float)NativeScreenPixelDims.X * ApproxSystemToDesiredDPIScale), (int32)FMath::RoundFromZero((float)NativeScreenPixelDims.Y * ApproxSystemToDesiredDPIScale));
	FIntVector2 Sanitized = AndroidWindowUtils::SanitizeAndroidScreenSize(MoveTemp(NativeScreenPixelDims), FIntVector2(Info.WindowDims));

	// ignore minor differences from sanitization.
	//float ScaleFromSanitizing = FMath::Sqrt((float)(Sanitized.X * Sanitized.Y) / (float)(Info.WindowDims.X * Info.WindowDims.Y));
	Info.WindowDims = Sanitized;
	//TargetDPI = ScaleFromSanitizing;

	UE_LOGF(LogAndroid, Display, "AndroidDisplayInfoFromDPITargets : Native screen dpi %d, res %d x %d", NativeScreenDensityDPI, NativeScreenPixelDims.X, NativeScreenPixelDims.Y);
	UE_CLOG(TargetDPI <= NativeScreenDensityDPI, LogAndroid, Display, TEXT("AndroidDisplayInfoFromDPITargets : New DPI target %d, window dims %d, %d"), TargetDPI, Info.WindowDims.X, Info.WindowDims.Y);
	UE_CLOGF(TargetDPI > NativeScreenDensityDPI, LogAndroid, Display, "AndroidDisplayInfoFromDPITargets : TargetDPI too high, using native screen DPI %d, window dims %d, %d", NativeScreenDensityDPI, Info.WindowDims.X, Info.WindowDims.Y);
	TargetDPI = FMath::Min(TargetDPI, NativeScreenDensityDPI);

	UE_CLOG(NativeScreenDensityDPI<LowerLimit3DDPI, LogAndroid, Display, TEXT("AndroidDisplayInfoFromDPITargets : 3d scene lower limit is too high (%d) for a native screen, clamping lower limit to DPI %d"), LowerLimit3DDPI, NativeScreenDensityDPI);
	LowerLimit3DDPI = FMath::Min(LowerLimit3DDPI, NativeScreenDensityDPI);

	int DesiredPixelCount = Info.WindowDims.X * Info.WindowDims.Y;
	if (SceneMaxDesiredPixelCount && DesiredPixelCount > SceneMaxDesiredPixelCount)
	{
		// if we're going to be pushing too many pixels, scale back 3d scene size to get us to SceneMaxDesiredPixelCount
		Info.SceneScaleFactor = FMath::Sqrt((float)SceneMaxDesiredPixelCount / (float)DesiredPixelCount);
		UE_LOGF(LogAndroid, Warning, "AndroidDisplayInfoFromDPITargets : DPI %d has a %d pixels, this exceeds the pixels limit of %d by %d%%. 3d scene target is reduced to %d x %d",
			TargetDPI,
			DesiredPixelCount, 
			SceneMaxDesiredPixelCount, 
			(uint32)(((float)DesiredPixelCount/(float)SceneMaxDesiredPixelCount)*100),
			(uint32)FMath::RoundFromZero((float)Info.WindowDims.X * Info.SceneScaleFactor),
			(uint32)FMath::RoundFromZero((float)Info.WindowDims.Y * Info.SceneScaleFactor)
			);
	}
	
	// if a min dpi was specified then clamp to that and accept a perf hit for res quality.
	if (LowerLimit3DDPI && (int32)((float)TargetDPI * Info.SceneScaleFactor) < LowerLimit3DDPI)
	{
		float DPILimitScale = (float)LowerLimit3DDPI / (float)TargetDPI;

		UE_LOGF(LogAndroid, Warning, "AndroidDisplayInfoFromDPITargets : 3d scene target of %d DPI is lower than specified limit of %d DPI, increasing scene target dims %dx%d -> %dx%d",
			(uint32)FMath::RoundFromZero((float)TargetDPI * Info.SceneScaleFactor),
			LowerLimit3DDPI,
			(uint32)FMath::RoundFromZero((float)Info.WindowDims.X * Info.SceneScaleFactor),
			(uint32)FMath::RoundFromZero((float)Info.WindowDims.Y * Info.SceneScaleFactor),
			(uint32)FMath::RoundFromZero((float)Info.WindowDims.X * DPILimitScale),
			(uint32)FMath::RoundFromZero((float)Info.WindowDims.Y * DPILimitScale)
			);
		Info.SceneScaleFactor = (float)LowerLimit3DDPI / (float)TargetDPI;
	}

	if (SceneMaxDesiredPixelCount)
	{
		int FinalSceneTargetPixelCount = (int)((float)DesiredPixelCount * Info.SceneScaleFactor * Info.SceneScaleFactor);
		UE_LOGF(LogAndroid, Display, "AndroidDisplayInfoFromDPITargets : SceneTarget Pixel count %d%% of limit.", (uint32)(((float)FinalSceneTargetPixelCount / (float)SceneMaxDesiredPixelCount) * 100));
	}
	
	return Info;
}

