// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleControllerInterface.h"

#include "Apple/AppleStringUtils.h"
#include "HAL/PlatformTime.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/ScopeLock.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "GenericPlatform/InputDeviceRegistry.h"

DEFINE_LOG_CATEGORY(LogAppleController);

#define APPLE_CONTROLLER_DEBUG 0

namespace
{
	// Controller Type Strings
	static const FName UnassignedControllerTypeName = "Unassigned";
	static const FName SiriRemoteControllerTypeName = "SiriRemote";
	static const FName ExtendedGamepadControllerTypeName = "ExtendedGamepad";
	static const FName XboxControllerTypeName = "Xbox Wireless Controller";
	static const FName DualShockControllerTypeName = "PS4 Wireless Controller";
	static const FName DualSenseControllerTypeName = "PS5 Wireless Controller";

	// Continuous haptic events use a long duration and are looped with sendParameters; a 30s window keeps
	// them from ever ending under sustained vibration without tying up loop bookkeeping.
	static constexpr NSTimeInterval AppleControllerContinuousHapticDuration = 30.0;

	// Map FForceFeedbackValues to a single (intensity, sharpness) pair for one side of the gamepad.
	// Convention shared with other UE platforms: large motors map to low-frequency rumble (low sharpness),
	// small motors map to high-frequency buzz (high sharpness). The combined intensity is the max of the two.
	static void ResolveSideIntensitySharpness(float Large, float Small, float& OutIntensity, float& OutSharpness)
	{
		OutIntensity = FMath::Clamp(FMath::Max(Large, Small), 0.0f, 1.0f);
		// When both are zero, intensity is zero and sharpness is irrelevant; keep it at 0 to avoid feeding NaN.
		const float Total = Large + Small;
		OutSharpness = (Total > 0.0f) ? FMath::Clamp(Small / Total, 0.0f, 1.0f) : 0.0f;
	}

	API_AVAILABLE(ios(14.0), macos(11.0), tvos(14.0))
	static CHHapticPattern* BuildContinuousHapticPattern(float Intensity, float Sharpness)
	{
		CHHapticEventParameter* IntensityParam = [[[CHHapticEventParameter alloc]
			initWithParameterID:CHHapticEventParameterIDHapticIntensity value:Intensity] autorelease];
		CHHapticEventParameter* SharpnessParam = [[[CHHapticEventParameter alloc]
			initWithParameterID:CHHapticEventParameterIDHapticSharpness value:Sharpness] autorelease];

		CHHapticEvent* Event = [[[CHHapticEvent alloc]
			initWithEventType:CHHapticEventTypeHapticContinuous
			parameters:@[IntensityParam, SharpnessParam]
			relativeTime:0.0
			duration:AppleControllerContinuousHapticDuration] autorelease];

		NSError* PatternError = nil;
		CHHapticPattern* Pattern = [[[CHHapticPattern alloc]
			initWithEvents:@[Event]
			parameters:@[]
			error:&PatternError] autorelease];

		if (PatternError != nil)
		{
			UE_LOGF(LogAppleController, Warning, "Failed to build continuous haptic pattern: %ls", *FString([PatternError localizedDescription]));
			return nil;
		}
		return Pattern;
	}

	API_AVAILABLE(ios(14.0), macos(11.0), tvos(14.0))
	static void SendDynamicHapticParameters(id<CHHapticPatternPlayer> Player, float Intensity, float Sharpness)
	{
		if (Player == nil)
		{
			return;
		}
		CHHapticDynamicParameter* IntensityParam = [[[CHHapticDynamicParameter alloc]
			initWithParameterID:CHHapticDynamicParameterIDHapticIntensityControl
			value:Intensity
			relativeTime:0.0] autorelease];
		CHHapticDynamicParameter* SharpnessParam = [[[CHHapticDynamicParameter alloc]
			initWithParameterID:CHHapticDynamicParameterIDHapticSharpnessControl
			value:Sharpness
			relativeTime:0.0] autorelease];

		NSError* SendError = nil;
		[Player sendParameters:@[IntensityParam, SharpnessParam] atTime:CHHapticTimeImmediate error:&SendError];
		if (SendError != nil)
		{
			UE_LOGF(LogAppleController, Verbose, "sendParameters failed: %ls", *FString([SendError localizedDescription]));
		}
	}
}

FString FAppleControllerInterface::HardwareDeviceIdentifier_DefaultGamepad;

TSharedRef< FAppleControllerInterface > FAppleControllerInterface::Create(  const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler )
{
	return MakeShareable( new FAppleControllerInterface( InMessageHandler ) );
}

FAppleControllerInterface::FAppleControllerInterface( const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler )
	: MessageHandler( InMessageHandler )
	, bAllowControllers(true)
{
	if(!IS_PROGRAM)
	{
		// Clear array and setup unset player index values
		FMemory::Memzero(Controllers, sizeof(Controllers));
		for (int32 ControllerIndex = 0; ControllerIndex < UE_ARRAY_COUNT(Controllers); ControllerIndex++)
		{
			FUserController& UserController = Controllers[ControllerIndex];
			UserController.PlayerIndex = PlayerIndex::PlayerUnset;

			// Setup the buttons for this controller. We pre-allocate for the worse case of MAX_NUM_CONTROLLERS (4) connected controllers.
			// Doing it here avoids creating a new TMap on controller connect and destruction on disconnect which
			// could happen many times during a play session. 
			// Allocate for both current and previous captured controller state.
			for (int32 ActiveIndex = 0; ActiveIndex < MAX_CONTROLLER_CAPTURE_STATES; ActiveIndex++)
			{
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::FaceButtonLeft);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::FaceButtonBottom);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::FaceButtonRight);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::FaceButtonTop);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::LeftShoulder);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::RightShoulder);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::LeftTriggerThreshold);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::RightTriggerThreshold);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::DPadUp);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::DPadDown);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::DPadRight);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::DPadLeft);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::LeftThumb);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::RightThumb);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::SpecialRight);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::SpecialLeft);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::LeftStickUp);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::LeftStickDown);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::LeftStickRight);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::LeftStickLeft);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::RightAnalogX);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::RightAnalogY);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::LeftAnalogX);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::LeftAnalogY);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::RightTriggerAnalog);
				ControllerButtons[ActiveIndex][ControllerIndex].Add(FGamepadKeyNames::LeftTriggerAnalog);
			}
		}

		for (GCController* Cont in [GCController controllers])
		{
			HandleConnection(Cont);
		}

		NSNotificationCenter* notificationCenter = [NSNotificationCenter defaultCenter];

		// Not in an operation queue, [NSOperationQueue currentQueue] will return
		// nil on macOS and iOS Notification callback will always be on the app main
		// thread - defer events for Unreal Engine update thread

		[notificationCenter addObserverForName:GCControllerDidDisconnectNotification object:nil queue:nil usingBlock:^(NSNotification* Notification)
		{
			SignalEvent(EAppleControllerEventType::Disconnect, Notification.object);
		}];

		[notificationCenter addObserverForName:GCControllerDidConnectNotification object:nil queue:nil usingBlock:^(NSNotification* Notification)
		{
			SignalEvent(EAppleControllerEventType::Connect, Notification.object);
		}];

		dispatch_async(dispatch_get_main_queue(), ^
		{
		   [GCController startWirelessControllerDiscoveryWithCompletionHandler:^{ }];
		});
	}

	HardwareDeviceIdentifier_DefaultGamepad = TEXT("Gamepad");
}

void FAppleControllerInterface::SetMessageHandler( const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler )
{
	MessageHandler = InMessageHandler;
}

void FAppleControllerInterface::SignalEvent(EAppleControllerEventType InEventType, GCController* InController)
{
	FScopeLock Lock(&DeferredEventCS);
	DeferredEvents.Add(FDeferredAppleControllerEvent(InEventType, InController));
}

void FAppleControllerInterface::Tick( float DeltaTime )
{
	FScopeLock Lock(&DeferredEventCS);

	for(uint32_t Index = 0;Index < DeferredEvents.Num();++Index)
	{
		FDeferredAppleControllerEvent& Event = DeferredEvents[Index];
		switch(Event.EventType)
		{
			case EAppleControllerEventType::Connect:
			{
				HandleConnection(Event.Controller);
				break;
			}
			case EAppleControllerEventType::Disconnect:
			{
				HandleDisconnect(Event.Controller);
				break;
			}
			case EAppleControllerEventType::BecomeCurrent:
			{
				SetCurrentController(Event.Controller);
				break;
			}
			case EAppleControllerEventType::Invalid:
			default:
			{
				// NOP
				break;
			}
		}
	}
	
	DeferredEvents.Empty();
}

void FAppleControllerInterface::SetControllerType(uint32 ControllerIndex)
{
	GCController *Controller = Controllers[ControllerIndex].Controller;

	if ([Controller.productCategory isEqualToString:@"DualShock 4"])
	{
		// validate it's of class GCDualShockGamepad so we can cast it later
		if ([Controller.extendedGamepad isKindOfClass:[GCDualShockGamepad class]])
		{
			Controllers[ControllerIndex].ControllerType = ControllerType::DualShockGamepad;
		}
		else
		{
			UE_LOGF(LogAppleController, Warning, "DualShock4 detected, but invalid class");
			Controllers[ControllerIndex].ControllerType = ControllerType::ExtendedGamepad;
		}
	}
	else if ([Controller.productCategory isEqualToString:@"Xbox One"])
	{
		Controllers[ControllerIndex].ControllerType = ControllerType::XboxGamepad;
	}
	else if ([Controller.productCategory isEqualToString:@"DualSense"])
	{
		// validate it is of class GCDualSenseGamepad so we can cast it later
		if ([Controller.extendedGamepad isKindOfClass:[GCDualSenseGamepad class]])
		{
			Controllers[ControllerIndex].ControllerType = ControllerType::DualSenseGamepad;
		}
		else
		{
			UE_LOGF(LogAppleController, Warning, "DualSense detected, but invalid class");
			Controllers[ControllerIndex].ControllerType = ControllerType::ExtendedGamepad;
		}
	}
	else if (Controller.extendedGamepad != nil)
	{
		Controllers[ControllerIndex].ControllerType = ControllerType::ExtendedGamepad;
	}
	else if (Controller.microGamepad != nil)
	{
		Controllers[ControllerIndex].ControllerType = ControllerType::SiriRemote;
	}
	else
	{
		Controllers[ControllerIndex].ControllerType = ControllerType::Unassigned;
		UE_LOGF(LogAppleController, Warning, "Controller type is not recognized");
	}
}

FName FAppleControllerInterface::GetControllerTypeName(const ControllerType InControllerType)
{
	FName ControllerTypeName = UnassignedControllerTypeName;
	switch (InControllerType)
	{
		case ControllerType::SiriRemote:
			ControllerTypeName = SiriRemoteControllerTypeName;
			break;
		case ControllerType::ExtendedGamepad:
			ControllerTypeName = ExtendedGamepadControllerTypeName;
			break;
		case ControllerType::XboxGamepad:
			ControllerTypeName = XboxControllerTypeName;
			break;
		case ControllerType::DualShockGamepad:
			ControllerTypeName = DualShockControllerTypeName;
			break;
		case ControllerType::DualSenseGamepad:
			ControllerTypeName = DualSenseControllerTypeName;
			break;
		default:
			UE_LOGF(LogAppleController, Warning, "[GetControllerTypeName] Controller type is not recognized");
			break;
	}

	return ControllerTypeName;
}

void FAppleControllerInterface::SetCurrentController(GCController* Controller)
{
	int32 ControllerIndex = 0;
	PlayerIndex PreviousIndex = PlayerIndex::PlayerUnset;

	if (Controller == nil)
	{
		return;
	}

	for (ControllerIndex = 0; ControllerIndex < UE_ARRAY_COUNT(Controllers); ControllerIndex++)
	{
		if (Controllers[ControllerIndex].Controller == Controller)
		{
			if (Controllers[ControllerIndex].PlayerIndex == PlayerIndex::PlayerOne)
			{
				// Already set as CurrentController
				return;
			}
			PreviousIndex = Controllers[ControllerIndex].PlayerIndex;
			Controllers[ControllerIndex].PlayerIndex = PlayerIndex::PlayerOne;
		}
	}
	
	for (ControllerIndex = 0; ControllerIndex < UE_ARRAY_COUNT(Controllers); ControllerIndex++)
	{
		if (Controllers[ControllerIndex].PlayerIndex == PlayerIndex::PlayerOne &&
			Controllers[ControllerIndex].Controller != Controller)
		{
			// The old PlayerOne, should swap place with the new PlayerOne
			Controllers[ControllerIndex].PlayerIndex = PreviousIndex;
		}
	}
}

void FAppleControllerInterface::CaptureControllerState(const GCController* Controller, uint32 ControllerIndex)
{
	if (!bAllowControllers || Controller == nil)
	{
		return;
	}

	GCController* Cont = Controllers[ControllerIndex].Controller;
	TMap<FName, FControllerButton>& State = ControllerButtons[CurrentButtonIndex][ControllerIndex];

	switch (Controllers[ControllerIndex].ControllerType)
	{
		case ControllerType::ExtendedGamepad:
		case ControllerType::DualShockGamepad:
		case ControllerType::XboxGamepad:
		case ControllerType::DualSenseGamepad:
		{
			// Capture buttons state
			const GCExtendedGamepad* ExtendedGamepad = Cont.extendedGamepad;
			State[FGamepadKeyNames::FaceButtonLeft].bPressed = ExtendedGamepad.buttonX.pressed;
			State[FGamepadKeyNames::FaceButtonBottom].bPressed = ExtendedGamepad.buttonA.pressed;
			State[FGamepadKeyNames::FaceButtonRight].bPressed = ExtendedGamepad.buttonB.pressed;
			State[FGamepadKeyNames::FaceButtonTop].bPressed = ExtendedGamepad.buttonY.pressed;
			State[FGamepadKeyNames::LeftShoulder].bPressed = ExtendedGamepad.leftShoulder.pressed;
			State[FGamepadKeyNames::RightShoulder].bPressed = ExtendedGamepad.rightShoulder.pressed;
			State[FGamepadKeyNames::LeftTriggerThreshold].bPressed = ExtendedGamepad.leftTrigger.pressed;
			State[FGamepadKeyNames::RightTriggerThreshold].bPressed = ExtendedGamepad.rightTrigger.pressed;
			State[FGamepadKeyNames::DPadUp].bPressed = ExtendedGamepad.dpad.up.pressed;
			State[FGamepadKeyNames::DPadDown].bPressed = ExtendedGamepad.dpad.down.pressed;
			State[FGamepadKeyNames::DPadRight].bPressed = ExtendedGamepad.dpad.right.pressed;
			State[FGamepadKeyNames::DPadLeft].bPressed = ExtendedGamepad.dpad.left.pressed;
			State[FGamepadKeyNames::LeftThumb].bPressed = ExtendedGamepad.leftThumbstickButton.pressed;
			State[FGamepadKeyNames::RightThumb].bPressed = ExtendedGamepad.rightThumbstickButton.pressed;
			State[FGamepadKeyNames::SpecialRight].bPressed = ExtendedGamepad.buttonMenu.pressed;

			// DualSense and DualShock4 use the touchpad button for the "SpecialLeft" button
			if (Controllers[ControllerIndex].ControllerType == ControllerType::DualSenseGamepad)
			{
				// SetControllerType() verified that ExtendedGamepad is of type GCDualSenseGamepad
				State[FGamepadKeyNames::SpecialLeft].bPressed = static_cast<GCDualShockGamepad*>(ExtendedGamepad).touchpadButton.pressed;
			}
			else if (Controllers[ControllerIndex].ControllerType == ControllerType::DualShockGamepad)
			{
				State[FGamepadKeyNames::SpecialLeft].bPressed = static_cast<GCDualShockGamepad*>(ExtendedGamepad).touchpadButton.pressed;
			}
			else
			{
				State[FGamepadKeyNames::SpecialLeft].bPressed = ExtendedGamepad.buttonOptions.pressed;
			}

			// Capture axis values
			State[FGamepadKeyNames::LeftAnalogX].AxisValue = ExtendedGamepad.leftThumbstick.xAxis.value;
			State[FGamepadKeyNames::LeftAnalogY].AxisValue = ExtendedGamepad.leftThumbstick.yAxis.value;
			State[FGamepadKeyNames::RightAnalogX].AxisValue = ExtendedGamepad.rightThumbstick.xAxis.value;
			State[FGamepadKeyNames::RightAnalogY].AxisValue = ExtendedGamepad.rightThumbstick.yAxis.value;
			State[FGamepadKeyNames::RightTriggerAnalog].AxisValue = ExtendedGamepad.rightTrigger.value;
			State[FGamepadKeyNames::LeftTriggerAnalog].AxisValue = ExtendedGamepad.leftTrigger.value;
			break;
		}

		case ControllerType::SiriRemote:
		{
			const GCMicroGamepad* MicroGamepad = Cont.microGamepad;
			State[FGamepadKeyNames::FaceButtonLeft].bPressed = MicroGamepad.buttonX.pressed;
			State[FGamepadKeyNames::FaceButtonBottom].bPressed = MicroGamepad.buttonA.pressed;
			State[FGamepadKeyNames::DPadUp].bPressed = MicroGamepad.dpad.up.pressed;
			State[FGamepadKeyNames::DPadDown].bPressed = MicroGamepad.dpad.down.pressed;
			State[FGamepadKeyNames::DPadRight].bPressed = MicroGamepad.dpad.right.pressed;
			State[FGamepadKeyNames::DPadLeft].bPressed = MicroGamepad.dpad.left.pressed;
			State[FGamepadKeyNames::LeftStickUp].bPressed = MicroGamepad.dpad.up.pressed;
			State[FGamepadKeyNames::LeftStickDown].bPressed = MicroGamepad.dpad.down.pressed;
			State[FGamepadKeyNames::LeftStickRight].bPressed = MicroGamepad.dpad.right.pressed;
			State[FGamepadKeyNames::LeftStickLeft].bPressed = MicroGamepad.dpad.left.pressed;
			State[FGamepadKeyNames::SpecialRight].bPressed = MicroGamepad.buttonMenu.pressed;
			
			// Capture "axis" values
			State[FGamepadKeyNames::LeftAnalogX].AxisValue = MicroGamepad.dpad.xAxis.value;
			State[FGamepadKeyNames::LeftAnalogY].AxisValue = MicroGamepad.dpad.yAxis.value;
			break;
		}

		default:
			UE_LOG(LogAppleController, Warning, TEXT("UNSUPPORTED CONTROLLER"));
			break;
	}
}

void FAppleControllerInterface::HandleConnection(GCController* Controller)
{
	if (!bAllowControllers || Controller == nil)
	{
		return;
	}
	
	static_assert(GCControllerPlayerIndex1 == 0 && GCControllerPlayerIndex4 == 3, "Apple changed the player index enums");

	// find a good controller index to use
	bool bFoundSlot = false;
	for (int32 ControllerIndex = 0; ControllerIndex < UE_ARRAY_COUNT(Controllers); ControllerIndex++)
	{
		if (Controllers[ControllerIndex].ControllerType != ControllerType::Unassigned)
		{
			continue;
		}
		
		Controllers[ControllerIndex].PlayerIndex = (PlayerIndex)ControllerIndex;
		Controllers[ControllerIndex].Controller = [Controller retain];
		Controllers[ControllerIndex].DefaultLightColor = FColor::Black;
		if (@available(iOS 14.0, macOS 11.0, tvOS 14.0, *))
		{
			if (Controller.light != nil)
			{
				const GCColor* CurrentLight = Controller.light.color;
				Controllers[ControllerIndex].DefaultLightColor = FColor(
					(uint8)FMath::Clamp(CurrentLight.red   * 255.0f, 0.0f, 255.0f),
					(uint8)FMath::Clamp(CurrentLight.green * 255.0f, 0.0f, 255.0f),
					(uint8)FMath::Clamp(CurrentLight.blue  * 255.0f, 0.0f, 255.0f));
			}
		}
		SetControllerType(ControllerIndex);
		
		// Reset values for contoller buttons and axis
		for (int32 ActiveIndex = 0; ActiveIndex < MAX_CONTROLLER_CAPTURE_STATES; ActiveIndex++)
		{
			TMap<FName, FControllerButton>& ButtonStates = ControllerButtons[ActiveIndex][ControllerIndex];
			for (TTuple<FName, FAppleControllerInterface::FControllerButton>& Pair : ButtonStates)
			{
				Pair.Value.Reset();
			}
		}
		
		bFoundSlot = true;
		
		IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
		FPlatformUserId UserId = FGenericPlatformMisc::GetPlatformUserForUserIndex(Controllers[ControllerIndex].PlayerIndex);
		FInputDeviceId DeviceId = INPUTDEVICEID_NONE;
		DeviceMapper.RemapControllerIdToPlatformUserAndDevice(Controllers[ControllerIndex].PlayerIndex, OUT UserId, OUT DeviceId);
		{
			FInputDeviceDescriptor Desc;
			Desc.HardwareDeviceHandle     = DeviceId;
			Desc.InputDeviceName          = GetControllerTypeName(Controllers[ControllerIndex].ControllerType);
			Desc.HardwareDeviceIdentifier = FName(*HardwareDeviceIdentifier_DefaultGamepad);
			FInputDeviceRegistry::RegisterDevice(Desc);
		}
		DeviceMapper.Internal_MapInputDeviceToUser(DeviceId, UserId, EInputDeviceConnectionState::Connected);
		
		UE_LOGF(LogAppleController, Log, "New %ls controller inserted, assigned to playerIndex %d",
			   Controllers[ControllerIndex].ControllerType == ControllerType::SiriRemote
			   ? TEXT("SiriRemote") : *FString(Controller.productCategory), Controllers[ControllerIndex].PlayerIndex);
		break;
	}
	checkf(bFoundSlot, TEXT("Used a fifth controller somehow!"));
}

void FAppleControllerInterface::HandleDisconnect(GCController* Controller)
{
	// if we don't allow controllers, there could be unset player index here
	if (!bAllowControllers || Controller == nil)
	{
		return;
	}
	
	for (int32 ControllerIndex = 0; ControllerIndex < UE_ARRAY_COUNT(Controllers); ControllerIndex++)
	{
		FUserController& UserController = Controllers[ControllerIndex];
		if (UserController.Controller == Controller)
		{
			// Player index of unset(-1) would indicate that it has become unset even though it is now 
			// trying to disconnect. This can occur on iOS when bGameSupportsMultipleActiveControllers is false
			UE_LOGF(LogAppleController, Log, "Controller for playerIndex %d, controller Index %d removed", UserController.PlayerIndex, ControllerIndex);

			// Check if there's any buttons/joysticks currently pressed and/or active analog axis.
			// If so, send off events as if the button was unpressed and axis back to 0.
			// This avoids issues like having a player stuck moving forward and/or stuck firing a weapon.
			TMap<FName, FControllerButton>& CurrentButtons = ControllerButtons[CurrentButtonIndex][ControllerIndex];
			TMap<FName, FControllerButton>& PrevButtons = ControllerButtons[PrevButtonIndex][ControllerIndex];
			for (auto& Pair : CurrentButtons)
			{
				Pair.Value.Reset();
				
				const FControllerButton* PrevButtonState = PrevButtons.Find(Pair.Key);
				if (Pair.Value.bPressed != PrevButtonState->bPressed)
				{
					HandleButtonGamepad(Pair.Key, ControllerIndex);
				}
				if (Pair.Value.AxisValue != PrevButtonState->AxisValue)
				{
					HandleAnalogGamepad(Pair.Key, ControllerIndex);
				}
			}
			HandleVirtualButtonGamepad(FGamepadKeyNames::LeftStickLeft, FGamepadKeyNames::LeftStickRight, ControllerIndex);
			HandleVirtualButtonGamepad(FGamepadKeyNames::LeftStickDown, FGamepadKeyNames::LeftStickUp, ControllerIndex);
			HandleVirtualButtonGamepad(FGamepadKeyNames::RightStickLeft, FGamepadKeyNames::RightStickRight, ControllerIndex);
			HandleVirtualButtonGamepad(FGamepadKeyNames::RightStickDown, FGamepadKeyNames::RightStickUp, ControllerIndex);
			
			// Clear out any old Previous captured button state. Above cleared the Current state.
			for (auto& Pair : PrevButtons)
			{
				Pair.Value.Reset();
			}
			
			IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
			FPlatformUserId UserId = FGenericPlatformMisc::GetPlatformUserForUserIndex(UserController.PlayerIndex);
			FInputDeviceId DeviceId = INPUTDEVICEID_NONE;
			DeviceMapper.RemapControllerIdToPlatformUserAndDevice(UserController.PlayerIndex, OUT UserId, OUT DeviceId);
			DeviceMapper.Internal_MapInputDeviceToUser(DeviceId, UserId, EInputDeviceConnectionState::Disconnected);

			// Release CoreHaptics state before Memzero so the retained ObjC pointers do not leak.
			TearDownHapticsForController(UserController);

			[UserController.Controller release];
			
			FMemory::Memzero(&UserController, sizeof(Controllers[ControllerIndex]));
			UserController.PlayerIndex = PlayerIndex::PlayerUnset;
			
			return;
		}
	}
}

void FAppleControllerInterface::SendControllerEvents()
{
	@autoreleasepool
	{
	for(int32 i = 0; i < UE_ARRAY_COUNT(Controllers); ++i)
	{
		FUserController& Controller = Controllers[i];
		
		// make sure the connection handler has run on this
		GCController* ControllerImpl = Controller.Controller;
		if (Controller.PlayerIndex == PlayerIndex::PlayerUnset || ControllerImpl == nil)
		{
			continue;
		}

		IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
		FPlatformUserId UserId = FGenericPlatformMisc::GetPlatformUserForUserIndex(Controller.PlayerIndex);
		FInputDeviceId DeviceId = DeviceMapper.GetPrimaryInputDeviceForUser(UserId);

		///The FInputDeviceScope::HardwareDeviceIdentifier has to be one of values in UInputPlatformSettings::HardwareDevices.
		// This is a temp solution with a hardcoded string which can be mapped to FHardwareDeviceIdentifier::DefaultGamepad.
		// TODO: Future improvement is needed to acquire them by values in the iOS input device info.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FInputDeviceScope InputScope(nullptr, GetControllerTypeName(Controller.ControllerType), DeviceId.GetId(), HardwareDeviceIdentifier_DefaultGamepad);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		uint32 ButtonIndex = PrevButtonIndex;	
		PrevButtonIndex = CurrentButtonIndex;
		CurrentButtonIndex = ButtonIndex;

		CaptureControllerState(ControllerImpl, i);

		HandleButtonGamepad(FGamepadKeyNames::FaceButtonBottom, i);
		HandleButtonGamepad(FGamepadKeyNames::FaceButtonLeft, i);
		HandleButtonGamepad(FGamepadKeyNames::FaceButtonRight, i);
		HandleButtonGamepad(FGamepadKeyNames::FaceButtonTop, i);
		HandleButtonGamepad(FGamepadKeyNames::LeftShoulder, i);
		HandleButtonGamepad(FGamepadKeyNames::RightShoulder, i);
		HandleButtonGamepad(FGamepadKeyNames::LeftTriggerThreshold, i);
		HandleButtonGamepad(FGamepadKeyNames::RightTriggerThreshold, i);
		HandleButtonGamepad(FGamepadKeyNames::DPadUp, i);
		HandleButtonGamepad(FGamepadKeyNames::DPadDown, i);
		HandleButtonGamepad(FGamepadKeyNames::DPadRight, i);
		HandleButtonGamepad(FGamepadKeyNames::DPadLeft, i);
		HandleButtonGamepad(FGamepadKeyNames::SpecialRight, i);
		HandleButtonGamepad(FGamepadKeyNames::SpecialLeft, i);

		HandleAnalogGamepad(FGamepadKeyNames::LeftAnalogX, i);
		HandleAnalogGamepad(FGamepadKeyNames::LeftAnalogY, i);
		HandleAnalogGamepad(FGamepadKeyNames::RightAnalogX, i);
		HandleAnalogGamepad(FGamepadKeyNames::RightAnalogY, i);
		HandleAnalogGamepad(FGamepadKeyNames::RightTriggerAnalog, i);
		HandleAnalogGamepad(FGamepadKeyNames::LeftTriggerAnalog, i);

		HandleVirtualButtonGamepad(FGamepadKeyNames::LeftStickLeft, FGamepadKeyNames::LeftStickRight, i);
		HandleVirtualButtonGamepad(FGamepadKeyNames::LeftStickDown, FGamepadKeyNames::LeftStickUp, i);
		HandleVirtualButtonGamepad(FGamepadKeyNames::RightStickLeft, FGamepadKeyNames::RightStickRight, i);
		HandleVirtualButtonGamepad(FGamepadKeyNames::RightStickDown, FGamepadKeyNames::RightStickUp, i);
		HandleButtonGamepad(FGamepadKeyNames::LeftThumb, i);
		HandleButtonGamepad(FGamepadKeyNames::RightThumb, i);
		}
	} //@autoreleasepool
}

bool FAppleControllerInterface::IsControllerAssignedToGamepad(int32 ControllerId) const
{
	return ControllerId < UE_ARRAY_COUNT(Controllers) &&
		(Controllers[ControllerId].ControllerType != ControllerType::Unassigned);
}

bool FAppleControllerInterface::IsGamepadAttached() const
{
	bool bIsAttached = false;
	for(int32 i = 0; i < UE_ARRAY_COUNT(Controllers); ++i)
	{
		bIsAttached |= IsControllerAssignedToGamepad(i);
	}
	return bIsAttached && bAllowControllers;
}

GCControllerButtonInput* FAppleControllerInterface::GetGCControllerButton(const FGamepadKeyNames::Type& ButtonKey, uint32 ControllerIndex)
{
	GCController* Cont = Controllers[ControllerIndex].Controller;
	
	GCExtendedGamepad* ExtendedGamepad = Cont.extendedGamepad;
	GCControllerButtonInput *ButtonToReturn = nullptr;
	
	if (ButtonKey == FGamepadKeyNames::FaceButtonBottom){ButtonToReturn = ExtendedGamepad.buttonA;}
	else if (ButtonKey == FGamepadKeyNames::FaceButtonRight){ButtonToReturn = ExtendedGamepad.buttonB;}
	else if (ButtonKey == FGamepadKeyNames::FaceButtonLeft){ButtonToReturn = ExtendedGamepad.buttonX;}
	else if (ButtonKey == FGamepadKeyNames::FaceButtonTop){ButtonToReturn = ExtendedGamepad.buttonY;}
	else if (ButtonKey == FGamepadKeyNames::LeftShoulder){ButtonToReturn = ExtendedGamepad.leftShoulder;}
	else if (ButtonKey == FGamepadKeyNames::RightShoulder){ButtonToReturn = ExtendedGamepad.rightShoulder;}
	else if (ButtonKey == FGamepadKeyNames::LeftTriggerThreshold){ButtonToReturn = ExtendedGamepad.leftTrigger;}
	else if (ButtonKey == FGamepadKeyNames::RightTriggerThreshold){ButtonToReturn = ExtendedGamepad.rightTrigger;}
	else if (ButtonKey == FGamepadKeyNames::LeftTriggerAnalog){ButtonToReturn = ExtendedGamepad.leftTrigger;}
	else if (ButtonKey == FGamepadKeyNames::RightTriggerAnalog){ButtonToReturn = ExtendedGamepad.rightTrigger;}
	else if (ButtonKey == FGamepadKeyNames::LeftThumb){ButtonToReturn = ExtendedGamepad.leftThumbstickButton;}
	else if (ButtonKey == FGamepadKeyNames::RightThumb){ButtonToReturn = ExtendedGamepad.rightThumbstickButton;}
	
	return ButtonToReturn;
}

const ControllerType FAppleControllerInterface::GetControllerType(uint32 ControllerIndex)
{
	if (Controllers[ControllerIndex].Controller != nullptr)
	{
		return Controllers[ControllerIndex].ControllerType;
	}
	return ControllerType::Unassigned;
}

void FAppleControllerInterface::HandleInputInternal(const FGamepadKeyNames::Type& UEButton, uint32 ControllerIndex, bool bIsPressed, bool bWasPressed)
{
    const double CurrentTime = FPlatformTime::Seconds();
    const float InitialRepeatDelay = 0.2f;
    const float RepeatDelay = 0.1f;
    GCController* Cont = Controllers[ControllerIndex].Controller;

	IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
	FPlatformUserId UserId = FGenericPlatformMisc::GetPlatformUserForUserIndex(Controllers[ControllerIndex].PlayerIndex);
	FInputDeviceId DeviceId = DeviceMapper.GetPrimaryInputDeviceForUser(UserId);

    if (bWasPressed != bIsPressed)
    {
#if APPLE_CONTROLLER_DEBUG
        NSLog(@"%@ button %s on controller %d", bIsPressed ? @"Pressed" : @"Released", TCHAR_TO_ANSI(*UEButton.ToString()), Controllers[ControllerIndex].PlayerIndex);
#endif
        bIsPressed ? MessageHandler->OnControllerButtonPressed(UEButton, UserId, DeviceId, false) : MessageHandler->OnControllerButtonReleased(UEButton,UserId, DeviceId, false);
        NextKeyRepeatTime.FindOrAdd(UEButton) = CurrentTime + InitialRepeatDelay;
    }
    else if(bIsPressed)
    {
        double* NextRepeatTime = NextKeyRepeatTime.Find(UEButton);
        if(NextRepeatTime && *NextRepeatTime <= CurrentTime)
        {
            MessageHandler->OnControllerButtonPressed(UEButton, UserId, DeviceId, true);
            *NextRepeatTime = CurrentTime + RepeatDelay;
        }
    }
    else
    {
        NextKeyRepeatTime.Remove(UEButton);
    }
}

void FAppleControllerInterface::HandleVirtualButtonGamepad(const FGamepadKeyNames::Type& UEButtonNegative, const FGamepadKeyNames::Type& UEButtonPositive, uint32 ControllerIndex)
{
	GCController* Cont = Controllers[ControllerIndex].Controller;

	// Send controller events any time we are passed the given input threshold similarly to PC/Console (see: XInputInterface.cpp)
	const float RepeatDeadzone = 0.24f;

	bool bWasNegativePressed = false;
	bool bNegativePressed = false;
	bool bWasPositivePressed = false;
	bool bPositivePressed = false;

	const TMap<FName, FControllerButton>& PrevState = ControllerButtons[PrevButtonIndex][ControllerIndex];
	const TMap<FName, FControllerButton>& CurrentState = ControllerButtons[CurrentButtonIndex][ControllerIndex];

	if (UEButtonNegative == FGamepadKeyNames::LeftStickLeft && UEButtonPositive == FGamepadKeyNames::LeftStickRight)
	{
		// Uses the "leftThumbstick.xAxis" of the controller, which is captured by "LeftAnalogX"
		bWasNegativePressed = PrevState.Find(FGamepadKeyNames::LeftAnalogX)->AxisValue <= -RepeatDeadzone;
		bNegativePressed = CurrentState.Find(FGamepadKeyNames::LeftAnalogX)->AxisValue <= -RepeatDeadzone;
		bWasPositivePressed = PrevState.Find(FGamepadKeyNames::LeftAnalogX)->AxisValue >= RepeatDeadzone;
		bPositivePressed = CurrentState.Find(FGamepadKeyNames::LeftAnalogX)->AxisValue >= RepeatDeadzone;
		
		HandleInputInternal(FGamepadKeyNames::LeftStickLeft, ControllerIndex, bNegativePressed, bWasNegativePressed);
		HandleInputInternal(FGamepadKeyNames::LeftStickRight, ControllerIndex, bPositivePressed, bWasPositivePressed);
	}
	else if (UEButtonNegative == FGamepadKeyNames::LeftStickDown && UEButtonPositive == FGamepadKeyNames::LeftStickUp)
	{
		// Uses the "leftThumbstick.yAxis" of the controller, which is captured by "LeftAnalogY"
		bWasNegativePressed = PrevState.Find(FGamepadKeyNames::LeftAnalogY)->AxisValue <= -RepeatDeadzone;
		bNegativePressed = CurrentState.Find(FGamepadKeyNames::LeftAnalogY)->AxisValue <= -RepeatDeadzone;
		bWasPositivePressed = PrevState.Find(FGamepadKeyNames::LeftAnalogY)->AxisValue >= RepeatDeadzone;
		bPositivePressed = CurrentState.Find(FGamepadKeyNames::LeftAnalogY)->AxisValue >= RepeatDeadzone;
		
		HandleInputInternal(FGamepadKeyNames::LeftStickDown, ControllerIndex, bNegativePressed, bWasNegativePressed);
		HandleInputInternal(FGamepadKeyNames::LeftStickUp, ControllerIndex, bPositivePressed, bWasPositivePressed);
	}
	else if (UEButtonNegative == FGamepadKeyNames::RightStickLeft && UEButtonPositive == FGamepadKeyNames::RightStickRight)
	{
		// Uses the "rightThumbstick.xAxis" of the controller, which is captured by "RightAnalogX"
		bWasNegativePressed = PrevState.Find(FGamepadKeyNames::RightAnalogX)->AxisValue <= -RepeatDeadzone;
		bNegativePressed = CurrentState.Find(FGamepadKeyNames::RightAnalogX)->AxisValue <= -RepeatDeadzone;
		bWasPositivePressed = PrevState.Find(FGamepadKeyNames::RightAnalogX)->AxisValue >= RepeatDeadzone;
		bPositivePressed = CurrentState.Find(FGamepadKeyNames::RightAnalogX)->AxisValue >= RepeatDeadzone;
		
		HandleInputInternal(FGamepadKeyNames::RightStickLeft, ControllerIndex, bNegativePressed, bWasNegativePressed);
		HandleInputInternal(FGamepadKeyNames::RightStickRight, ControllerIndex, bPositivePressed, bWasPositivePressed);
	}
	else if (UEButtonNegative == FGamepadKeyNames::RightStickDown && UEButtonPositive == FGamepadKeyNames::RightStickUp)
	{
		// Uses the "rightThumbstick.yAxis" of the controller, which is captured by "RightAnalogY"
		bWasNegativePressed = PrevState.Find(FGamepadKeyNames::RightAnalogY)->AxisValue <= -RepeatDeadzone;
		bNegativePressed = CurrentState.Find(FGamepadKeyNames::RightAnalogY)->AxisValue <= -RepeatDeadzone;
		bWasPositivePressed = PrevState.Find(FGamepadKeyNames::RightAnalogY)->AxisValue >= RepeatDeadzone;
		bPositivePressed = CurrentState.Find(FGamepadKeyNames::RightAnalogY)->AxisValue >= RepeatDeadzone;
		
		HandleInputInternal(FGamepadKeyNames::RightStickDown, ControllerIndex, bNegativePressed, bWasNegativePressed);
		HandleInputInternal(FGamepadKeyNames::RightStickUp, ControllerIndex, bPositivePressed, bWasPositivePressed);
	}
}

void FAppleControllerInterface::HandleButtonGamepad(const FGamepadKeyNames::Type& UEButton, uint32 ControllerIndex)
{
	bool bWasPressed = false;
	bool bIsPressed = false;
	const TMap<FName, FControllerButton>& PrevState = ControllerButtons[PrevButtonIndex][ControllerIndex];
	const TMap<FName, FControllerButton>& CurrentState = ControllerButtons[CurrentButtonIndex][ControllerIndex];

	bIsPressed = CurrentState.Find(UEButton)->bPressed;
	bWasPressed = PrevState.Find(UEButton)->bPressed;

	HandleInputInternal(UEButton, ControllerIndex, bIsPressed, bWasPressed);
}

void FAppleControllerInterface::HandleAnalogGamepad(const FGamepadKeyNames::Type& UEAxis, uint32 ControllerIndex)
{
	GCController* Cont = Controllers[ControllerIndex].Controller;
	IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
	FPlatformUserId UserId = FGenericPlatformMisc::GetPlatformUserForUserIndex(Controllers[ControllerIndex].PlayerIndex);
	FInputDeviceId DeviceId = DeviceMapper.GetPrimaryInputDeviceForUser(UserId);

	// Send controller events any time we are passed the given input threshold
	// similarly to PC/Console (see: XInputInterface.cpp)
	const float RepeatDeadzone = 0.24f;
	float AxisValue = 0;
	bool bSendAnalogChangedMessage = false;

	const TMap<FName, FControllerButton>& PrevState = ControllerButtons[PrevButtonIndex][ControllerIndex];
	const TMap<FName, FControllerButton>& CurrentState = ControllerButtons[CurrentButtonIndex][ControllerIndex];

	switch (Controllers[ControllerIndex].ControllerType)
	{
		case ControllerType::ExtendedGamepad:
		case ControllerType::DualShockGamepad:
		case ControllerType::XboxGamepad:
		case ControllerType::DualSenseGamepad:
		{
			if ((UEAxis == FGamepadKeyNames::LeftAnalogX) || (UEAxis == FGamepadKeyNames::RightAnalogX) || (UEAxis == FGamepadKeyNames::LeftTriggerAnalog) ||
				(UEAxis == FGamepadKeyNames::LeftAnalogY) || (UEAxis == FGamepadKeyNames::RightAnalogY) || (UEAxis == FGamepadKeyNames::RightTriggerAnalog))
			{
				float CurrentValue = CurrentState.Find(UEAxis)->AxisValue;
				float PrevValue = PrevState.Find(UEAxis)->AxisValue;
				if ((CurrentValue != PrevValue) || (CurrentValue < -RepeatDeadzone || CurrentValue > RepeatDeadzone))
				{
					AxisValue = CurrentValue;
					bSendAnalogChangedMessage = true;
				}
			}
			break;
		}

		case ControllerType::SiriRemote:
		{
			if ((UEAxis == FGamepadKeyNames::LeftAnalogX) || (UEAxis == FGamepadKeyNames::LeftAnalogY))
			{
				float CurrentValue = CurrentState.Find(UEAxis)->AxisValue;
				float PrevValue = CurrentState.Find(UEAxis)->AxisValue;
				if ((CurrentValue != PrevValue) || (CurrentValue < -RepeatDeadzone || CurrentValue > RepeatDeadzone))
				{
					AxisValue = CurrentValue;
					bSendAnalogChangedMessage = true;
				}
			}
			break;
		}

		default:
			UE_LOGF(LogAppleController, Warning, "Controller type is recognized you analogue");
			break;
	}
#if APPLE_CONTROLLER_DEBUG
	NSLog(@"Axis %s is %f", TCHAR_TO_ANSI(*UEAxis.ToString()), AxisValue);
#endif
	if (bSendAnalogChangedMessage)
	{
		MessageHandler->OnControllerAnalog(UEAxis, UserId, DeviceId, AxisValue);
	}
}

bool FAppleControllerInterface::ControllerHasHaptics(int32 ControllerId) const
{
	if (ControllerId < 0 || ControllerId >= UE_ARRAY_COUNT(Controllers))
	{
		return false;
	}
	GCController* Cont = Controllers[ControllerId].Controller;
	if (Cont == nil)
	{
		return false;
	}
	if (@available(iOS 14.0, macOS 11.0, tvOS 14.0, *))
	{
		return Cont.haptics != nil;
	}
	return false;
}

void FAppleControllerInterface::InitializeHapticsForController(FUserController& UserController)
{
	if (UserController.bHapticsInitialized || UserController.bHapticsInitFailed)
	{
		return;
	}
	if (UserController.Controller == nil)
	{
		return;
	}

	if (@available(iOS 14.0, macOS 11.0, tvOS 14.0, *))
	{
		GCDeviceHaptics* Haptics = UserController.Controller.haptics;
		if (Haptics == nil)
		{
			UserController.bHapticsInitFailed = true;
			return;
		}

		NSSet<GCHapticsLocality>* Supported = Haptics.supportedLocalities;
		const bool bHasLeft    = [Supported containsObject:GCHapticsLocalityLeftHandle];
		const bool bHasRight   = [Supported containsObject:GCHapticsLocalityRightHandle];
		const bool bHasHandles = [Supported containsObject:GCHapticsLocalityHandles];
		const bool bHasDefault = [Supported containsObject:GCHapticsLocalityDefault];

		UserController.bSplitHandleLocalities = bHasLeft && bHasRight;

		GCHapticsLocality LeftLocality = nil;
		GCHapticsLocality RightLocality = nil;
		if (UserController.bSplitHandleLocalities)
		{
			LeftLocality = GCHapticsLocalityLeftHandle;
			RightLocality = GCHapticsLocalityRightHandle;
		}
		else if (bHasHandles)
		{
			LeftLocality = GCHapticsLocalityHandles;
		}
		else if (bHasDefault)
		{
			LeftLocality = GCHapticsLocalityDefault;
		}
		else
		{
			UE_LOGF(LogAppleController, Log, "Controller exposes haptics but no usable locality; disabling.");
			UserController.bHapticsInitFailed = true;
			return;
		}

		auto BuildEngineAndPlayer = [this, &UserController](GCDeviceHaptics* H, GCHapticsLocality Locality,
			CHHapticEngine*& OutEngine, id<CHHapticPatternPlayer>& OutPlayer) -> bool
		{
			CHHapticEngine* Engine = [H createEngineWithLocality:Locality];
			if (Engine == nil)
			{
				return false;
			}

			[Engine retain];
			Engine.playsHapticsOnly = YES;
			Engine.autoShutdownEnabled = NO;

			// Reset/stop callbacks fire on Apple's internal queue. Plain bool stores are racy in theory,
			// but the only consumer is the next SetForceFeedback* call on the game thread which performs
			// a single read; a stale-by-one-frame read at worst delays re-init by one tick.
			FUserController* UCPtr = &UserController;
			Engine.resetHandler = ^{
				UCPtr->bHapticsInitialized = false;
				UCPtr->bHapticsPlayersStarted = false;
			};
			Engine.stoppedHandler = ^(CHHapticEngineStoppedReason) {
				UCPtr->bHapticsInitialized = false;
				UCPtr->bHapticsPlayersStarted = false;
			};

			NSError* StartError = nil;
			if (![Engine startAndReturnError:&StartError])
			{
				UE_LOGF(LogAppleController, Warning, "Failed to start CHHapticEngine: %ls", *FString([StartError localizedDescription]));
				[Engine release];
				return false;
			}

			// Build the pattern with full static intensity. CHHapticDynamicParameterIDHapticIntensityControl
			// MULTIPLIES the static value, so a static of 0 would mean every dynamic update produces silence.
			// Static sharpness is left at 0 because the dynamic sharpness parameter is ADDITIVE (0 + N = N).
			CHHapticPattern* Pattern = BuildContinuousHapticPattern(1.0f, 0.0f);
			if (Pattern == nil)
			{
				[Engine stopWithCompletionHandler:nil];
				[Engine release];
				return false;
			}

			NSError* PlayerError = nil;
			id<CHHapticPatternPlayer> Player = [Engine createPlayerWithPattern:Pattern error:&PlayerError];
			if (Player == nil)
			{
				UE_LOGF(LogAppleController, Warning, "Failed to create haptic player: %ls", *FString([PlayerError localizedDescription]));
				[Engine stopWithCompletionHandler:nil];
				[Engine release];
				return false;
			}
			[Player retain];

			OutEngine = Engine;
			OutPlayer = Player;
			return true;
		};

		if (!BuildEngineAndPlayer(Haptics, LeftLocality, UserController.EngineHandlesLeft, UserController.PlayerHandlesLeft))
		{
			UserController.bHapticsInitFailed = true;
			return;
		}

		if (UserController.bSplitHandleLocalities)
		{
			if (!BuildEngineAndPlayer(Haptics, RightLocality, UserController.EngineHandlesRight, UserController.PlayerHandlesRight))
			{
				// Roll back the left-side engine so we don't leak it; fall back to non-split mode.
				[UserController.PlayerHandlesLeft release];
				UserController.PlayerHandlesLeft = nil;
				[UserController.EngineHandlesLeft stopWithCompletionHandler:nil];
				[UserController.EngineHandlesLeft release];
				UserController.EngineHandlesLeft = nil;
				UserController.bSplitHandleLocalities = false;

				if (bHasHandles)
				{
					if (!BuildEngineAndPlayer(Haptics, GCHapticsLocalityHandles, UserController.EngineHandlesLeft, UserController.PlayerHandlesLeft))
					{
						UserController.bHapticsInitFailed = true;
						return;
					}
				}
				else
				{
					UserController.bHapticsInitFailed = true;
					return;
				}
			}
		}

		UserController.bHapticsInitialized = true;
		UserController.bHapticsPlayersStarted = false;
		UserController.LastForceFeedbackValues = FForceFeedbackValues();
	}
}

void FAppleControllerInterface::TearDownHapticsForController(FUserController& UserController)
{
	if (@available(iOS 14.0, macOS 11.0, tvOS 14.0, *))
	{
		// Stop and release players first so their handlers don't fire against a freed engine.
		if (UserController.PlayerHandlesLeft != nil)
		{
			NSError* StopErr = nil;
			[UserController.PlayerHandlesLeft stopAtTime:CHHapticTimeImmediate error:&StopErr];
			[UserController.PlayerHandlesLeft release];
			UserController.PlayerHandlesLeft = nil;
		}
		if (UserController.PlayerHandlesRight != nil)
		{
			NSError* StopErr = nil;
			[UserController.PlayerHandlesRight stopAtTime:CHHapticTimeImmediate error:&StopErr];
			[UserController.PlayerHandlesRight release];
			UserController.PlayerHandlesRight = nil;
		}
		if (UserController.EngineHandlesLeft != nil)
		{
			[UserController.EngineHandlesLeft stopWithCompletionHandler:nil];
			[UserController.EngineHandlesLeft release];
			UserController.EngineHandlesLeft = nil;
		}
		if (UserController.EngineHandlesRight != nil)
		{
			[UserController.EngineHandlesRight stopWithCompletionHandler:nil];
			[UserController.EngineHandlesRight release];
			UserController.EngineHandlesRight = nil;
		}
	}
	UserController.bHapticsInitialized = false;
	UserController.bHapticsInitFailed = false;
	UserController.bSplitHandleLocalities = false;
	UserController.bHapticsPlayersStarted = false;
	UserController.LastForceFeedbackValues = FForceFeedbackValues();
}

void FAppleControllerInterface::ApplyForceFeedbackValues(FUserController& UserController, const FForceFeedbackValues& Values)
{
	if (@available(iOS 14.0, macOS 11.0, tvOS 14.0, *))
	{
		if (!UserController.bHapticsInitialized)
		{
			InitializeHapticsForController(UserController);
		}
		if (!UserController.bHapticsInitialized)
		{
			return;
		}

		float LeftIntensity = 0.0f;
		float LeftSharpness = 0.0f;
		float RightIntensity = 0.0f;
		float RightSharpness = 0.0f;
		ResolveSideIntensitySharpness(Values.LeftLarge,  Values.LeftSmall,  LeftIntensity,  LeftSharpness);
		ResolveSideIntensitySharpness(Values.RightLarge, Values.RightSmall, RightIntensity, RightSharpness);

		const bool bAnythingActive = (LeftIntensity > 0.0f) || (RightIntensity > 0.0f);

		// Lazy-start players: the underlying CHHapticEvent is continuous and will run for ~30s once started,
		// so we only call start() when transitioning from idle->active and stop() on the reverse transition.
		if (bAnythingActive && !UserController.bHapticsPlayersStarted)
		{
			NSError* StartError = nil;
			if (UserController.PlayerHandlesLeft != nil &&
				![UserController.PlayerHandlesLeft startAtTime:CHHapticTimeImmediate error:&StartError])
			{
				UE_LOGF(LogAppleController, Warning, "Failed to start left haptic player: %ls", *FString([StartError localizedDescription]));
			}
			if (UserController.bSplitHandleLocalities && UserController.PlayerHandlesRight != nil &&
				![UserController.PlayerHandlesRight startAtTime:CHHapticTimeImmediate error:&StartError])
			{
				UE_LOGF(LogAppleController, Warning, "Failed to start right haptic player: %ls", *FString([StartError localizedDescription]));
			}
			UserController.bHapticsPlayersStarted = true;
		}

		if (UserController.bSplitHandleLocalities)
		{
			SendDynamicHapticParameters(UserController.PlayerHandlesLeft,  LeftIntensity,  LeftSharpness);
			SendDynamicHapticParameters(UserController.PlayerHandlesRight, RightIntensity, RightSharpness);
		}
		else
		{
			// Combined-handles mode: collapse both sides into a single intensity/sharpness pair.
			const float CombinedIntensity = FMath::Max(LeftIntensity, RightIntensity);
			const float CombinedTotal = LeftIntensity + RightIntensity;
			const float CombinedSharpness = (CombinedTotal > 0.0f)
				? FMath::Clamp((LeftSharpness * LeftIntensity + RightSharpness * RightIntensity) / CombinedTotal, 0.0f, 1.0f)
				: 0.0f;
			SendDynamicHapticParameters(UserController.PlayerHandlesLeft, CombinedIntensity, CombinedSharpness);
		}

		if (!bAnythingActive && UserController.bHapticsPlayersStarted)
		{
			NSError* StopError = nil;
			if (UserController.PlayerHandlesLeft != nil)
			{
				[UserController.PlayerHandlesLeft stopAtTime:CHHapticTimeImmediate error:&StopError];
			}
			if (UserController.bSplitHandleLocalities && UserController.PlayerHandlesRight != nil)
			{
				[UserController.PlayerHandlesRight stopAtTime:CHHapticTimeImmediate error:&StopError];
			}
			UserController.bHapticsPlayersStarted = false;
		}

		UserController.LastForceFeedbackValues = Values;
	}
}

void FAppleControllerInterface::SetForceFeedbackChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value)
{
	if (ControllerId < 0 || ControllerId >= UE_ARRAY_COUNT(Controllers))
	{
		return;
	}
	FUserController& UserController = Controllers[ControllerId];
	if (UserController.Controller == nil)
	{
		return;
	}

	FForceFeedbackValues Values = UserController.LastForceFeedbackValues;
	switch (ChannelType)
	{
		case FForceFeedbackChannelType::LEFT_LARGE:  Values.LeftLarge  = Value; break;
		case FForceFeedbackChannelType::LEFT_SMALL:  Values.LeftSmall  = Value; break;
		case FForceFeedbackChannelType::RIGHT_LARGE: Values.RightLarge = Value; break;
		case FForceFeedbackChannelType::RIGHT_SMALL: Values.RightSmall = Value; break;
	}
	ApplyForceFeedbackValues(UserController, Values);
}

void FAppleControllerInterface::SetForceFeedbackChannelValues(int32 ControllerId, const FForceFeedbackValues& Values)
{
	if (ControllerId < 0 || ControllerId >= UE_ARRAY_COUNT(Controllers))
	{
		return;
	}
	FUserController& UserController = Controllers[ControllerId];
	if (UserController.Controller == nil)
	{
		return;
	}
	ApplyForceFeedbackValues(UserController, Values);
}

void FAppleControllerInterface::SetLightColor(int32 ControllerId, FColor Color)
{
	if (ControllerId < 0 || ControllerId >= UE_ARRAY_COUNT(Controllers))
	{
		return;
	}
	GCController* Cont = Controllers[ControllerId].Controller;
	if (Cont == nil)
	{
		return;
	}
	if (@available(iOS 14.0, macOS 11.0, tvOS 14.0, *))
	{
		if (Cont.light != nil)
		{
			Cont.light.color = [[[GCColor alloc]
				initWithRed:(float)Color.R / 255.0f
				green:(float)Color.G / 255.0f
				blue:(float)Color.B / 255.0f] autorelease];
		}
	}
}

void FAppleControllerInterface::ResetLightColor(int32 ControllerId)
{
	if (ControllerId < 0 || ControllerId >= UE_ARRAY_COUNT(Controllers))
	{
		return;
	}
	const FColor& Default = Controllers[ControllerId].DefaultLightColor;
	SetLightColor(ControllerId, Default);
}

void FAppleControllerInterface::SetDeviceProperty(int32 ControllerId, const FInputDeviceProperty* Property)
{
	if (Property == nullptr)
	{
		return;
	}
	if (ControllerId < 0 || ControllerId >= UE_ARRAY_COUNT(Controllers))
	{
		return;
	}
	GCController* Cont = Controllers[ControllerId].Controller;
	if (Cont == nil)
	{
		return;
	}

	const FName PropertyName = Property->Name;

	if (PropertyName == FInputDeviceLightColorProperty::PropertyName())
	{
		const FInputDeviceLightColorProperty* Light = static_cast<const FInputDeviceLightColorProperty*>(Property);
		if (Light->bEnable)
		{
			SetLightColor(ControllerId, Light->Color);
		}
		else
		{
			ResetLightColor(ControllerId);
		}
		return;
	}

	// Adaptive-trigger properties only apply to GCDualSenseGamepad. iOS 14.5 / macOS 11.3 minimum.
	if (@available(iOS 14.5, macOS 11.3, tvOS 14.5, *))
	{
		if (![Cont.extendedGamepad isKindOfClass:[GCDualSenseGamepad class]])
		{
			return;
		}
		GCDualSenseGamepad* DS = (GCDualSenseGamepad*)Cont.extendedGamepad;

		auto ApplyToTriggers = [DS](EInputDeviceTriggerMask Mask, void (^Fn)(GCDualSenseAdaptiveTrigger*))
		{
			if (EnumHasAnyFlags(Mask, EInputDeviceTriggerMask::Left))
			{
				Fn(DS.leftTrigger);
			}
			if (EnumHasAnyFlags(Mask, EInputDeviceTriggerMask::Right))
			{
				Fn(DS.rightTrigger);
			}
		};

		if (PropertyName == FInputDeviceTriggerResetProperty::PropertyName())
		{
			const FInputDeviceTriggerResetProperty* Reset = static_cast<const FInputDeviceTriggerResetProperty*>(Property);
			ApplyToTriggers(Reset->AffectedTriggers, ^(GCDualSenseAdaptiveTrigger* T){ [T setModeOff]; });
		}
		else if (PropertyName == FInputDeviceTriggerFeedbackProperty::PropertyName())
		{
			const FInputDeviceTriggerFeedbackProperty* Feedback = static_cast<const FInputDeviceTriggerFeedbackProperty*>(Property);
			const float StartPos = FMath::Clamp((float)Feedback->Position / 255.0f, 0.0f, 1.0f);
			const float Strength = FMath::Clamp((float)Feedback->Strengh  / 255.0f, 0.0f, 1.0f);
			ApplyToTriggers(Feedback->AffectedTriggers, ^(GCDualSenseAdaptiveTrigger* T)
			{
				[T setModeFeedbackWithStartPosition:StartPos resistiveStrength:Strength];
			});
		}
		else if (PropertyName == FInputDeviceTriggerResistanceProperty::PropertyName())
		{
			const FInputDeviceTriggerResistanceProperty* Resist = static_cast<const FInputDeviceTriggerResistanceProperty*>(Property);
			const float StartPos = FMath::Clamp((float)Resist->StartPosition / 255.0f, 0.0f, 1.0f);
			const float EndPos   = FMath::Clamp((float)Resist->EndPosition   / 255.0f, 0.0f, 1.0f);
			// Apple takes a single resistive strength for weapon mode; average the two strengths.
			const float Strength = FMath::Clamp(((float)Resist->StartStrengh + (float)Resist->EndStrengh) / 510.0f, 0.0f, 1.0f);
			ApplyToTriggers(Resist->AffectedTriggers, ^(GCDualSenseAdaptiveTrigger* T)
			{
				[T setModeWeaponWithStartPosition:StartPos endPosition:EndPos resistiveStrength:Strength];
			});
		}
		else if (PropertyName == FInputDeviceTriggerVibrationProperty::PropertyName())
		{
			const FInputDeviceTriggerVibrationProperty* Vib = static_cast<const FInputDeviceTriggerVibrationProperty*>(Property);
			const float StartPos  = FMath::Clamp((float)Vib->TriggerPosition    / 255.0f, 0.0f, 1.0f);
			const float Amplitude = FMath::Clamp((float)Vib->VibrationAmplitude / 255.0f, 0.0f, 1.0f);
			const float Frequency = FMath::Clamp((float)Vib->VibrationFrequency / 255.0f, 0.0f, 1.0f);
			ApplyToTriggers(Vib->AffectedTriggers, ^(GCDualSenseAdaptiveTrigger* T)
			{
				[T setModeVibrationWithStartPosition:StartPos amplitude:Amplitude frequency:Frequency];
			});
		}
	}
}
