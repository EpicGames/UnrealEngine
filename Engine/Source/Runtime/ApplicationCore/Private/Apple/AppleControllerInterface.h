// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/CoreMisc.h"
#include "GenericPlatform/IInputInterface.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#include <GameController/GameController.h>
#include <CoreHaptics/CoreHaptics.h>

DECLARE_LOG_CATEGORY_EXTERN(LogAppleController, Log, All);

#define MAX_NUM_CONTROLLERS					4	// reasonable limit for now
#define MAX_CONTROLLER_CAPTURE_STATES		2	// track current and previous button states

enum ControllerType
{
	Unassigned,
	SiriRemote,
	ExtendedGamepad,
	XboxGamepad,
	DualShockGamepad,
	DualSenseGamepad
};

enum PlayerIndex
{
	PlayerOne,
	PlayerTwo,
	PlayerThree,
	PlayerFour,
	
	PlayerUnset = -1
};

enum class EAppleControllerEventType : int32
{
	Invalid,
	Connect,
	Disconnect,
	BecomeCurrent
};

struct FDeferredAppleControllerEvent
{
	FDeferredAppleControllerEvent(EAppleControllerEventType InEventType, GCController* InController)
	: EventType(InEventType)
	, Controller([InController retain])
	{}
	FDeferredAppleControllerEvent(const FDeferredAppleControllerEvent& Other)
	{
		EventType = Other.EventType;
		Controller = [Other.Controller retain];
	}
	~FDeferredAppleControllerEvent()
	{
		[Controller release];
	}
	EAppleControllerEventType EventType;
	GCController* Controller;
};

/**
 * Interface class for Apple Controllers
 */
class FAppleControllerInterface : public IInputInterface
{
public:

	static TSharedRef< FAppleControllerInterface > Create(  const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler );
	static TSharedPtr< FAppleControllerInterface > Get();

public:

	virtual ~FAppleControllerInterface() {}

	virtual void SetMessageHandler( const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler );

	/** Tick the interface (i.e check for new controllers) */
	void Tick( float DeltaTime );

	/**
	 * Poll for controller state and send events if needed
	 */
	void SendControllerEvents();

	/**
	 * Force Feedback / haptics / lightbar / adaptive trigger implementation routed through GameController and CoreHaptics.
	 */
	virtual void SetForceFeedbackChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) override;
	virtual void SetForceFeedbackChannelValues(int32 ControllerId, const FForceFeedbackValues &Values) override;
	virtual void SetLightColor(int32 ControllerId, FColor Color) override;
	virtual void ResetLightColor(int32 ControllerId) override;
	virtual void SetDeviceProperty(int32 ControllerId, const FInputDeviceProperty* Property) override;

	/** Returns true when the controller bound to ControllerId exposes a non-nil GCDeviceHaptics. */
	bool ControllerHasHaptics(int32 ControllerId) const;

	bool IsControllerAssignedToGamepad(int32 ControllerId) const;
	bool IsGamepadAttached() const;

	const ControllerType GetControllerType(uint32 ControllerIndex);
	void SetControllerType(uint32 ControllerIndex);
	FName GetControllerTypeName(const ControllerType InControllerType);
	
	GCControllerButtonInput* GetGCControllerButton(const FGamepadKeyNames::Type& ButtonKey, uint32 ControllerIndex);
	
	void HandleInputInternal(const FGamepadKeyNames::Type& UEButton, uint32 ControllerIndex, bool bIsPressed, bool bWasPressed);
	void HandleButtonGamepad(const FGamepadKeyNames::Type& UEButton, uint32 ControllerIndex);
	void HandleAnalogGamepad(const FGamepadKeyNames::Type& UEAxis, uint32 ControllerIndex);
	void HandleVirtualButtonGamepad(const FGamepadKeyNames::Type& UEButtonNegative, const FGamepadKeyNames::Type& UEButtonPositive, uint32 ControllerIndex);

protected:

	FAppleControllerInterface( const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler );
	
	void SignalEvent(EAppleControllerEventType InEventType, GCController* InController);

	struct FUserController;

private:

	void HandleConnection(GCController* Controller);
	void HandleDisconnect(GCController* Controller);

	void SetCurrentController(GCController* Controller);

	void CaptureControllerState(const GCController* Controller, uint32 ControllerIndex);

	void InitializeHapticsForController(FUserController& UserController);
	void TearDownHapticsForController(FUserController& UserController);
	void ApplyForceFeedbackValues(FUserController& UserController, const FForceFeedbackValues& Values);

protected:

	TSharedRef< FGenericApplicationMessageHandler > MessageHandler;

	/** Game controller objects (per user)*/
	struct FUserController
	{
		GCController* Controller;
		ControllerType ControllerType;
		PlayerIndex PlayerIndex;

		// CoreHaptics state — lazily initialized on first SetForceFeedback* call when the controller exposes
		// a non-nil GCController.haptics. All fields start nil/zeroed via FMemory::Memzero on connect/disconnect;
		CHHapticEngine* EngineHandlesLeft;
		CHHapticEngine* EngineHandlesRight;
		id<CHHapticPatternPlayer> PlayerHandlesLeft;
		id<CHHapticPatternPlayer> PlayerHandlesRight;
		bool bHapticsInitialized;
		bool bHapticsInitFailed;
		bool bSplitHandleLocalities;
		bool bHapticsPlayersStarted;       // true while at least one player is in the running state
		FForceFeedbackValues LastForceFeedbackValues;
		FColor DefaultLightColor;          // captured at connect time, used by ResetLightColor

#if !PLATFORM_MAC
		FQuat ReferenceAttitude;
		bool bNeedsReferenceAttitude;
		bool bHasReferenceAttitude;
#endif
	};

	/** Track individual buttons and axis values */
	struct FControllerButton
	{
		bool bPressed = false;
		float PressedValue = 0;	// the level of pressure being applied to the button
		float AxisValue = 0;

		void Reset()
		{
			bPressed = false;
			PressedValue = 0;
			AxisValue = 0;
		}
	};

	// Controller Event Callbacks are on the main thread - defer to tick processing
	FCriticalSection DeferredEventCS;
	TArray<FDeferredAppleControllerEvent> DeferredEvents;

	// there is a hardcoded limit of 4 controllers in the API
	FUserController Controllers[MAX_NUM_CONTROLLERS];

	uint32 CurrentButtonIndex = 0;
	uint32 PrevButtonIndex = 1;
	// We track 2 sets of buttons - the current state and the previous state
	// And there is a hardcoded limit of 4 controllers in the API
	TMap<FName, FControllerButton> ControllerButtons[MAX_CONTROLLER_CAPTURE_STATES][MAX_NUM_CONTROLLERS];
	
	TMap<FName, double> NextKeyRepeatTime;

	// should we allow controllers to send input
	bool bAllowControllers;

	static FString HardwareDeviceIdentifier_DefaultGamepad;
};
