// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleKeyboardController.h"

#include "AppleControllerInterface.h"

#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "HAL/PlatformTime.h"

#import <GameController/GCControllerButtonInput.h>
#import <GameController/GCKeyboard.h>
#import <GameController/GCKeyboardInput.h>
#import <GameController/GCKeyCodes.h>

#include "HAL/IConsoleManager.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ScopeRWLock.h"

namespace Apple::Cvars
{
	static bool bEnableGCKeyboardSupport = true;
	static FAutoConsoleVariableRef CVarEnableGCKeyboardSupport(
				TEXT("input.apple.enablegckeyboardsupport"),
				bEnableGCKeyboardSupport,
				TEXT("If set to false, the implementation for GCKeyboard support will tell the rest of the system that no keyboard is connected"));
}

void FAppleKeyboardController::RegisterGCKeyboardModifierKeyMappings() 
{
	constexpr int32 GCKeysToMapNum = 9;
	IncreaseModifierKeyMapAllocation(GCKeysToMapNum);

	ModifierKeysKeymap.Emplace(GCKeyCodeCapsLock, EModifierKeyFlags::CapsLocked);
	ModifierKeysKeymap.Emplace(GCKeyCodeLeftAlt, EModifierKeyFlags::LeftAltDown);
	ModifierKeysKeymap.Emplace(GCKeyCodeLeftGUI, EModifierKeyFlags::LeftCommandDown);
	ModifierKeysKeymap.Emplace(GCKeyCodeLeftControl, EModifierKeyFlags::LeftControlDown);
	ModifierKeysKeymap.Emplace(GCKeyCodeLeftShift, EModifierKeyFlags::LeftShiftDown);
	ModifierKeysKeymap.Emplace(GCKeyCodeRightAlt, EModifierKeyFlags::RightAltDown);
	ModifierKeysKeymap.Emplace(GCKeyCodeRightGUI, EModifierKeyFlags::RightCommandDown);
	ModifierKeysKeymap.Emplace(GCKeyCodeRightControl, EModifierKeyFlags::RightControlDown);
	ModifierKeysKeymap.Emplace(GCKeyCodeRightShift, EModifierKeyFlags::RightShiftDown);
}

FAppleKeyboardController::FAppleKeyboardController(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler, EInitFlags InitFlags) : 
	InitializationFlags(InitFlags),
	MessageHandler(InMessageHandler)
{
	if (@available(macOS 11.0, iOS 14.0, tvOS 14.0, *))
	{

		if (!EnumHasAnyFlags(InitFlags, EInitFlags::GCKeyboardKeyHandlerDisabled))
		{
			RegisterGCKeyboardModifierKeyMappings();
		}

		ConnectedEventObserverID = [[NSNotificationCenter defaultCenter] addObserverForName:GCKeyboardDidConnectNotification 
																		object:nil
																		queue:[NSOperationQueue mainQueue]
																		usingBlock:^(NSNotification* Notification) { HandleKeyboardConnected(); }];
		
		DisconnectedEventObserverID = [[NSNotificationCenter defaultCenter] addObserverForName:GCKeyboardDidDisconnectNotification
																			object:nil 
																			queue:[NSOperationQueue mainQueue]
																			usingBlock:^(NSNotification* Notification) { HandleKeyboardDisconnected(); }];
		
		FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddRaw(this, &FAppleKeyboardController::ForceUpdatePressedKeysState);
		
		if (!bHandledKeyboardConnection && GCKeyboard.coalescedKeyboard)
		{
			HandleKeyboardConnected();
		}
		
		bIsInitialized = true;
	}
	else
	{
		UE_LOGF(LogAppleController, Warning, "GCKeyboard API is not available in the current OS version. Physical keyboad dupport via FAppleKeyboardController will not be available.");	
	}
}

FAppleKeyboardController::~FAppleKeyboardController()
{
	if (@available(macOS 11.0, iOS 14.0, tvOS 14.0, *))
	{
		if (ConnectedEventObserverID)
		{
			[[NSNotificationCenter defaultCenter] removeObserver:ConnectedEventObserverID];
		}
		
		if (DisconnectedEventObserverID)
		{
			[[NSNotificationCenter defaultCenter] removeObserver:DisconnectedEventObserverID];
		}
		
		GCKeyboardInput* KeyboardInput = GCKeyboard.coalescedKeyboard ? GCKeyboard.coalescedKeyboard.keyboardInput : nullptr;
		if (KeyboardInput)
		{
			KeyboardInput.keyChangedHandler = nil;
		}
		
		FCoreDelegates::ApplicationHasEnteredForegroundDelegate.RemoveAll(this);
	}
}

void FAppleKeyboardController::SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
{
	MessageHandler = InMessageHandler;
}

bool FAppleKeyboardController::IsPrintableChar(TCHAR InChar) const
{
	const bool bIsPrintableASCII = InChar >= 32 && InChar <= 126;
	const bool bIsPrintableUnicode = InChar > 160;
	const bool bIsBackspace = InChar == '\b';
	const bool bIsEnter = InChar == '\n';
	return bIsPrintableASCII || bIsPrintableUnicode || bIsBackspace || bIsEnter;
}

void FAppleKeyboardController::SendControllerEvents()
{
	if (!bIsInitialized)
	{
		return;
	}

	const double CurrentTime = FPlatformTime::Seconds();
	FDeferredEvent CurrentEvent;
	while (DeferredKeyEvents.Dequeue(CurrentEvent))
	{
		UpdateEventTracking(CurrentEvent, CurrentTime);
		DispatchEvent(CurrentEvent);
	}
	
	UpdateAndDispatchRepeatEvents();
}

void FAppleKeyboardController::HandleKeyChangeEvent(GCKeyCode keyCode, BOOL pressed) 
{ 
	if (bIsKeyboardInhibited)
	{
		return;
	}

	FDeferredEvent NewKeyEvent;
	NewKeyEvent.KeyEventType = pressed ? EKeyEvent::KeyDown : EKeyEvent::KeyUp;
	NewKeyEvent.GCKeyCode = keyCode;
	NewKeyEvent.TranslatedKeyCode = KeyCodeToPlatformCodeTranslator.IsBound() ? KeyCodeToPlatformCodeTranslator.Execute(*NewKeyEvent.GCKeyCode) : *NewKeyEvent.GCKeyCode;
	NewKeyEvent.CharCode = KeyCodeToCharTranslator.IsBound() ? KeyCodeToCharTranslator.Execute(*NewKeyEvent.GCKeyCode) : 0;
	
	UE_LOGF(LogAppleController, Verbose, "[GC AppleKeyboardController %s] Key State Changed | GC KeyCode [%u] | State [%ls]", __func__, *NewKeyEvent.GCKeyCode, pressed ? TEXT("True") : TEXT("False"));
	
	DeferredKeyEvents.Enqueue(NewKeyEvent);
}

void FAppleKeyboardController::QueueExternalKeyChangeEvent(const FDeferredEvent& InExternalKeyEvent)
{
	if (bIsKeyboardInhibited)
	{
		return;
	}

	UE_LOGF(LogAppleController, Verbose, "[GC AppleKeyboardController %s] Key State Changed | KeyCode [%u] | State [%ls]", __func__, InExternalKeyEvent.TranslatedKeyCode, InExternalKeyEvent.KeyEventType ==  EKeyEvent::KeyDown ? TEXT("True") : TEXT("False"));
	
	DeferredKeyEvents.Enqueue(InExternalKeyEvent);
}

void FAppleKeyboardController::QueueOneOffExternalKeyEvent(const FDeferredEvent& InExternalKeyEvent)
{
	if (bIsKeyboardInhibited)
	{
		return;
	}
	
	FDeferredEvent EventCopy = InExternalKeyEvent;
	
	EventCopy.KeyEventType = EKeyEvent::KeyDown;
	QueueExternalKeyChangeEvent(EventCopy);
	
	EventCopy.KeyEventType = EKeyEvent::KeyUp;
	QueueExternalKeyChangeEvent(EventCopy);
	
	UE_LOGF(LogAppleController, Verbose, "[GC AppleKeyboardController %s] Key State Changed | KeyCode [%u] | State [%ls]", __func__, InExternalKeyEvent.TranslatedKeyCode, InExternalKeyEvent.KeyEventType ==  EKeyEvent::KeyDown ? TEXT("True") : TEXT("False"));	
}

FModifierKeysState FAppleKeyboardController::GetModifierKeys() const
{
	return CurrentModifierKeys.ToUEModifierKeyStates();
}

void FAppleKeyboardController::SetGCKeyCodeToPlatformCodeTranslator(const FKeyCodeTranslator& InExternalTranslatorDelegate)
{
	KeyCodeToPlatformCodeTranslator = InExternalTranslatorDelegate;
}

void FAppleKeyboardController::SetGCKeyCodeToCharTranslator(const FKeyCodeTranslator& InExternalTranslatorDelegate)
{
	KeyCodeToCharTranslator = InExternalTranslatorDelegate;
}

void FAppleKeyboardController::ProcessModifierKeyEvent(const FDeferredEvent& InDeferredEvent)
{
	
	if (EModifierKeyFlags* MappedModifierFlag = ModifierKeysKeymap.Find(InDeferredEvent.TranslatedKeyCode))
	{
		EModifierKeyFlags ModifiedFlagRef = *MappedModifierFlag;
		if (InDeferredEvent.KeyEventType == EKeyEvent::KeyDown)
		{
			EnumAddFlags(CurrentModifierKeys.ModifierFlags, ModifiedFlagRef);
		}
		else
		{
			EnumRemoveFlags(CurrentModifierKeys.ModifierFlags, ModifiedFlagRef);
		}
	}
}

FAppleKeyboardController::EKeyEvent FAppleKeyboardController::GetUpdatedKeyStateForKey(const FDeferredEvent& InDeferredEvent)
{
	GCKeyboardInput* KeyboardInput = GCKeyboard.coalescedKeyboard ? GCKeyboard.coalescedKeyboard.keyboardInput : nullptr;

	// We might be tracking external key events, for these we don't have an easy way to track the current state.
	// This method is used mainly for refreshing the local state after something outside the normal update loop happened, so assume key up.
	
	// @note : The only provider of external events at the moment is UI Kit, and that has a Press Cancelled callback we are using to clean up the state
	// when the app goes into the background or any other unexpected acction in the middle of a press happened, so Key Up is safe to assume. But if this API is started to being use
	// in macOS it might need a specific way to handle it.

	if (!InDeferredEvent.GCKeyCode.IsSet() || !KeyboardInput || bIsKeyboardInhibited)
	{
		return EKeyEvent::KeyUp;
	}

	GCControllerButtonInput* KeyInput = [KeyboardInput buttonForKeyCode: *InDeferredEvent.GCKeyCode];

	return KeyInput && KeyInput.pressed ? EKeyEvent::KeyDown : EKeyEvent::KeyUp;
}

void FAppleKeyboardController::DispatchEvent(const FDeferredEvent& InDeferredEvent)
{
	UE_LOGF(LogAppleController,
		   Verbose,
		   "[GC AppleKeyboardController %s] KeyCode [%u] | Type [%ls] | Is Repeat [%ls]",
		   __func__,
		   InDeferredEvent.TranslatedKeyCode,
		   InDeferredEvent.KeyEventType == EKeyEvent::KeyUp ? TEXT("UP") : TEXT("DOWN"),
		   InDeferredEvent.bIsRepeat ? TEXT("True") : TEXT("False"));
	
	ProcessModifierKeyEvent(InDeferredEvent);

	switch (InDeferredEvent.KeyEventType)
	{
		case EKeyEvent::KeyUp:
		{
			MessageHandler->OnKeyUp(InDeferredEvent.TranslatedKeyCode, InDeferredEvent.CharCode, false);
			break;
		}
		case EKeyEvent::KeyDown:
		{
			MessageHandler->OnKeyDown(InDeferredEvent.TranslatedKeyCode, InDeferredEvent.CharCode, InDeferredEvent.bIsRepeat);
			
			// Mimic how CMD is handled in MacApplication
			const bool IsCMDPressed = EnumHasAnyFlags(CurrentModifierKeys.ModifierFlags, EModifierKeyFlags::LeftCommandDown) || 
										EnumHasAnyFlags(CurrentModifierKeys.ModifierFlags, EModifierKeyFlags::RightCommandDown);

			if (!IsCMDPressed && IsPrintableChar((TCHAR)InDeferredEvent.CharCode))
			{
				MessageHandler->OnKeyChar((TCHAR) InDeferredEvent.CharCode, InDeferredEvent.bIsRepeat);
			}
			break;
		}
	case EKeyEvent::Invalid:
	default:
		ensure(false);
		break;
	}
}

void FAppleKeyboardController::UpdateEventTracking(const FDeferredEvent& InDeferredEvent, double CurrentTime)
{
	switch (InDeferredEvent.KeyEventType)
	{
		case EKeyEvent::KeyUp:
		{
			CurrentlyPressedKeyStates.Remove(InDeferredEvent.TranslatedKeyCode);
			break;
		}
		case EKeyEvent::KeyDown:
		{
			FDeferredEvent PressedKeyEventCopy = InDeferredEvent;
			PressedKeyEventCopy.LastRepeatTime = CurrentTime;
			CurrentlyPressedKeyStates.Add(InDeferredEvent.TranslatedKeyCode, PressedKeyEventCopy);
			break;
		}
	case EKeyEvent::Invalid:
	default:
		ensure(false);
		break;
	}
}

void FAppleKeyboardController::UpdateAndDispatchRepeatEvents()
{
	if (bIsKeyboardInhibited)
	{
		return;
	}

	const double CurrentTime = FPlatformTime::Seconds();
	
	bool bForceStateUpdate = bNeedUpdatePressedKeyState.exchange(false);
	
	// Force state update is used to double-check what is the actual state of a key in case it changed while the app was in the background
	// If it was unpressed we need to remove it from the current pressed key list, and dispatch the key up event
	TArray<FDeferredEvent, TInlineAllocator<32>> KeysUnpressedOutsideNormalLoop;

	for (TPair<uint32, FDeferredEvent>& PressedKeyEventWithId : CurrentlyPressedKeyStates)
	{
		double ElapsedTime = CurrentTime - PressedKeyEventWithId.Value.LastRepeatTime;
	
		// TODO : These are usually system settings. Find a way to read them that works in macOS and iOS
		constexpr double InitialRepeatDelay = 0.7;
		constexpr double RepeatInterval = 0.4;
	
		const double DesiredRepeatInterval = FMath::IsNearlyZero(PressedKeyEventWithId.Value.LastRepeatTime) ? InitialRepeatDelay : RepeatInterval;
		if (ElapsedTime > DesiredRepeatInterval)
		{
			PressedKeyEventWithId.Value.bIsRepeat = true;
			PressedKeyEventWithId.Value.LastRepeatTime = CurrentTime;
			DispatchEvent(PressedKeyEventWithId.Value);
		}
		
		if (bForceStateUpdate)
		{
			EKeyEvent CurrentKeyState = GetUpdatedKeyStateForKey(PressedKeyEventWithId.Value);
			
			if (CurrentKeyState == EKeyEvent::KeyUp)
			{
				FDeferredEvent KeyEventCopy = PressedKeyEventWithId.Value;
				KeyEventCopy.KeyEventType = CurrentKeyState;
				KeysUnpressedOutsideNormalLoop.Emplace(KeyEventCopy);
			}
		}
	}
	
	for (const FDeferredEvent& UnpressedKeyEvent : KeysUnpressedOutsideNormalLoop)
	{
		DispatchEvent(UnpressedKeyEvent);
		CurrentlyPressedKeyStates.Remove(UnpressedKeyEvent.TranslatedKeyCode);
	}
}

void FAppleKeyboardController::HandleKeyboardConnected() 
{
	UE_LOGF(LogAppleController, Verbose, "[GC AppleKeyboardController %s] Handling Keyboard Connection ", __func__);

	GCKeyboardInput* KeyboardInput = GCKeyboard.coalescedKeyboard ? GCKeyboard.coalescedKeyboard.keyboardInput : nullptr;
	
	if (ensure(KeyboardInput))
	{
		if (!EnumHasAnyFlags(InitializationFlags, EInitFlags::GCKeyboardKeyHandlerDisabled))
		{
			KeyboardInput.keyChangedHandler = ^(GCKeyboardInput* keyboard, GCDeviceButtonInput* key, GCKeyCode keyCode, BOOL pressed)
			{
				this->HandleKeyChangeEvent(keyCode, pressed); 
			};
		}
		
		OnAvailabilityChanged().Broadcast();

		bHandledKeyboardConnection = true;
	}
}

void FAppleKeyboardController::HandleKeyboardDisconnected() 
{ 
	UE_LOGF(LogAppleController, Verbose, "[GC AppleKeyboardController %s] Handling Keyboard Disconnection ", __func__);

	OnAvailabilityChanged().Broadcast();
	
	if (!GCKeyboard.coalescedKeyboard)
	{
		ForceUpdatePressedKeysState();
	}
	
	bHandledKeyboardConnection = false;
}

void FAppleKeyboardController::ForceUpdatePressedKeysState() 
{ 	
	// This is called in the OS main thread, but we want to update the data in the Game Thread
	bNeedUpdatePressedKeyState = true;
}

void FAppleKeyboardController::SetInhibited(bool bNewIsInhibited) 
{ 
	if (bIsKeyboardInhibited != bNewIsInhibited)
	{
		bIsKeyboardInhibited = bNewIsInhibited;
		
		ForceUpdatePressedKeysState();
	}
}

bool FAppleKeyboardController::IsAnyKeyboardConnected() const 
{ 
	if (@available(macOS 11.0, iOS 14.0, tvOS 14.0, *))
	{
		return Apple::Cvars::bEnableGCKeyboardSupport && GCKeyboard.coalescedKeyboard != nullptr;
	}
	else
	{
		return false;
	}
}

void FAppleKeyboardController::IncreaseModifierKeyMapAllocation(int32 ElementCount)
{
	if (ElementCount < 0)
	{
		return;
	}
	
	ModifierKeysKeymap.Reserve(ModifierKeysKeymap.Num() + ElementCount);
}

void FAppleKeyboardController::AddModifierKeyMapping(uint32 KeyCode, EModifierKeyFlags ModifierFlag) 
{ 
	ModifierKeysKeymap.Emplace(KeyCode, ModifierFlag);
}

