// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/SpscQueue.h"
#include "GenericPlatform/GenericApplication.h"
#include "Misc/Optional.h"

#include "Templates/SharedPointer.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#import <GameController/GCKeyCodes.h>

class FGenericApplicationMessageHandler;

/**
 * Class that intercepts, queues and forwards keyboard events from GCKeyboard to UE's input system 
 */
class FAppleKeyboardController
{
public:
	
	/** Set of flags used to determine how keyboard input events are handled */
	enum class EInitFlags : uint8
	{
		/** Default option where the internal GC Keyboard key change handler will be initialized */
		None = 0,
		/** If set, the GC keyboard Key Change handler will not be initialized, and this controller will only work with external input events.
		 * GC Keyboard will still be used to report if a physical keyboard is connected or not
		 */
		GCKeyboardKeyHandlerDisabled = 1 << 0, 
	};
	FRIEND_ENUM_CLASS_FLAGS(EInitFlags)

	
	explicit FAppleKeyboardController(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler, EInitFlags InitFlags = EInitFlags::None);
	~FAppleKeyboardController();

	/**
	 * Sets the message handler to which any input even will be forwarded to
	 * @param InMessageHandler New Message handler instance
	 */
	void SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler);
	
	/** Process and dispatches all queued key events to UE's input system */
	void SendControllerEvents();
	
	/** Returns the current state of all modifier keys */
	FModifierKeysState GetModifierKeys() const;
	
	DECLARE_DELEGATE_RetVal_OneParam(uint32, FKeyCodeTranslator, uint32);
	
	/**
	 * Sets an external callback that will perform any key code translation from GC Keycodes to an ASCII character if needed.
	 * @param InExternalTranslatorDelegate Callback that performs the transaltion
	 */
	void SetGCKeyCodeToCharTranslator(const FKeyCodeTranslator& InExternalTranslatorDelegate);
	
	/**
	 * Sets an external callback that will perform any key code translation from GC Keycodes to other keycode format if needed
	 * @param InExternalTranslatorDelegate Callback that performs the transaltion
	 */
	void SetGCKeyCodeToPlatformCodeTranslator(const FKeyCodeTranslator& InExternalTranslatorDelegate);

	/**
	 * Sets a flag to make sure in the next update look any currently key tracked as being pressed, re-queries the pressed state from the GCKeyboard API
	 */
	void ForceUpdatePressedKeysState();

	/**
	 * If set to true, keyboard input will be ignored 
	 * @param bNewIsInhibited new inhibited state
	 */
	void SetInhibited(bool bNewIsInhibited);

	/** Returns true if we are ignoring keyboard inputs */
	bool IsInhibited() const
	{
		return bIsKeyboardInhibited;
	}

	/** Returns true if any physical keyboard is connected */
	bool IsAnyKeyboardConnected() const;
	
	DECLARE_MULTICAST_DELEGATE(FKeyboardAvailabilityChanged)

	/**
	 * Delegate that is executed if a keyboard is connected or disconnected
	 */
	FKeyboardAvailabilityChanged& OnAvailabilityChanged()
	{
		return AvailabilityChangedDelegate;
	}
	
	/** Type of key events */
	enum class EKeyEvent : uint8
	{
		Invalid,
		KeyUp,
		KeyDown
	};

	/**
	 * Structure containing information about a key event that will be dispatched in the game thread
	 */
	struct FDeferredEvent
	{
		EKeyEvent KeyEventType = EKeyEvent::Invalid;
		TOptional<uint32> GCKeyCode;
		uint32 TranslatedKeyCode = 0;
		uint32 CharCode = 0;
		bool bIsRepeat = false;
		double LastRepeatTime = 0.0;
	};
	
	/**
	 * Queues a key event built externally  to be dispatched in the game thread
	 * @param Event to queue
	 */
	void QueueExternalKeyChangeEvent(const FDeferredEvent& InExternalKeyEvent);
	
	/**
	 * Queues a key event built externally to be dispatched in the game thread. The event will be transformed in its respective key up and key down events, to act as a one off key event
	 * @param Event to queue
	 */
	void QueueOneOffExternalKeyEvent(const FDeferredEvent& InExternalKeyEvent);
	
	/**
	 * Evaluates a character to see if it is a printable character (ASCII or Unicode)
	 * @return true if the character is printable
	 */
	bool IsPrintableChar(TCHAR InChar) const;
	
	enum class EModifierKeyFlags : uint16
	{
		None = 				0,
		LeftShiftDown = 	1 << 0,
		RightShiftDown = 	1 << 1,
		LeftControlDown = 	1 << 2,
		RightControlDown = 	1 << 3,
		LeftAltDown = 		1 << 4,
		RightAltDown = 		1 << 5,
		LeftCommandDown = 	1 << 6,
		RightCommandDown =	1 << 7,
		CapsLocked = 		1 << 8,
	};
	FRIEND_ENUM_CLASS_FLAGS(EModifierKeyFlags);

	/**
	 * Increases the memory allocation of the modifier keys map container
	 * by the specified number of elements. Use this if you expect to a known elements of mappings in a row
	 * @param ElementCount Number of elements you need room for
	 */
	void IncreaseModifierKeyMapAllocation(int32 ElementCount);
	
	/**
	 * Maps a specific key code to a key modifier flag
	 * @param KeyCode Key code to map
	 * @param ModifierFlag Modifier flag to map
	 */
	void AddModifierKeyMapping(uint32 KeyCode, EModifierKeyFlags ModifierFlag);

protected:
	
	EInitFlags InitializationFlags;
	
	FKeyboardAvailabilityChanged AvailabilityChangedDelegate;
	
	FKeyCodeTranslator KeyCodeToPlatformCodeTranslator;
	FKeyCodeTranslator KeyCodeToCharTranslator;

	TSharedRef<FGenericApplicationMessageHandler> MessageHandler;
	
	struct FInternalModifierKeyState
	{
		EModifierKeyFlags ModifierFlags = EModifierKeyFlags::None;
		
		FModifierKeysState ToUEModifierKeyStates() const
		{
			return FModifierKeysState(EnumHasAnyFlags(ModifierFlags, EModifierKeyFlags::LeftShiftDown), EnumHasAnyFlags(ModifierFlags, EModifierKeyFlags::RightShiftDown),
									  EnumHasAnyFlags(ModifierFlags, EModifierKeyFlags::LeftControlDown), EnumHasAnyFlags(ModifierFlags, EModifierKeyFlags::RightControlDown), 
									  EnumHasAnyFlags(ModifierFlags, EModifierKeyFlags::LeftAltDown), EnumHasAnyFlags(ModifierFlags, EModifierKeyFlags::RightAltDown), 
									  EnumHasAnyFlags(ModifierFlags, EModifierKeyFlags::LeftCommandDown), EnumHasAnyFlags(ModifierFlags, EModifierKeyFlags::RightCommandDown),
									  EnumHasAnyFlags(ModifierFlags, EModifierKeyFlags::CapsLocked));
		}
	};
	
	void RegisterGCKeyboardModifierKeyMappings();
	
	void HandleKeyboardConnected();
	void HandleKeyboardDisconnected();
	void HandleKeyChangeEvent(GCKeyCode keyCode, BOOL pressed);
	
	EKeyEvent GetUpdatedKeyStateForKey(const FDeferredEvent& InDeferredEvent);
	
	void ProcessModifierKeyEvent(const FDeferredEvent& InDeferredEvent);
	void DispatchEvent(const FDeferredEvent& InDeferredEvent);
	void UpdateAndDispatchRepeatEvents();
	void UpdateEventTracking(const FDeferredEvent& InDeferredEvent, double CurrentTime);

	FInternalModifierKeyState CurrentModifierKeys;
	
	bool bHandledKeyboardConnection = false;
	
	TMap<uint32, FDeferredEvent> CurrentlyPressedKeyStates;
	
	TSpscQueue<FDeferredEvent> DeferredKeyEvents;
	
	std::atomic<bool> bNeedUpdatePressedKeyState = false;
	std::atomic<bool> bIsKeyboardInhibited = false;

	bool bIsInitialized = false;
	
	id ConnectedEventObserverID = nil;
	id DisconnectedEventObserverID = nil;
	
	TMap<uint32, EModifierKeyFlags> ModifierKeysKeymap;
};
