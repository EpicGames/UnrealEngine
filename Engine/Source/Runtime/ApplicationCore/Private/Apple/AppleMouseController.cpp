// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleMouseController.h"

#include "AppleControllerInterface.h"
#include "Math/UnrealMathUtility.h"

#import "GameController/GCControllerButtonInput.h"
#import "GameController/GCDeviceCursor.h"
#import "GameController/GCMouseInput.h"

FAppleMouseController::FAppleMouseController(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler, EMouseControllerCapabilities InCapabilities) : MessageHandler(InMessageHandler), EnabledCapabilities(InCapabilities)
{
	if (@available(macOS 11.0, iOS 14.0, *))
	{
		ConnectedEventObserverID = [[NSNotificationCenter defaultCenter] addObserverForName:GCMouseDidConnectNotification 
																		object:nil
																		queue:[NSOperationQueue mainQueue]
																		usingBlock:^(NSNotification* Notification) { HandleMouseConnected((GCMouse *) Notification.object); }];
		
		DisconnectedEventObserverID = [[NSNotificationCenter defaultCenter] addObserverForName:GCMouseDidDisconnectNotification
																			object:nil 
																			queue:[NSOperationQueue mainQueue]
																			usingBlock:^(NSNotification* Notification) { HandleMouseDisconnected(); }];
		
		for (GCMouse* ConnectedMouse in GCMouse.mice)
		{
			HandleMouseConnected(ConnectedMouse);
		}
		
		bIsInitialized = true;
	}
	else
	{
		UE_LOGF(LogAppleController, Warning, "GCMouse API is not available in the current OS version. Physical mouse support via FAppleMouseController will not be available.");	
	}
}

FAppleMouseController::~FAppleMouseController()
{
	if (@available(macOS 11.0, iOS 14.0, *))
	{
		if (ConnectedEventObserverID)
		{
			[[NSNotificationCenter defaultCenter] removeObserver:ConnectedEventObserverID];
		}
		
		if (DisconnectedEventObserverID)
		{
			[[NSNotificationCenter defaultCenter] removeObserver:DisconnectedEventObserverID];
		}
		

		for (GCMouse* ConnectedMouse in GCMouse.mice)
		{
			GCMouseInput* MouseInput = ConnectedMouse ? ConnectedMouse.mouseInput : nullptr;
			
			if (MouseInput)
			{
				MouseInput.mouseMovedHandler = nil;
				MouseInput.scroll.valueChangedHandler = nil;
				MouseInput.leftButton.pressedChangedHandler = nil;
				MouseInput.rightButton.pressedChangedHandler = nil;
				MouseInput.middleButton.pressedChangedHandler = nil;
				
				constexpr uint64 MaxAuxButtonIndex = EMouseButtons::Type::Invalid - EMouseButtons::Type::Thumb01;
				const uint64 CurrentMaxAuxButtonIndex = MaxAuxButtonIndex > MouseInput.auxiliaryButtons.count ? MouseInput.auxiliaryButtons.count : MaxAuxButtonIndex;

				for (uint64 AuxButtonIndex = 0; AuxButtonIndex < CurrentMaxAuxButtonIndex; AuxButtonIndex++)
				{
					MouseInput.auxiliaryButtons[AuxButtonIndex].pressedChangedHandler = nil;
				}
			}
		}
	}
}

void FAppleMouseController::HandleMouseConnected(GCMouse* Mouse)
{
	UE_LOGF(LogAppleController, Log, "[%s] Handling physical mouse connection...", __func__);
	
	GCMouseInput* MouseInput = Mouse ? Mouse.mouseInput : nullptr;
	
	if (ensure(MouseInput))
	{
		if (EnumHasAnyFlags(EnabledCapabilities, EMouseControllerCapabilities::Movement))
		{
			MouseInput.mouseMovedHandler = ^(GCMouseInput * mouse, float deltaX, float deltaY)
			{
				if (!bEnabled)
				{
					return;
				}
				CurrentMovementState.MouseDeltaX += deltaX;
				CurrentMovementState.MouseDeltaY -= deltaY;
			};
		}

		if (EnumHasAnyFlags(EnabledCapabilities, EMouseControllerCapabilities::Scroll))
		{
			MouseInput.scroll.valueChangedHandler = ^(GCControllerDirectionPad* dpad, float xValue, float yValue)
			{
				if (!bEnabled)
				{
					return;
				}
				CurrentMovementState.ScrollDelta += yValue;
			};
		}

		if (EnumHasAnyFlags(EnabledCapabilities, EMouseControllerCapabilities::Buttons))
		{
			MouseInput.leftButton.pressedChangedHandler = ^(GCControllerButtonInput* eventButton, float value, BOOL pressed)
			{
				HandleKeyEvent(pressed, EMouseButtons::Type::Left);
			};

			MouseInput.rightButton.pressedChangedHandler = ^(GCControllerButtonInput* eventButton, float value, BOOL pressed) {
				HandleKeyEvent(pressed, EMouseButtons::Type::Right);
			};

			MouseInput.middleButton.pressedChangedHandler = ^(GCControllerButtonInput* eventButton, float value, BOOL pressed)
			{
				HandleKeyEvent(pressed, EMouseButtons::Type::Middle);
			};

			constexpr uint64 MaxAuxButtonIndex = EMouseButtons::Type::Invalid - EMouseButtons::Type::Thumb01;
			const uint64 CurrentMaxAuxButtonIndex = MaxAuxButtonIndex > MouseInput.auxiliaryButtons.count ? MouseInput.auxiliaryButtons.count : MaxAuxButtonIndex;

			for (uint64 AuxButtonIndex = 0; AuxButtonIndex < CurrentMaxAuxButtonIndex; AuxButtonIndex++)
			{
				MouseInput.auxiliaryButtons[AuxButtonIndex].pressedChangedHandler = ^(GCControllerButtonInput* eventButton, float value, BOOL pressed)
				{
					const EMouseButtons::Type TargetButton = static_cast<EMouseButtons::Type>(EMouseButtons::Type::Thumb01 + AuxButtonIndex);
					HandleKeyEvent(pressed, TargetButton);
				};
			}
		}
	}
	
	AvailabilityChangedDelegate.Broadcast();
}

void FAppleMouseController::HandleKeyEvent(bool bIsPressed, EMouseButtons::Type Button)
{
	if (!bEnabled)
	{
		return;
	}
	FDeferredKeyEvent KeyEvent;
	KeyEvent.ButtonID = Button;
	KeyEvent.Type = bIsPressed ? EKeyEventType::Down : EKeyEventType::Up;
	DeferredKeyEvents.Enqueue(KeyEvent);
}

void FAppleMouseController::DispatchMovementUpdates()
{
	if (EnumHasAnyFlags(EnabledCapabilities, EMouseControllerCapabilities::Movement))
	{
		FVector2D RawMovementDelta(CurrentMovementState.MouseDeltaX.exchange(0.0f), CurrentMovementState.MouseDeltaY.exchange(0.0f));
		
		// UE API only takes integer values as that is the unity most usb mouses report, but some devices like trackpads can return fractional, subpixel, movement deltas
		// To not lose that fractional delta value, we store it in a pending delta property, and apply it in the next loop
		RawMovementDelta += CurrentMovementState.PendingDeltaXY;
		CurrentMovementState.PendingDeltaXY.X = FMath::Fractional(RawMovementDelta.X);
		CurrentMovementState.PendingDeltaXY.Y = FMath::Fractional(RawMovementDelta.Y);

		FIntVector2 MovementDelta(FMath::TruncToInt(RawMovementDelta.X), FMath::TruncToInt(RawMovementDelta.Y));

		MessageHandler->OnRawMouseMove(MovementDelta.X, MovementDelta.Y);
		MessageHandler->OnMouseMove();
		MessageHandler->OnCursorSet();
		
		MovementDispatchedDelegate.Broadcast(MovementDelta);
	}
	
	if (EnumHasAnyFlags(EnabledCapabilities, EMouseControllerCapabilities::Scroll))
	{
		const float ScrollDelta = CurrentMovementState.ScrollDelta.exchange(0.0f);
		MessageHandler->OnMouseWheel(ScrollDelta);
	}
}

void FAppleMouseController::DispatchKeyUpdates()
{
	FDeferredKeyEvent DeferredEvent;
	while(DeferredKeyEvents.Dequeue(DeferredEvent))
	{
		DispatchKeyUpdate(DeferredEvent);
	}
}

void FAppleMouseController::DispatchKeyUpdate(const FDeferredKeyEvent& Event)
{
	switch (Event.Type) 
	{
		case EKeyEventType::Up:
			{
				MessageHandler->OnMouseUp(Event.ButtonID);
				break;
			}
		case EKeyEventType::Down:
			{
				MessageHandler->OnMouseDown(nullptr, Event.ButtonID);
				break;
			}
		case EKeyEventType::Invalid:
		default:
			ensure(false);
			break;
	}
}

void FAppleMouseController::HandleMouseDisconnected()
{
	AvailabilityChangedDelegate.Broadcast();

	UE_LOGF(LogAppleController, Log, "[%s] Handling physical mouse disconnection...", __func__);
}

void FAppleMouseController::SetEnabled(bool bInEnabled)
{
	if (bEnabled == bInEnabled)
	{
		return;
	}

	UE_LOGF(LogAppleController, Verbose, "[%s] %ls", __func__, bInEnabled ? TEXT("Enabling") : TEXT("Disabling"));

	bEnabled = bInEnabled;

	if (!bInEnabled)
	{
		bNeedFlushPendingState = true;
	}
}

bool FAppleMouseController::IsEnabled() const
{
	return bEnabled;
}

void FAppleMouseController::SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
{
	MessageHandler = InMessageHandler;
}

void FAppleMouseController::SendControllerEvents()
{
	if (!bIsInitialized)
	{
		return;
	}

	if (!bEnabled && !bNeedFlushPendingState.exchange(false))
	{
		return;
	}

	if (EnumHasAnyFlags(EnabledCapabilities, EMouseControllerCapabilities::Movement | EMouseControllerCapabilities::Scroll))
	{
		DispatchMovementUpdates();
	}

	if (EnumHasAnyFlags(EnabledCapabilities, EMouseControllerCapabilities::Buttons))
	{
		DispatchKeyUpdates();
	}
}

void FAppleMouseController::RebindMouseHandlers()
{
	if (@available(macOS 11.0, iOS 14.0, *))
	{
		for (GCMouse* ConnectedMouse in GCMouse.mice)
		{
			HandleMouseConnected(ConnectedMouse);
		}
	}
}

void FAppleMouseController::SuspendMovementHandler()
{
	if (@available(macOS 11.0, iOS 14.0, *))
	{
		for (GCMouse* ConnectedMouse in GCMouse.mice)
		{
			GCMouseInput* MouseInput = ConnectedMouse ? ConnectedMouse.mouseInput : nullptr;
			if (MouseInput)
			{
				MouseInput.mouseMovedHandler = nil;
			}
		}
	}
}

bool FAppleMouseController::IsAnyMouseConnected() const
{
	if (@available(macOS 11.0, iOS 14.0, *))
	{
		return GCMouse.mice.count > 0;
	}

	return false;
}
