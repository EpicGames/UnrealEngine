// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "Templates/SharedPointer.h"
#include <atomic>

#include "Containers/SpscQueue.h"
#include "Misc/EnumClassFlags.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#import "GameController/GCMouse.h"

/** Controls which GCMouse handlers FAppleMouseController binds */
enum class EMouseControllerCapabilities : uint8
{
	None       = 0,
	Movement   = 1 << 0,
	Scroll     = 1 << 1,
	Buttons    = 1 << 2,
	All        = Movement | Scroll | Buttons
};
ENUM_CLASS_FLAGS(EMouseControllerCapabilities)

/**
 * Class that intercepts, queues and dispatches GCMouse events to UE's input system
 */
class FAppleMouseController
{
public:
	explicit FAppleMouseController(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler, EMouseControllerCapabilities InCapabilities = EMouseControllerCapabilities::All);
	
	~FAppleMouseController();
	
	/**
	 * Sets the message handler to which any input even will be forwarded to
	 * @param InMessageHandler New Message handler instance
	 */
	void SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler);
	
	/** Process and dispatches all queued key events to UE's input system */
	void SendControllerEvents();
	
	/**
	 * Enables or disables mouse event accumulation and dispatch. Used it to in practice disable this controller,
	 * but keeping it initialized
	 * @param bInEnabled new enabled state
	 */
	void SetEnabled(bool bInEnabled);

	/** Returns true if event accumulation and dispatch are active */
	bool IsEnabled() const;

	bool IsAnyMouseConnected() const;

	/** Re-enumerate connected mice and re-bind GCMouse handlers. Call after macOS state
	 *  transitions (sleep, minimize) that may silently invalidate existing handlers. */
	void RebindMouseHandlers();

	/** Nil out the GCMouse movement handler on all connected mice so macOS routes
	 *  mouse deltas back through NSEvent. Call from ReassociateMouseInput. */
	void SuspendMovementHandler();
	
	DECLARE_MULTICAST_DELEGATE(FAvailabilityChanged)

	/**
	 * Delegate that is executed if a mouse  is connected or disconnected
	 */
	FAvailabilityChanged& OnAvailabilityChanged()
	{
		return AvailabilityChangedDelegate;
	}
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FMouseMovementDispatched, FIntVector2)

	/**
	 * Delegate that is executed every time a movement event is processed in the game thread
	 */
	FMouseMovementDispatched& OnMovementDispatched()
	{
		return MovementDispatchedDelegate;
	}

protected:
	
	FAvailabilityChanged AvailabilityChangedDelegate;
	FMouseMovementDispatched MovementDispatchedDelegate;
	
	enum class EKeyEventType
	{
		Invalid,
		Up,
		Down
	};
	
	struct FDeferredKeyEvent
	{
		EKeyEventType Type = EKeyEventType::Invalid;
		EMouseButtons::Type ButtonID;
	};

	void DispatchMovementUpdates();
	void DispatchKeyUpdates();
	void DispatchKeyUpdate(const FDeferredKeyEvent& Event);
	
	void HandleMouseConnected(GCMouse* Mouse);
	void HandleMouseDisconnected();
	void HandleKeyEvent(bool bIsPressed, EMouseButtons::Type Button);
	
	struct FMouseMovementState
	{
		std::atomic<float> MouseDeltaX = 0.0f;
		std::atomic<float> MouseDeltaY = 0.0f;
		std::atomic<float> ScrollDelta = 0.0f;
		
		/** The UE API only takes integer values. This property accumulates fractional delta (if any) to be consumed in the next iteration*/
		FVector2D PendingDeltaXY = FVector2D::ZeroVector;
	};
	
	FMouseMovementState CurrentMovementState;

	TSpscQueue<FDeferredKeyEvent> DeferredKeyEvents;
	TSharedRef<FGenericApplicationMessageHandler> MessageHandler;
	EMouseControllerCapabilities EnabledCapabilities;

	bool bIsInitialized = false;
	std::atomic<bool> bEnabled = true;
	std::atomic<bool> bNeedFlushPendingState = false;
	
	id ConnectedEventObserverID = nil;
	id DisconnectedEventObserverID = nil;
};
