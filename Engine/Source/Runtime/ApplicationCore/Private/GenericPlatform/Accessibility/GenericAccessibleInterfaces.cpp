// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"

DEFINE_LOG_CATEGORY(LogAccessibility);

#if WITH_ACCESSIBILITY
#include "HAL/IConsoleManager.h"

/** A flag that can be set by a user to force accessibility off regardless of other settings. */
bool GAllowAccessibility = false;
FAutoConsoleVariableRef AllowAccessibilityRef(
	TEXT("Accessibility.Enable"),
	GAllowAccessibility,
	TEXT("If false, all queries from accessible APIs will be ignored. On some platforms, the application must be restarted in order to take effect.")
);

FGenericAccessibleMessageHandler::FGenericAccessibleMessageHandler()
	: bApplicationIsAccessible(false)
	, bIsActive(false)
	, AccessibleUserRegistry(MakeShared<FGenericAccessibleUserRegistry>())
{}

bool FGenericAccessibleMessageHandler::ApplicationIsAccessible() const
{
	return bApplicationIsAccessible && GAllowAccessibility;
}

void FGenericAccessibleMessageHandler::SetActive(bool bActive)
{
	bActive &= GAllowAccessibility;
	if (bActive != bIsActive)
	{
		bIsActive = bActive;

		if (bIsActive)
		{
			UE_LOGF(LogAccessibility, Verbose, "Enabling Accessibility");
			OnActivate();
		}
		else
		{
			OnDeactivate();
			UE_LOGF(LogAccessibility, Verbose, "Accessibility Disabled");
		}
	}
}

bool FGenericAccessibleUserRegistry::RegisterUser(const TSharedRef<FGenericAccessibleUser>& User)
{
	const FAccessibleUserIndex UserIndex = User->GetIndex();
	if (!IsUserRegistered(UserIndex))
	{
		UsersMap.Add(UserIndex, User);
		User->OnRegistered();
		return true;
	}
	UE_LOGF(LogAccessibility, Verbose, "Registering accessible user %d failed. Another accessible user with accessible index %d already registered.", UserIndex, UserIndex);
	return false;
}

bool FGenericAccessibleUserRegistry::UnregisterUser(const FAccessibleUserIndex UserIndex)
{
	TSharedRef<FGenericAccessibleUser>* User = UsersMap.Find(UserIndex);
	if (User)
	{
		(*User)->OnUnregistered();
		UsersMap.Remove(UserIndex);
		return true;
	}
	UE_LOGF(LogAccessibility, Verbose, "Failed to unregister accessible user with accessible index %d. No accessible user with accessible index %d currently registered.", UserIndex, UserIndex);
	return false;
}

void FGenericAccessibleUserRegistry::UnregisterAllUsers()
{
	for (TPair<FAccessibleUserIndex, TSharedRef<FGenericAccessibleUser>>& UserPair : UsersMap)
	{
		UserPair.Value->OnUnregistered();
	}
	UsersMap.Empty();
}

bool FGenericAccessibleUserRegistry::IsUserRegistered(const FAccessibleUserIndex UserIndex) const
{
	return UsersMap.Contains(UserIndex);
}

TSharedPtr<FGenericAccessibleUser> FGenericAccessibleUserRegistry::GetUser(const FAccessibleUserIndex UserIndex) const
{
	if (IsUserRegistered(UserIndex))
	{
		return UsersMap[UserIndex];
	}
	return nullptr;
}

int32 FGenericAccessibleUserRegistry::GetNumberofUsers() const
{
	return UsersMap.Num();
}

TArray<TSharedRef<FGenericAccessibleUser>> FGenericAccessibleUserRegistry::GetAllUsers() const
{
	TArray<TSharedRef<FGenericAccessibleUser>> AllUsers;
	AllUsers.Reserve(UsersMap.Num());
	for (const TPair<FAccessibleUserIndex, TSharedRef<FGenericAccessibleUser>>& UserPair : UsersMap)
	{
		AllUsers.Add(UserPair.Value);
	}
	return AllUsers;
}

#endif

