// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/CursorUtils.h"
#include "CoreMinimal.h"


namespace UE::Cursor
{
	float CalculateDeltaWithAcceleration(int32 Delta, float Sensitivity)
	{
		constexpr float NominalMovement = 20.0f;
		const float Sign = (Delta > 0) ? 1.0f : -1.0f;
		const float FDelta = (float)Delta;

		return Sign * FMath::Max(FMath::Abs(FDelta), FMath::Abs(Sensitivity * FDelta * FDelta / NominalMovement));
	}
}
