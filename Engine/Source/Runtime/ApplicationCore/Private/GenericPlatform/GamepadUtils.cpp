// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GamepadUtils.h"
#include "HAL/IConsoleManager.h"
#include "Math/UnrealMathUtility.h"

namespace DynamicTriggerReleaseCVars
{
	static float DefaultDeadZone = 1.f;
	static FAutoConsoleVariableRef CVarTriggerDynamicReleaseDeadZone(TEXT("Input.TriggerDynamicReleaseDeadZone"),
		DefaultDeadZone,
		TEXT("Dynamic DeadZone for releasing analog triggers. It's dynamic in that it's relative to how far the trigger was pulled. Default of 1 means no dynamic release, letting triggers be released with their usual fixed threshold."));

	static bool bAllowOverride = true;
	static FAutoConsoleVariableRef CVarAllowTriggerDynamicReleaseDeadZoneCustomization(TEXT("Input.AllowTriggerDynamicReleaseDeadZoneCustomization"),
		bAllowOverride,
		TEXT("If true, the Dynamic Release DeadZone for a given analog trigger can be customized on supported devices by calling SetDeviceProperty with a FInputDeviceTriggerDynamicReleaseDeadZoneProperty. Otherwise, the Dynamic Release DeadZone is always the value of the CVar Input.TriggerDynamicReleaseDeadZone at initialization or when last Refreshed."));

	static float RePressFactor = 0.1f;
	static FAutoConsoleVariableRef CVarRePressFactor(TEXT("Input.TriggerDynamicReleaseDeadZoneRePressFactor"),
		RePressFactor,
		TEXT("When the trigger is considered released due to the Dynamic DeadZone, how far it needs to be pulled again to be considered pressed again. This will be multiplied by the Dynamic DeadZone, so lower values mean more sensitivity to pressing again."));

	static float MinimumRePress = 0.05f;
	static FAutoConsoleVariableRef CVarMinimumRePress(TEXT("Input.TriggerDynamicReleaseDeadZoneMinimumRePress"),
		MinimumRePress,
		TEXT("When the trigger is considered released due to the Dynamic DeadZone, how far it needs to be pulled again at a minimum to be considered pressed again. This is to prevent that a small Dynamic DeadZone and a small RePress Factor (see Input.TriggerDynamicReleaseDeadZoneRePressFactor) make the trigger overly sensitive to being considered pressed again."));
}

void FDynamicReleaseDeadZone::RefreshSettings()
{
	// Main settings:
	if (!DynamicTriggerReleaseCVars::bAllowOverride)
	{
		bHasOverride = false;
	}
	if (!bHasOverride)
	{
		DeadZone = FMath::Clamp(DynamicTriggerReleaseCVars::DefaultDeadZone, 0.f, 1.f);
	}
}

void FDynamicReleaseDeadZone::OverrideDeadZone(const float InDeadZone)
{
	if (DynamicTriggerReleaseCVars::bAllowOverride)
	{
		bHasOverride = true;
		DeadZone = FMath::Clamp(InDeadZone, 0.f, 1.f);
	}
	else if (bHasOverride)
	{
		bHasOverride = false;
		DeadZone = FMath::Clamp(DynamicTriggerReleaseCVars::DefaultDeadZone, 0.f, 1.f);
	}
}

bool FDynamicReleaseDeadZone::IsDynamicReleaseEnabled() const
{
	return DeadZone < 1.f;
}

bool FDynamicReleaseDeadZone::IsPressed(const uint8 TriggerAnalog, const uint8 PreviousTriggerAnalog)
{
	return IsPressed(TriggerAnalog, PreviousTriggerAnalog, TriggerAnalog > 0, PreviousTriggerAnalog > 0);
}

bool FDynamicReleaseDeadZone::IsPressed(const uint8 TriggerAnalog, const uint8 PreviousTriggerAnalog, const bool bIsSimplePressed, const bool bPreviousSimplePressed)
{
	bWasSimplePressed = bPreviousSimplePressed;
	bWasDynamicPressed = PreviousTriggerAnalog > TriggerThreshold;

	return IsPressed(TriggerAnalog, bIsSimplePressed);
}

bool FDynamicReleaseDeadZone::IsPressed(const uint8 TriggerAnalog, const bool bIsSimplePressed)
{
	bool bIsPressedResult = bIsSimplePressed;

	// If the threshold is 1.0, this trigger's thresholds aren't dynamic -- it's the same as the default behavior
	if (DeadZone < 1.f)
	{
		// Conservatively, changes in "Simple" trigger state (Pressed to Released or Released to Pressed) will take precedence over Dynamic Thresholds
		const bool bSimpleChanged = bIsSimplePressed != bWasSimplePressed;
		const bool bDynamicPressed = TriggerAnalog > TriggerThreshold;
		const bool bDynamicChanged = bDynamicPressed != bWasDynamicPressed;

		if (!bSimpleChanged)
		{
			bIsPressedResult = bDynamicPressed;
		}

		if (bSimpleChanged || bDynamicChanged)
		{
			// Reset dynamic thresholds so they can be recalculated correctly
			TriggerThreshold = bIsPressedResult ? 0 : 255;
		}

		const float RePressFactor = DynamicTriggerReleaseCVars::RePressFactor;
		const float MinimumRePress = DynamicTriggerReleaseCVars::MinimumRePress;

		// Update current dynamic thresholds
		if (bIsPressedResult)
		{
			const float ReleaseThresholdOffset = FMath::Max(DeadZone, MinimumRePress);
			const uint8 NewThreshold = (uint8)FMath::Max((int32)TriggerAnalog - (int32)(255 * ReleaseThresholdOffset) - 1, 0);
			TriggerThreshold = FMath::Max(TriggerThreshold, NewThreshold);
		}
		else
		{
			const float PressThresholdOffset = FMath::Max(DeadZone * RePressFactor, MinimumRePress);
			const uint8 NewThreshold = (uint8)FMath::Min((int32)TriggerAnalog + (int32)(255 * PressThresholdOffset), 254);
			TriggerThreshold = FMath::Min(TriggerThreshold, NewThreshold);
		}

		bWasDynamicPressed = bDynamicPressed;
	}
	else
	{
		bWasDynamicPressed = false;
	}

	bWasSimplePressed = bIsSimplePressed;

	return bIsPressedResult;
}
