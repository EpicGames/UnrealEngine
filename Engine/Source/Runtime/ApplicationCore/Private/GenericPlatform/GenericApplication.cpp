// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CommandLine.h"

const FGamepadKeyNames::Type FGamepadKeyNames::Invalid(NAME_None);

// Ensure that the GamepadKeyNames match those in InputCoreTypes.cpp
const FGamepadKeyNames::Type FGamepadKeyNames::LeftAnalogX("Gamepad_LeftX");
const FGamepadKeyNames::Type FGamepadKeyNames::LeftAnalogY("Gamepad_LeftY");
const FGamepadKeyNames::Type FGamepadKeyNames::RightAnalogX("Gamepad_RightX");
const FGamepadKeyNames::Type FGamepadKeyNames::RightAnalogY("Gamepad_RightY");
const FGamepadKeyNames::Type FGamepadKeyNames::LeftTriggerAnalog("Gamepad_LeftTriggerAxis");
const FGamepadKeyNames::Type FGamepadKeyNames::RightTriggerAnalog("Gamepad_RightTriggerAxis");

const FGamepadKeyNames::Type FGamepadKeyNames::LeftThumb("Gamepad_LeftThumbstick");
const FGamepadKeyNames::Type FGamepadKeyNames::RightThumb("Gamepad_RightThumbstick");
const FGamepadKeyNames::Type FGamepadKeyNames::SpecialLeft("Gamepad_Special_Left");
const FGamepadKeyNames::Type FGamepadKeyNames::SpecialLeft_X("Gamepad_Special_Left_X");
const FGamepadKeyNames::Type FGamepadKeyNames::SpecialLeft_Y("Gamepad_Special_Left_Y");
const FGamepadKeyNames::Type FGamepadKeyNames::SpecialLeft_Touched("Gamepad_Special_Left_Touched");
const FGamepadKeyNames::Type FGamepadKeyNames::SpecialRight("Gamepad_Special_Right");
const FGamepadKeyNames::Type FGamepadKeyNames::FaceButtonBottom("Gamepad_FaceButton_Bottom");
const FGamepadKeyNames::Type FGamepadKeyNames::FaceButtonRight("Gamepad_FaceButton_Right");
const FGamepadKeyNames::Type FGamepadKeyNames::FaceButtonLeft("Gamepad_FaceButton_Left");
const FGamepadKeyNames::Type FGamepadKeyNames::FaceButtonTop("Gamepad_FaceButton_Top");
const FGamepadKeyNames::Type FGamepadKeyNames::LeftShoulder("Gamepad_LeftShoulder");
const FGamepadKeyNames::Type FGamepadKeyNames::RightShoulder("Gamepad_RightShoulder");
const FGamepadKeyNames::Type FGamepadKeyNames::LeftTriggerThreshold("Gamepad_LeftTrigger");
const FGamepadKeyNames::Type FGamepadKeyNames::RightTriggerThreshold("Gamepad_RightTrigger");
const FGamepadKeyNames::Type FGamepadKeyNames::DPadUp("Gamepad_DPad_Up");
const FGamepadKeyNames::Type FGamepadKeyNames::DPadDown("Gamepad_DPad_Down");
const FGamepadKeyNames::Type FGamepadKeyNames::DPadRight("Gamepad_DPad_Right");
const FGamepadKeyNames::Type FGamepadKeyNames::DPadLeft("Gamepad_DPad_Left");

const FGamepadKeyNames::Type FGamepadKeyNames::LeftStickUp("Gamepad_LeftStick_Up");
const FGamepadKeyNames::Type FGamepadKeyNames::LeftStickDown("Gamepad_LeftStick_Down");
const FGamepadKeyNames::Type FGamepadKeyNames::LeftStickRight("Gamepad_LeftStick_Right");
const FGamepadKeyNames::Type FGamepadKeyNames::LeftStickLeft("Gamepad_LeftStick_Left");

const FGamepadKeyNames::Type FGamepadKeyNames::RightStickUp("Gamepad_RightStick_Up");
const FGamepadKeyNames::Type FGamepadKeyNames::RightStickDown("Gamepad_RightStick_Down");
const FGamepadKeyNames::Type FGamepadKeyNames::RightStickRight("Gamepad_RightStick_Right");
const FGamepadKeyNames::Type FGamepadKeyNames::RightStickLeft("Gamepad_RightStick_Left");

// The FInputDeviceRegistry should be used instead of the FInputDeviceScope for setting 
// device metadata like the name and type of device. The FInputDeviceRegistry improves on
// the FInputDeviceScope by being thread safe, as well as only being required once upon
// device connection, instead of being scoped around every single input event.
//
// If your project does depend on the FInputDeviceScope stack, you can set this in your *.Target.cs file:
//
// 	GlobalDefinitions.Add("UE_USE_LEGACY_INPUT_DEVICE_SCOPE=1");
#ifndef UE_USE_LEGACY_INPUT_DEVICE_SCOPE
	#define UE_USE_LEGACY_INPUT_DEVICE_SCOPE 0
#endif

PRAGMA_DISABLE_DEPRECATION_WARNINGS

namespace UE::InputDeviceScope::Private
{
using TArrayInputDeviceScope = TArray<FInputDeviceScope*, TInlineAllocator<16>>; 
TArrayInputDeviceScope& GetScopeStack()
{
	static thread_local TArrayInputDeviceScope ScopeStack;
	return ScopeStack;
}
}

FInputDeviceScope::FInputDeviceScope(IInputDevice* InInputDevice, FName InInputDeviceName, int32 InHardwareDeviceHandle, FString InHardwareDeviceIdentifier)
	: InputDevice(InInputDevice)
	, InputDeviceName(InInputDeviceName)
	, HardwareDeviceHandle(InHardwareDeviceHandle)
	, HardwareDeviceIdentifier(InHardwareDeviceIdentifier)
{
#if UE_USE_LEGACY_INPUT_DEVICE_SCOPE
	// Add to scope stack
	UE::InputDeviceScope::Private::GetScopeStack().Add(this);
#endif
}

FInputDeviceScope::~FInputDeviceScope()
{
#if UE_USE_LEGACY_INPUT_DEVICE_SCOPE
	UE::InputDeviceScope::Private::TArrayInputDeviceScope& ScopeStack = UE::InputDeviceScope::Private::GetScopeStack();

	// This should always be the top of the stack
	ensureMsgf((ScopeStack.Num() > 0 && ScopeStack.Last() == this), TEXT("FInputDeviceScope was not destroyed in correct order!"));
	ScopeStack.Remove(this);
#endif
}

const FInputDeviceScope* FInputDeviceScope::GetCurrent()
{
	UE::InputDeviceScope::Private::TArrayInputDeviceScope& ScopeStack = UE::InputDeviceScope::Private::GetScopeStack();
	if (ScopeStack.Num() > 0)
	{
		return ScopeStack.Last();
	}
	return nullptr;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

float GDebugSafeZoneRatio = 1.0f;
float GDebugActionZoneRatio = 1.0f;

static TAutoConsoleVariable<bool> CVarEnableSafeZoneOverrides(TEXT("SafeZone.EnableOverrides"), false, TEXT("Enables usage of individual SafeZone.Ratio CVars"));
static TAutoConsoleVariable<float> CVarSafeZoneRatioLeft(TEXT("SafeZone.Ratio.Left"), 1.0f, TEXT("Safe Zone Ratio - Left"));
static TAutoConsoleVariable<float> CVarSafeZoneRatioTop(TEXT("SafeZone.Ratio.Top"), 1.0f, TEXT("Safe Zone Ratio - Top"));
static TAutoConsoleVariable<float> CVarSafeZoneRatioRight(TEXT("SafeZone.Ratio.Right"), 1.0f, TEXT("Safe Zone Ratio - Right"));
static TAutoConsoleVariable<float> CVarSafeZoneRatioBottom(TEXT("SafeZone.Ratio.Bottom"), 1.0f, TEXT("Safe Zone Ratio - Bottom"));

struct FSafeZoneConsoleVariables
{
	FAutoConsoleVariableRef DebugSafeZoneRatioCVar;
	FAutoConsoleVariableRef DebugActionZoneRatioCVar;

	FSafeZoneConsoleVariables()
		: DebugSafeZoneRatioCVar(
			TEXT("r.DebugSafeZone.TitleRatio"),
			GDebugSafeZoneRatio,
			TEXT("The safe zone ratio that will be returned by FDisplayMetrics::GetDisplayMetrics on platforms that don't have a defined safe zone (0..1)\n")
			TEXT(" default: 1.0"))
		, DebugActionZoneRatioCVar(
			TEXT("r.DebugActionZone.ActionRatio"),
			GDebugActionZoneRatio,
			TEXT("The action zone ratio that will be returned by FDisplayMetrics::GetDisplayMetrics on platforms that don't have a defined safe zone (0..1)\n")
			TEXT(" default: 1.0"))
	{
		DebugSafeZoneRatioCVar->SetOnChangedCallback(FConsoleVariableDelegate::CreateRaw(this, &FSafeZoneConsoleVariables::OnDebugSafeZoneChanged));
		DebugActionZoneRatioCVar->SetOnChangedCallback(FConsoleVariableDelegate::CreateRaw(this, &FSafeZoneConsoleVariables::OnDebugSafeZoneChanged));
	}

	void OnDebugSafeZoneChanged(IConsoleVariable* Var)
	{
		FCoreDelegates::OnSafeFrameChangedEvent.Broadcast();
	}
};

FSafeZoneConsoleVariables GSafeZoneConsoleVariables;

FPlatformRect FDisplayMetrics::GetMonitorWorkAreaFromPoint(const FVector2D& Point) const
{
	for (const FMonitorInfo& Info : MonitorInfo)
	{
		// The point may not actually be inside the work area (for example on Windows taskbar or Mac menu bar), so we use DisplayRect to find the monitor
		if (Point.X >= Info.DisplayRect.Left && Point.X < Info.DisplayRect.Right && Point.Y >= Info.DisplayRect.Top && Point.Y < Info.DisplayRect.Bottom)
		{
			return Info.WorkArea;
		}
	}

	return FPlatformRect(0, 0, 0, 0);
}

int32 FDisplayMetrics::GetClosestMonitorFromIDAndIndex(const FString& MonitorID, int32 MonitorIndex) const
{
	if (!MonitorID.IsEmpty())
	{
		// Match monitor by ID and index
		for (int32 Index = 0, Num = MonitorInfo.Num(); Index < Num; ++Index)
		{
			const FMonitorInfo& Monitor = MonitorInfo[Index];
			if (Monitor.ID == MonitorID && Index == MonitorIndex)
			{
				return Index;
			}
		}

		// Match monitor by just ID
		for (int32 Index = 0, Num = MonitorInfo.Num(); Index < Num; ++Index)
		{
			const FMonitorInfo& Monitor = MonitorInfo[Index];
			if (Monitor.ID == MonitorID)
			{
				return Index;
			}
		}
	}

	// Match monitor by just index
	if (MonitorInfo.IsValidIndex(MonitorIndex))
	{
		return MonitorIndex;
	}

	// Grab the primary monitor
	for (int32 Index = 0, Num = MonitorInfo.Num(); Index < Num; ++Index)
	{
		const FMonitorInfo& Monitor = MonitorInfo[Index];
		if (Monitor.bIsPrimary)
		{
			return Index;
		}
	}

	// First monitor if there is one
	return MonitorInfo.IsEmpty() ? -1 : 0;
}

float FDisplayMetrics::GetDebugTitleSafeZoneRatio()
{
	return GDebugSafeZoneRatio;
}

bool FDisplayMetrics::TryGetTitleSafeZoneOverwrite(FVector4& InOutTitleSafePaddingSize)
{
#if PLATFORM_MAC
	if (CVarEnableSafeZoneOverrides.GetValueOnAnyThread())
#else
	if (CVarEnableSafeZoneOverrides.GetValueOnGameThread())
#endif
	{
		InOutTitleSafePaddingSize.X = (float)PrimaryDisplayWidth * (1.0f - CVarSafeZoneRatioLeft.GetValueOnGameThread()) * 0.5f;
		InOutTitleSafePaddingSize.Y = (float)PrimaryDisplayHeight * (1.0f - CVarSafeZoneRatioTop.GetValueOnGameThread()) * 0.5f;
		InOutTitleSafePaddingSize.Z = (float)PrimaryDisplayWidth * (1.0f - CVarSafeZoneRatioRight.GetValueOnGameThread()) * 0.5f;
		InOutTitleSafePaddingSize.W = (float)PrimaryDisplayHeight * (1.0f - CVarSafeZoneRatioBottom.GetValueOnGameThread()) * 0.5f;

		return true;
	}

	return false;
}

float FDisplayMetrics::GetDebugActionSafeZoneRatio()
{
	return GDebugActionZoneRatio;
}

void FDisplayMetrics::ApplyDefaultSafeZones()
{
	// Allow safe zones to be overridden by CVars. Used by streaming and mobile PIE.
	TitleSafePaddingSize = FVector4(0.0f, 0.0f, 0.0f, 0.0f);
	if (!TryGetTitleSafeZoneOverwrite(TitleSafePaddingSize))
	{
		const float SafeZoneRatio = GetDebugTitleSafeZoneRatio();
		if (SafeZoneRatio < 1.0f)
		{
			const float HalfUnsafeRatio = (1.0f - SafeZoneRatio) * 0.5f;
			TitleSafePaddingSize = FVector4((float)PrimaryDisplayWidth * HalfUnsafeRatio, (float)PrimaryDisplayHeight * HalfUnsafeRatio, (float)PrimaryDisplayWidth * HalfUnsafeRatio, (float)PrimaryDisplayHeight * HalfUnsafeRatio);
		}
	}
	
	const float ActionSafeZoneRatio = GetDebugActionSafeZoneRatio();
	if (ActionSafeZoneRatio < 1.0f)
	{
		const float HalfUnsafeRatio = (1.0f - ActionSafeZoneRatio) * 0.5f;
		ActionSafePaddingSize = FVector4((float)PrimaryDisplayWidth * HalfUnsafeRatio, (float)PrimaryDisplayHeight * HalfUnsafeRatio, (float)PrimaryDisplayWidth * HalfUnsafeRatio, (float)PrimaryDisplayHeight * HalfUnsafeRatio);
	}
}

void FDisplayMetrics::PrintToLog() const
{
	UE_LOGF(LogInit, Log, "Display metrics:");
	UE_LOGF(LogInit, Log, "  PrimaryDisplayWidth: %d", PrimaryDisplayWidth);
	UE_LOGF(LogInit, Log, "  PrimaryDisplayHeight: %d", PrimaryDisplayHeight);
	UE_LOGF(LogInit, Log, "  PrimaryDisplayWorkAreaRect:");
	UE_LOGF(LogInit, Log, "    Left=%d, Top=%d, Right=%d, Bottom=%d", 
		PrimaryDisplayWorkAreaRect.Left, PrimaryDisplayWorkAreaRect.Top, 
		PrimaryDisplayWorkAreaRect.Right, PrimaryDisplayWorkAreaRect.Bottom);

	UE_LOGF(LogInit, Log, "  VirtualDisplayRect:");
	UE_LOGF(LogInit, Log, "    Left=%d, Top=%d, Right=%d, Bottom=%d", 
		VirtualDisplayRect.Left, VirtualDisplayRect.Top, 
		VirtualDisplayRect.Right, VirtualDisplayRect.Bottom);

	UE_LOGF(LogInit, Log, "  TitleSafePaddingSize: %ls", *TitleSafePaddingSize.ToString());
	UE_LOGF(LogInit, Log, "  ActionSafePaddingSize: %ls", *ActionSafePaddingSize.ToString());

	const int NumMonitors = MonitorInfo.Num();
	UE_LOGF(LogInit, Log, "  Number of monitors: %d", NumMonitors);

	for (int MonitorIdx = 0; MonitorIdx < NumMonitors; ++MonitorIdx)
	{
		const FMonitorInfo & Info = MonitorInfo[MonitorIdx];
		UE_LOGF(LogInit, Log, "    Monitor %d", MonitorIdx);
		UE_LOGF(LogInit, Log, "      Name: %ls", *Info.Name);
		UE_LOGF(LogInit, Log, "      ID: %ls", *Info.ID);
		UE_LOGF(LogInit, Log, "      NativeWidth: %d", Info.NativeWidth);
		UE_LOGF(LogInit, Log, "      NativeHeight: %d", Info.NativeHeight);
		UE_LOGF(LogInit, Log, "      bIsPrimary: %ls", Info.bIsPrimary ? TEXT("true") : TEXT("false"));
	}
}

GenericApplication::GenericApplication(const TSharedPtr< ICursor >& InCursor)
	: Cursor(InCursor)
	, MessageHandler(MakeShareable(new FGenericApplicationMessageHandler()))
#if WITH_ACCESSIBILITY
	, AccessibleMessageHandler(MakeShareable(new FGenericAccessibleMessageHandler()))
#endif
{

}
GenericApplication::~GenericApplication() = default;

#if WITH_ACCESSIBILITY
void GenericApplication::SetAccessibleMessageHandler(const TSharedRef<FGenericAccessibleMessageHandler>& InAccessibleMessageHandler)
{
	AccessibleMessageHandler = InAccessibleMessageHandler;
}

TSharedRef<FGenericAccessibleMessageHandler> GenericApplication::GetAccessibleMessageHandler() const
{
	return AccessibleMessageHandler;
}
#endif
