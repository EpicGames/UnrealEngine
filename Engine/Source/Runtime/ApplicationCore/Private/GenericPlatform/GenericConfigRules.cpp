// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericConfigRules.h"
#include "Containers/StringConv.h"
#include "String/ParseLines.h"
#include "String/ParseTokens.h"
#include "String/RemoveFrom.h"
#include "Containers/StringConv.h"
#include "Internationalization/Regex.h"

DEFINE_LOG_CATEGORY(LogConfigRules);

namespace ConfigRulesUtil
{
	static FStringView RemoveSurrounds(FStringView Input, FStringView Entry, FStringView Exit);
	static FStringView SubStr(FStringView Input, int32 StartIdx, int32 EndIdx);
	static int32 IndexOf(const FString& Input, FStringView Str, int32 FromIndex = 0);
	static TArray<FStringView> ParseSegments(FStringView Input, FStringView Split, FStringView Entry, FStringView Exit);
	static FString ExpandVariables(TMap<FString, FString>& Variables, FStringView Input);
	static bool EvaluateConditions(TMap<FString, FString>& Variables, TArray<FStringView>& Conditions, FString& PreviousRegexMatch);
	static bool EvaluateConditions(TMap<FString, FString>& Variables, TArray<FStringView>& Conditions);
}

TMap<FString, FString> FGenericConfigRules::ParseConfigRules(TConstArrayView<uint8> ConfigRulesData,
	TMap<FString, FString>&& PredefinedVariables)
{
	enum class EConfRuleState : uint8
	{
		Run = 0,
		ExecTrue,
		FindElse,
		ExecFalse,
		FindEnd
	};

	// TODO: we should be able to parse this entire file without conversion to tchar.
	// Some functions such as ParseLines, LexFromString etc. expect tchar.
	auto ConvertedString = StringCast<TCHAR>((const char*)ConfigRulesData.GetData(), ConfigRulesData.Num());
	FStringView ConfigRules = FStringView(ConvertedString.Get(), ConvertedString.Length());

	TMap<FString, FString> ConfigRuleVars;
	ConfigRuleVars.Append(MoveTemp(PredefinedVariables));
	TArray<EConfRuleState> StateStack;

	EConfRuleState CurrentState = EConfRuleState::Run;
	int32 NestDepth = 0;
	bool bAbort = false;
	UE::String::ParseLines(ConfigRules,
		[&ConfigRuleVars, &CurrentState, &StateStack, &NestDepth, &bAbort](FStringView Line)
		{
			FStringView line = Line.TrimStartAndEnd();
			if (line.Len() < 1 || bAbort)
			{
				return;
			}
			if (line.StartsWith(TEXT("//"), ESearchCase::Type::CaseSensitive) || line.StartsWith(TEXT(";"), ESearchCase::Type::CaseSensitive))
			{
				if (line.StartsWith(TEXT("// version:")))
				{
					int32 configRulesVersion = 0;
					LexFromString(configRulesVersion, *FString(line.RightChop(11)));
					ConfigRuleVars.Add(TEXT("configRulesVersion"), FString::FromInt(configRulesVersion));
					UE_LOGF(LogConfigRules, Log, "ConfigRules version: %d", configRulesVersion);
				}
				return;
			}

			// look for Command
			int32 index = INDEX_NONE;
			if (!line.FindChar(':', index))
			{
				return;
			}
			FStringView Command = line.Left(index).TrimStartAndEnd();
			line = line.RightChop(index + 1).TrimStartAndEnd();

			// handle states
			switch (CurrentState)
			{
			case EConfRuleState::Run:
			{
				if (Command.Equals(TEXT("else")) || Command.Equals(TEXT("elseif")) || Command.Equals(TEXT("endif")))
				{
					UE_LOGF(LogConfigRules, Error, "ConfigRules: unexpected %ls encountered!", *FString(Command));
				}
				break;
			}

			case EConfRuleState::ExecTrue:
			{
				if (Command.Equals(TEXT("else")))
				{
					CurrentState = EConfRuleState::FindEnd;
				}
				else if (Command.Equals(TEXT("endif")))
				{
					CurrentState = StateStack.Pop();
				}
				return;
			}

			case EConfRuleState::FindElse:
			{
				if (Command.Equals(TEXT("if")))
				{
					NestDepth++;
					return;
				}
				if (NestDepth > 0)
				{
					if (Command.Equals(TEXT("endif")))
					{
						NestDepth--;
					}
					return;
				}
				if (Command.Equals(TEXT("endif")))
				{
					CurrentState = StateStack.Pop();
					return;
				}
				if (Command.Equals(TEXT("else")))
				{
					CurrentState = EConfRuleState::ExecFalse;
					return;
				}
				if (Command.Equals(TEXT("elseif")))
				{
					CurrentState = EConfRuleState::FindEnd;

					TArray<FStringView> Conditions;
					UE::String::ParseTokensMultiple(line, TConstArrayView<TCHAR>({ ',','(',')' }), Conditions);
					if (Conditions.Num() > 0)
					{
						bool bConditionTrue = ConfigRulesUtil::EvaluateConditions(ConfigRuleVars, Conditions);
						CurrentState = bConditionTrue ? EConfRuleState::ExecTrue : EConfRuleState::FindElse;
					}
				}
				return;
			}
			case EConfRuleState::ExecFalse:
			{
				if (Command.Equals(TEXT("endif")))
				{
					CurrentState = StateStack.Pop();
				}
				if (Command.Equals(TEXT("else")) || Command.Equals(TEXT("elseif")))
				{
					UE_LOGF(LogConfigRules, Error, "ConfigRules: unexpected %ls while handling false condition!", *FString(Command));
				}
				return;
			}
			case EConfRuleState::FindEnd:
			{
				if (Command.Equals(TEXT("if")))
				{
					NestDepth++;
				}
				else if (Command.Equals(TEXT("endif")))
				{
					if (NestDepth > 0)
					{
						NestDepth--;
					}
					else
					{
						CurrentState = StateStack.Pop();
					}
				}
				return;
			}
			default:
				UE_LOGF(LogConfigRules, Error, "ConfigRules: unknown state!");
				return;
			}

			// handle commands
			if (Command.Equals(TEXT("set")))
			{
				// set:(a=b[,c=d,...])
				TArray<FStringView> Sets = ConfigRulesUtil::ParseSegments(ConfigRulesUtil::RemoveSurrounds(line, TEXT("("), TEXT(")")), TEXT(","), TEXT("(\")"), TEXT(")\")"));
				for (FStringView Assignment : Sets)
				{
					TArray<FStringView> KeyValue = ConfigRulesUtil::ParseSegments(Assignment, TEXT("="), TEXT("\""), TEXT("\""));
					if (KeyValue.Num() == 2)
					{
						FStringView Key = ConfigRulesUtil::RemoveSurrounds(KeyValue[0], TEXT("\""), TEXT("\""));
						FString Value = ConfigRulesUtil::ExpandVariables(ConfigRuleVars, ConfigRulesUtil::RemoveSurrounds(KeyValue[1], TEXT("\""), TEXT("\"")));
						check(!Key.IsEmpty());
						if (Key.StartsWith(TEXT("APPEND_")))
						{
							Key = Key.RightChop(7);
							if (FString* Found = ConfigRuleVars.Find(FString(Key)))
							{
								Value = (*Found) + Value;
							}
							ConfigRuleVars.Add(FString(Key), Value);
						}
						else
						{
							ConfigRuleVars.Add(FString(Key), Value);
						}
					}
				}
			}
			else if (Command.Equals(TEXT("clear")))
			{
				// clear:(a[,b,...])
				TArray<FStringView> Sets = ConfigRulesUtil::ParseSegments(ConfigRulesUtil::RemoveSurrounds(Line, TEXT("("), TEXT(")")), TEXT(","), TEXT("(\""), TEXT(")\""));
				for (FStringView Key : Sets)
				{
					ConfigRuleVars.Remove(FString(ConfigRulesUtil::RemoveSurrounds(Key, TEXT("\""), TEXT("\""))));
				}
			}
			else if (Command.Equals(TEXT("chipset")))
			{
				// not supported here.
			}
			else if (Command.Equals(TEXT("if")))
			{
				// if:(SourceType=SRC_DeviceMake,CompareType=CMP_Equal,MatchString="samsung")
				// ... commands for true for all conditions
				// elseif:(SourceType=SRC_DeviceMake,CompareType=CMP_Equal,MatchString="Google")
				// ... commands for true for all conditions
				// else:
				// ... commands for false for any condition
				// end:
				StateStack.Push(CurrentState);
				CurrentState = EConfRuleState::FindEnd;

				TArray<FStringView> Conditions = ConfigRulesUtil::ParseSegments(Line, TEXT(","), TEXT("(\""), TEXT(")\""));
				if (!Conditions.IsEmpty())
				{
					const bool bConditionTrue = ConfigRulesUtil::EvaluateConditions(ConfigRuleVars, Conditions);
					CurrentState = bConditionTrue ? EConfRuleState::ExecTrue : EConfRuleState::FindElse;
				}
			}
			else if (Command.Equals(TEXT("condition")))
			{
				// condition:((SourceType=SRC_DeviceMake,CompareType=CMP_Equal,MatchString="samsung")),(SourceType=,CompareType=,MatchString=),...]),(a=b[,c=d,...]),(a[,b,...])
				// if all the conditions are true, execute the optional sets and/or clears
				TArray<FStringView> ConditionAndSets = ConfigRulesUtil::ParseSegments(line, TEXT(","), TEXT("(\""), TEXT(")\""));
				int32 SetSize = ConditionAndSets.Num();
				if (SetSize == 2 || SetSize == 3)
				{
					TArray<FStringView> Conditions = ConfigRulesUtil::ParseSegments(ConfigRulesUtil::RemoveSurrounds(ConditionAndSets[0], TEXT("("), TEXT(")")), TEXT(","), TEXT("(\""), TEXT(")\""));
					TArray<FStringView> Sets = ConfigRulesUtil::ParseSegments(ConfigRulesUtil::RemoveSurrounds(ConditionAndSets[1], TEXT("("), TEXT(")")), TEXT(","), TEXT("(\""), TEXT(")\""));
					TArray<FStringView> Clears = (SetSize == 3) ? ConfigRulesUtil::ParseSegments(ConfigRulesUtil::RemoveSurrounds(ConditionAndSets[2], TEXT("("), TEXT(")")), TEXT(","), TEXT("(\""), TEXT(")\"")) : TArray<FStringView>();

					bool bConditionTrue = ConfigRulesUtil::EvaluateConditions(ConfigRuleVars, Conditions);

					if (bConditionTrue)
					{
						// run the sets
						for (FStringView Assignment : Sets)
						{
							TArray<FStringView> KeyValue = ConfigRulesUtil::ParseSegments(Assignment, TEXT("="), TEXT("\""), TEXT("\""));
							if (KeyValue.Num() == 2)
							{
								FStringView Key = ConfigRulesUtil::RemoveSurrounds(KeyValue[0], TEXT("\""), TEXT("\""));
								FString Value = ConfigRulesUtil::ExpandVariables(ConfigRuleVars, ConfigRulesUtil::RemoveSurrounds(KeyValue[1], TEXT("\""), TEXT("\"")));
								check(!Key.IsEmpty());
								if (Key.StartsWith(TEXT("APPEND_")))
								{
									Key = Key.RightChop(7);
									if (FString* Found = ConfigRuleVars.Find(FString(Key)))
									{
										Value = (*Found) + Value;
									}
									ConfigRuleVars.Add(FString(Key), Value);
								}
								else
								{
									ConfigRuleVars.Add(FString(Key), Value);
								}
							}
						}

						// run the clears
						for (FStringView Key : Clears)
						{
							ConfigRuleVars.Remove(FString(ConfigRulesUtil::RemoveSurrounds(Key, TEXT("\""), TEXT("\""))));
						}
					}
				}
			}
			// see if log message requested
			static const FString LogStr("log");
			if (FString* Found = ConfigRuleVars.Find(LogStr))
			{
				UE_LOGF(LogConfigRules, Log, "ConfigRules log output:\n %ls", **Found);
				ConfigRuleVars.Remove(LogStr);
			}

			// check if requested to dump variables to the log
			static const FString DumpVarsStr("dumpvars");
			if (FString* Found = ConfigRuleVars.Find(DumpVarsStr))
			{
				ConfigRuleVars.Remove(DumpVarsStr);
				UE_LOGF(LogConfigRules, Log, "ConfigRules vars:");
				for (TPair<FString, FString> VarEntry : ConfigRuleVars)
				{
					UE_LOGF(LogConfigRules, Log, "%ls = %ls", *VarEntry.Key, *VarEntry.Value);
				}
			}

			// if there was a raised error or break, stop
			const bool HasError = ConfigRuleVars.Contains(TEXT("error"));
			const bool HasBreak = ConfigRuleVars.Contains(TEXT("break"));

			// stop if user wants to break
			if (HasBreak || HasError)
			{
				UE_LOGF(LogConfigRules, Warning, "Config rules aborting parse due to %ls.", HasBreak ? TEXT("break command") : TEXT("error"));
				bAbort = true;
			}
		});

	return ConfigRuleVars;
}

static FStringView ConfigRulesUtil::RemoveSurrounds(FStringView Input, FStringView Entry, FStringView Exit)
{
	if (Input.Len() >= 2 && Entry.Len() > 0 && Entry.Len() == Exit.Len())
	{
		FStringView Ret = UE::String::RemoveFromEnd(UE::String::RemoveFromStart(Input, Entry), Exit);
		if (Ret.Len() == (Input.Len() - (Entry.Len() * 2)))
		{
			return Ret;
		}
	}
	return Input;
}

static FStringView ConfigRulesUtil::SubStr(FStringView Input, int32 StartIdx, int32 EndIdx)
{
	return Input.Left(EndIdx).RightChop(StartIdx);
};

static int32 ConfigRulesUtil::IndexOf(const FString& Input, FStringView Str, int32 FromIndex /*= 0*/)
{
	return Input.Find(Str, ESearchCase::CaseSensitive, ESearchDir::FromStart, FromIndex);
}

// returns list of strings separated by Split using Entry/Exit as pairing sets (ex. "(" and ")"). The Entry and Exit characters need to be in same order
static TArray<FStringView> ConfigRulesUtil::ParseSegments(FStringView Input, FStringView Split, FStringView Entry, FStringView Exit)
{
	TArray<FStringView> Output;
	TArray<int32> EntryStack;

	int32 StartIndex = 0;
	int32 ScanIndex = 0;
	int32 ExitIndex = INDEX_NONE;
	int32 InputLength = Input.Len();

	while (ScanIndex < InputLength)
	{
		FStringView Scan = SubStr(Input, ScanIndex, ScanIndex + 1);

		if (Scan.Equals(Split) && EntryStack.Num() == 0)
		{
			Output.Add(SubStr(Input, StartIndex, ScanIndex).TrimStartAndEnd());
			ScanIndex++;
			StartIndex = ScanIndex;
			continue;
		}
		ScanIndex++;

		if (Scan.Equals(TEXT("\\")))
		{
			ScanIndex++;
			continue;
		}

		if (EntryStack.Num() > 0 && Exit.Find(Scan) == ExitIndex)
		{
			int32 StackLength = EntryStack.Num() - 1;
			EntryStack.RemoveAt(StackLength);
			ExitIndex = StackLength > 0 ? EntryStack[StackLength - 1] : INDEX_NONE;
			continue;
		}

		int32 EntryIndex = Entry.Find(Scan);
		if (EntryIndex >= 0)
		{
			EntryStack.Add(EntryIndex);
			ExitIndex = EntryIndex;
			continue;
		}
	}
	if (StartIndex < InputLength)
	{
		Output.Add(Input.RightChop(StartIndex).TrimStartAndEnd());
	}

	return Output;
}

static FString ConfigRulesUtil::ExpandVariables(TMap<FString, FString>& Variables, FStringView Input)
{
	FString Result(Input);
	int32 Idx;
	for (Idx = IndexOf(Result, TEXT("$(")); Idx != INDEX_NONE; Idx = IndexOf(Result, TEXT("$("), Idx))
	{
		// Find the end of the variable name
		int32 EndIdx = IndexOf(Result, TEXT(")"), Idx + 2);
		if (EndIdx == INDEX_NONE)
		{
			break;
		}

		// Extract the variable name from the string
		FStringView VarKey = SubStr(Result, Idx + 2, EndIdx);

		// Find the value for it if it exists
		if (FString* VarValue = Variables.Find(FString(VarKey)))
		{
			// Replace the variable
			Result = SubStr(Result, 0, Idx) + (*VarValue) + Result.RightChop(EndIdx + 1);
		}
		else
		{
			// or skip past it
			Idx = EndIdx + 1;
			continue;
		}

	}
	return Result;
}

static bool ConfigRulesUtil::EvaluateConditions(TMap<FString, FString>& Variables, TArray<FStringView>& Conditions)
{
	FString PrevRegEx;
	return EvaluateConditions(Variables, Conditions, PrevRegEx);
}

static bool ConfigRulesUtil::EvaluateConditions(TMap<FString, FString>& Variables, TArray<FStringView>& Conditions, FString& PreviousRegexMatch)
{
	bool bConditionTrue = true;
	for (FStringView Condition : Conditions)
	{
		FString SourceType;
		FString CompareType;
		FString MatchString;

		// deal with Condition group (src,cmp,match)
		TArray<FStringView> Groups = ParseSegments(RemoveSurrounds(Condition, TEXT("("), TEXT(")")), TEXT(","), TEXT("\""), TEXT("\""));
		for (FStringView Group : Groups)
		{
			TArray<FStringView> KeyValue = ParseSegments(Group, TEXT("="), TEXT("\""), TEXT("\""));
			if (KeyValue.Num() == 2)
			{
				FStringView Key = RemoveSurrounds(KeyValue[0], TEXT("\""), TEXT("\""));
				FStringView Value = RemoveSurrounds(KeyValue[1], TEXT("\""), TEXT("\""));

				if (Key.Equals(TEXT("SourceType")))
				{
					SourceType = Value;
				}
				else if (Key.Equals(TEXT("CompareType")))
				{
					CompareType = Value;
				}
				else if (Key.Equals(TEXT("MatchString")))
				{
					MatchString = Value;
				}
			}
		}

		FString Source;
		if (SourceType.Equals(TEXT("SRC_PreviousRegexMatch")))
		{
			Source = PreviousRegexMatch;
		}
		else if (SourceType.Equals(TEXT("SRC_CommandLine")))
		{
			checkNoEntry();
		}
		else if (FString* Found = Variables.Find(SourceType))
		{
			Source = *Found;
		}
		else if (SourceType.Equals(TEXT("[EXIST]")))
		{
			Source = MatchString;
		}
		else
		{
			bConditionTrue = false;
			break;
		}

		// apply operation
		if (CompareType.Equals(TEXT("CMP_Exist")))
		{
			if (!Variables.Contains(Source))
			{
				bConditionTrue = false;
				break;
			}
		}
		else if (CompareType.Equals(TEXT("CMP_NotExist")))
		{
			if (Variables.Contains(Source))
			{
				bConditionTrue = false;
				break;
			}
		}
		else if (CompareType.Equals(TEXT("CMP_Equal")))
		{
			if (!Source.Equals(MatchString))
			{
				bConditionTrue = false;
				break;
			}
		}
		else if (CompareType.Equals(TEXT("CMP_NotEqual")))
		{
			if (Source.Equals(MatchString))
			{
				bConditionTrue = false;
				break;
			}
		}
		else if (CompareType.Equals(TEXT("CMP_EqualIgnore")))
		{
			if (!Source.ToLower().Equals(MatchString.ToLower()))
			{
				bConditionTrue = false;
				break;
			}
		}
		else if (CompareType.Equals(TEXT("CMP_NotEqualIgnore")))
		{
			if (Source.ToLower().Equals(MatchString.ToLower()))
			{
				bConditionTrue = false;
				break;
			}
		}
		else if (CompareType.Equals(TEXT("CMP_Regex")))
		{
			const FRegexPattern RegexPattern(MatchString);
			FRegexMatcher RegexMatcher(RegexPattern, Source);

			if (RegexMatcher.FindNext())
			{
				FString Captured = RegexMatcher.GetCaptureGroup(1);
				PreviousRegexMatch = Captured.IsEmpty() ? RegexMatcher.GetCaptureGroup(0) : Captured;
			}
			else
			{
				bConditionTrue = false;
				break;
			}
		}
		else
		{
			bool bNumericOperands = true;
			float SourceFloat = 0.0f;
			float MatchFloat = 0.0f;

			// convert source and match to float if numeric
			bNumericOperands = LexTryParseString(SourceFloat, *Source);
			bNumericOperands = LexTryParseString(MatchFloat, *MatchString) && bNumericOperands;

			// if comparison ends with Ignore, do case-insensitive compare by converting both to lowercase
			if (CompareType.EndsWith(TEXT("Ignore")))
			{
				bNumericOperands = false;
				CompareType = SubStr(CompareType, 0, CompareType.Len() - 6);

				Source = Source.ToLower();
				MatchString = MatchString.ToLower();
			}

			if (CompareType.Equals(TEXT("CMP_Less")))
			{
				if ((bNumericOperands && (SourceFloat >= MatchFloat)) || (!bNumericOperands && (Source.Compare(MatchString) >= 0)))
				{
					bConditionTrue = false;
					break;
				}
			}
			else if (CompareType.Equals(TEXT("CMP_LessEqual")))
			{
				if ((bNumericOperands && (SourceFloat > MatchFloat)) || (!bNumericOperands && (Source.Compare(MatchString) > 0)))
				{
					bConditionTrue = false;
					break;
				}
			}
			else if (CompareType.Equals(TEXT("CMP_Greater")))
			{
				if ((bNumericOperands && (SourceFloat <= MatchFloat)) || (!bNumericOperands && (Source.Compare(MatchString) <= 0)))
				{
					bConditionTrue = false;
					break;
				}
			}
			else if (CompareType.Equals(TEXT("CMP_GreaterEqual")))
			{
				if ((bNumericOperands && (SourceFloat < MatchFloat)) || (!bNumericOperands && (Source.Compare(MatchString) < 0)))
				{
					bConditionTrue = false;
					break;
				}
			}
			else
			{
				bConditionTrue = false;
				break;
			}
		}
	}

	return bConditionTrue;
}
