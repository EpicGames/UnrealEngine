// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformApplicationMisc.h"
#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "Misc/OutputDeviceAnsiError.h"
#include "HAL/FeedbackContextAnsi.h"
#include "Math/Color.h"

#include "HAL/PlatformOutputDevices.h"
#include "HAL/PlatformApplicationMisc.h"

/** Hooks for moving ClipboardCopy and ClipboardPaste into FPlatformApplicationMisc */
CORE_API extern void (*ClipboardCopyShim)(const TCHAR* Text);
CORE_API extern void (*ClipboardPasteShim)(FString& Dest);

bool FGenericPlatformApplicationMisc::CachedPhysicalScreenData = false;
EScreenPhysicalAccuracy FGenericPlatformApplicationMisc::CachedPhysicalScreenAccuracy = EScreenPhysicalAccuracy::Unknown;
int32 FGenericPlatformApplicationMisc::CachedPhysicalScreenDensity = 0;

EScreenPhysicalAccuracy FGenericPlatformApplicationMisc::CachedPhysicalScreenDimensionsAccuracy = EScreenPhysicalAccuracy::Unknown;
FVector2f FGenericPlatformApplicationMisc::CachedPhysicalScreenDimensions = FVector2f::ZeroVector;
float FGenericPlatformApplicationMisc::CachedPhysicalScreenSize = 0.f;

static int32 bEnableHighDPIAwareness = 1;

FAutoConsoleVariableRef FGenericPlatformApplicationMisc::CVarEnableHighDPIAwareness(
	TEXT("EnableHighDPIAwareness"),
	bEnableHighDPIAwareness,
	TEXT("Enables or disables high dpi mode"),
	ECVF_ReadOnly
);

static bool bAllowVirtualKeyboard  = false;

FAutoConsoleVariableRef FGenericPlatformApplicationMisc::CVarAllowVirtualKeyboard(
	TEXT("AllowVirtualKeyboard"),
	bAllowVirtualKeyboard,
	TEXT("Allow the use of a virtual keyboard despite platform main screen being non-touch"),
	ECVF_Default
);

TAutoConsoleVariable<bool> FGenericPlatformApplicationMisc::CVarEnableNativeGestureRecognition(
	TEXT("Input.EnableNativeGestureRecognition"),
	false,
	TEXT("Use native platform gesture recognizers instead of the software gesture detection. Enabled via platform ini on supported platforms."),
	ECVF_Default
);

bool FGenericPlatformApplicationMisc::AreNativeGesturesEnabled()
{
	return CVarEnableNativeGestureRecognition.GetValueOnAnyThread();
}

void FGenericPlatformApplicationMisc::PreInit()
{
}

void FGenericPlatformApplicationMisc::Init()
{
	ClipboardCopyShim = &FPlatformApplicationMisc::ClipboardCopy;
	ClipboardPasteShim = &FPlatformApplicationMisc::ClipboardPaste;
}

void FGenericPlatformApplicationMisc::PostInit()
{
}

void FGenericPlatformApplicationMisc::TearDown()
{
	ClipboardCopyShim = nullptr;
	ClipboardPasteShim = nullptr;
}

class FOutputDeviceConsole* FGenericPlatformApplicationMisc::CreateConsoleOutputDevice()
{
	return nullptr; // normally only used for PC
}

class FOutputDeviceError* FGenericPlatformApplicationMisc::GetErrorOutputDevice()
{
	return FPlatformOutputDevices::GetError();
}

class FFeedbackContext* FGenericPlatformApplicationMisc::GetFeedbackContext()
{
	return FPlatformOutputDevices::GetFeedbackContext();
}

IPlatformInputDeviceMapper* FGenericPlatformApplicationMisc::CreatePlatformInputDeviceManager()
{
	return new FGenericPlatformInputDeviceMapper(/* bUsingControllerIdAsUserId = */ true, /* bShouldBroadcastLegacyDelegates = */ true);
}

GenericApplication* FGenericPlatformApplicationMisc::CreateApplication()
{
	return new GenericApplication( nullptr );
}

void FGenericPlatformApplicationMisc::RequestMinimize()
{
}

bool FGenericPlatformApplicationMisc::IsThisApplicationForeground()
{
	UE_LOGF(LogHAL, Fatal, "FGenericPlatformProcess::IsThisApplicationForeground not implemented on this platform");
	return false;
}

bool FGenericPlatformApplicationMisc::RequiresVirtualKeyboard()
{
	return PLATFORM_HAS_TOUCH_MAIN_SCREEN || bAllowVirtualKeyboard;
}

FLinearColor FGenericPlatformApplicationMisc::GetScreenPixelColor(const FVector2D& InScreenPos, float InGamma)
{ 
	return FLinearColor::Black;
}

bool FGenericPlatformApplicationMisc::IsHighDPIAwarenessEnabled()
{
	return bEnableHighDPIAwareness != 0;
}

void FGenericPlatformApplicationMisc::ClipboardCopy(const TCHAR* Str)
{

}
void FGenericPlatformApplicationMisc:: ClipboardPaste(class FString& Dest)
{
	Dest = FString();
}

void FGenericPlatformApplicationMisc::ComputePhysicalScreenData()
{
	CachedPhysicalScreenAccuracy = FPlatformApplicationMisc::ComputePhysicalScreenDensity(CachedPhysicalScreenDensity);
	if (CachedPhysicalScreenAccuracy == EScreenPhysicalAccuracy::Unknown)
	{
		// If the screen density is unknown we use 96, which is the default on Windows,
		// but it's also the assumed DPI scale of Slate internally for fonts.
		CachedPhysicalScreenDensity = 96;
	}

	CachedPhysicalScreenDimensionsAccuracy = FPlatformApplicationMisc::ComputePhysicalScreenDimensions(CachedPhysicalScreenDimensions, CachedPhysicalScreenSize);

	CachedPhysicalScreenData = true;
}

EScreenPhysicalAccuracy FGenericPlatformApplicationMisc::ComputePhysicalScreenDensity(int32& ScreenDensity)
{
	ScreenDensity = 0;
	return EScreenPhysicalAccuracy::Unknown;
}

EScreenPhysicalAccuracy FGenericPlatformApplicationMisc::ComputePhysicalScreenDimensions(FVector2f& OutScreenDimensions, float& OutScreenSize)
{
	OutScreenDimensions = FVector2f::ZeroVector;
	OutScreenSize = 0.f;
	return EScreenPhysicalAccuracy::Unknown;
}

EScreenPhysicalAccuracy FGenericPlatformApplicationMisc::GetPhysicalScreenDensity(int32& ScreenDensity)
{
	if (!CachedPhysicalScreenData)
	{
		ComputePhysicalScreenData();
	}

	ScreenDensity = CachedPhysicalScreenDensity;
	return CachedPhysicalScreenAccuracy;
}

EScreenPhysicalAccuracy FGenericPlatformApplicationMisc::GetPhysicalScreenDimensions(FVector2f& OutScreenDimensions)
{
	if (!CachedPhysicalScreenData)
	{
		ComputePhysicalScreenData();
	}

	OutScreenDimensions = CachedPhysicalScreenDimensions;
	return CachedPhysicalScreenDimensionsAccuracy;
}

EScreenPhysicalAccuracy FGenericPlatformApplicationMisc::GetPhysicalScreenSize(float& OutScreenSize)
{
	if (!CachedPhysicalScreenData)
	{
		ComputePhysicalScreenData();
	}

	OutScreenSize = CachedPhysicalScreenSize;
	return CachedPhysicalScreenDimensionsAccuracy;
}
