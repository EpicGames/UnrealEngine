// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformFramePacer.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/IConsoleManager.h"

int32 FGenericPlatformRHIFramePacer::GetFramePaceFromSyncInterval()
{
	int32 SyncInterval = 0;
	static IConsoleVariable* SyncIntervalCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("rhi.SyncInterval"));
	if (ensure(SyncIntervalCVar != nullptr))
	{
		SyncInterval = SyncIntervalCVar->GetInt();
	}

	if (SyncInterval <= 0)
		return 0;

	return FPlatformMisc::GetMaxRefreshRate() / FMath::Clamp(SyncInterval, 1, FPlatformMisc::GetMaxSyncInterval());
}

bool FGenericPlatformRHIFramePacer::SupportsFramePace(int32 QueryFramePace)
{
	if (QueryFramePace < 0)
		return false;

	if (QueryFramePace == 0)
		return true; // No Vsync

	if (FPlatformMisc::GetMaxRefreshRate() % QueryFramePace != 0)
		return false; // Must be a multiple

	int32 TargetSyncInterval = FPlatformMisc::GetMaxRefreshRate() / QueryFramePace;
	return TargetSyncInterval <= FPlatformMisc::GetMaxSyncInterval();
}

int32 FGenericPlatformRHIFramePacer::SetFramePaceToSyncInterval(int32 InFramePace)
{
	static IConsoleVariable* SyncIntervalCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("rhi.SyncInterval"));
	if (SupportsFramePace(InFramePace) && ensure(SyncIntervalCVar != nullptr))
	{
		int32 NewSyncInterval = InFramePace > 0
			? FMath::Clamp(FPlatformMisc::GetMaxRefreshRate() / InFramePace, 1, FPlatformMisc::GetMaxSyncInterval())
			: 0;

		SyncIntervalCVar->Set(NewSyncInterval, ECVF_SetByCode);
	}

	return GetFramePace();
}
