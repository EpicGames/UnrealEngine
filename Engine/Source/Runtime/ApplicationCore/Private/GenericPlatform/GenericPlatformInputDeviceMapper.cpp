// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ConfigCacheIni.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "GenericPlatform/InputDeviceMappingPolicy.h"
#include "GenericPlatform/InputDeviceRegistry.h"

///////////////////////////////////////////////////////////////////////////
// IPlatformInputDeviceMapper

DEFINE_LOG_CATEGORY(LogInputDeviceMapper);

IPlatformInputDeviceMapper::FOnInputDeviceHardwareInfoAvailable IPlatformInputDeviceMapper::OnInputDeviceHardwareInfoAvailable;
IPlatformInputDeviceMapper::FOnUserInputDeviceConnectionChange IPlatformInputDeviceMapper::OnInputDeviceConnectionChange;
IPlatformInputDeviceMapper::FOnUserInputDevicePairingChange IPlatformInputDeviceMapper::OnInputDevicePairingChange;

namespace UE::Input
{
	// The cached value of MaxPlatformUserCount from the input config.
	static int32 CachedMaxUserCount = -1;

	static int32 GetCachedMaxUserCount()
	{
		// If we are uninitialized, read from the platform config
		if (CachedMaxUserCount == -1)
		{
			int32 ConfigInt = -1;
			const FString InputSettingsSection = FString::Printf(TEXT("InputPlatformSettings_%s InputPlatformSettings"), ANSI_TO_TCHAR(FPlatformProperties::IniPlatformName()));
			static const TCHAR* DeviceMappingPolicyName = TEXT("MaxPlatformUserCount");

			if (GConfig->GetInt(*InputSettingsSection, DeviceMappingPolicyName, OUT ConfigInt, GInputIni))
			{
				CachedMaxUserCount = ConfigInt;
			}
			else
			{
				// Require a MaxPlatformUserCount setting to exist in some Input.ini.
				// Default to 8 as a reasonably safe fallback.
				ensureAlwaysMsgf(false, TEXT("Unable to find MaxPlatformUserCount from config, a max of 8 will be used."));
				CachedMaxUserCount = 8;
			}
		}
		
		return CachedMaxUserCount;
	}

	/** Cache the input device policy from the config so we don't need to read from the config cache as often */
	static EInputDeviceMappingPolicy CachedDevicePolicy = EInputDeviceMappingPolicy::Invalid;
	
	/**
	* Returns the currently set mapping policy for the current platform.
	* This is set by UInputSettings::DeviceMappingPolicy.
	*/
	static EInputDeviceMappingPolicy GetDeviceMappingPolicyFromConfig()
	{
		if (CachedDevicePolicy == EInputDeviceMappingPolicy::Invalid)
		{
			int32 PolicyInt = -1;
			const FString InputSettingsSection = FString::Printf(TEXT("InputPlatformSettings_%s InputPlatformSettings"), ANSI_TO_TCHAR(FPlatformProperties::IniPlatformName()));
			static const TCHAR* DeviceMappingPolicyName = TEXT("input.DeviceMappingPolicy");
			
			if (GConfig->GetInt(*InputSettingsSection, DeviceMappingPolicyName, OUT PolicyInt, GInputIni))
			{
				CachedDevicePolicy = static_cast<EInputDeviceMappingPolicy>(PolicyInt);
			}
			else
			{
				// Require a input.DeviceMappingPolicy setting to exist in some Input.ini
#if PLATFORM_DESKTOP
				ensureMsgf(false, TEXT("Unable to find an input device mapping policy, PrimaryUserSharesKeyboardAndFirstGamepad will be used."));
				CachedDevicePolicy = EInputDeviceMappingPolicy::PrimaryUserSharesKeyboardAndFirstGamepad;
#else
				ensureMsgf(false, TEXT("Unable to find an input device mapping policy, UseManagedPlatformLogin will be used."));
				CachedDevicePolicy = EInputDeviceMappingPolicy::UseManagedPlatformLogin;
#endif
			}	
		}
		
		return CachedDevicePolicy;
	}
};

FString LexToString(const EInputDeviceMappingPolicy Policy)
{
	switch (Policy)
	{
	case EInputDeviceMappingPolicy::Invalid: return TEXT("Invalid");
	case EInputDeviceMappingPolicy::UseManagedPlatformLogin: return TEXT("UseManagedPlatformLogin");
	case EInputDeviceMappingPolicy::PrimaryUserSharesKeyboardAndFirstGamepad: return TEXT("PrimaryUserSharesKeyboardAndFirstGamepad");
	case EInputDeviceMappingPolicy::CreateUniquePlatformUserForEachDevice: return TEXT("CreateUniquePlatformUserForEachDevice");
	case EInputDeviceMappingPolicy::MapAllDevicesToPrimaryUser: return TEXT("MapAllDevicesToPrimaryUser");
	}

	return TEXT("Unknown EInputDeviceMappingPolicy");
}

IPlatformInputDeviceMapper& IPlatformInputDeviceMapper::Get()
{
	static IPlatformInputDeviceMapper* StaticManager = nullptr;
	if (!StaticManager)
	{
		StaticManager = FPlatformApplicationMisc::CreatePlatformInputDeviceManager();
		check(StaticManager);
	}

	return *StaticManager;
}

IPlatformInputDeviceMapper::IPlatformInputDeviceMapper()
{
	BindCoreDelegates();
}

IPlatformInputDeviceMapper::~IPlatformInputDeviceMapper()
{
	UnbindCoreDelegates();
}

int32 IPlatformInputDeviceMapper::GetAllInputDevicesForUser(const FPlatformUserId UserId, TArray<FInputDeviceId>& OutInputDevices) const
{
	OutInputDevices.Reset();
	
	for (TPair<FInputDeviceId, FPlatformInputDeviceState> MappedDevice : MappedInputDevices)
	{
		if (MappedDevice.Value.OwningPlatformUser == UserId)
		{
			OutInputDevices.AddUnique(MappedDevice.Key);
		}
	}
	
	return OutInputDevices.Num();
}

int32 IPlatformInputDeviceMapper::GetAllInputDevices(TArray<FInputDeviceId>& OutInputDevices) const
{
	return MappedInputDevices.GetKeys(OutInputDevices);
}

int32 IPlatformInputDeviceMapper::GetAllConnectedInputDevices(TArray<FInputDeviceId>& OutInputDevices) const
{
	OutInputDevices.Reset();
	
	for (TPair<FInputDeviceId, FPlatformInputDeviceState> MappedDevice : MappedInputDevices)
	{
		if(MappedDevice.Value.ConnectionState == EInputDeviceConnectionState::Connected)
		{
			OutInputDevices.AddUnique(MappedDevice.Key);
		}
	}
	
	return OutInputDevices.Num();
}

int32 IPlatformInputDeviceMapper::GetAllConnectedInputDevicesForUser(const FPlatformUserId UserId, TArray<FInputDeviceId>& OutInputDevices) const
{
	OutInputDevices.Reset();

	for (TPair<FInputDeviceId, FPlatformInputDeviceState> MappedDevice : MappedInputDevices)
	{
		if (MappedDevice.Value.OwningPlatformUser == UserId && MappedDevice.Value.ConnectionState == EInputDeviceConnectionState::Connected)
		{
			OutInputDevices.AddUnique(MappedDevice.Key);
		}
	}

	return OutInputDevices.Num();
}

int32 IPlatformInputDeviceMapper::GetAllActiveUsers(TArray<FPlatformUserId>& OutUsers) const
{
	OutUsers.Reset();

	// Add the owning platform user for each input device
	for (TPair<FInputDeviceId, FPlatformInputDeviceState> MappedDevice : MappedInputDevices)
	{
		if (MappedDevice.Value.OwningPlatformUser.IsValid())
		{
			OutUsers.AddUnique(MappedDevice.Value.OwningPlatformUser);
		}
	}
	
	return OutUsers.Num();
}

FPlatformUserId IPlatformInputDeviceMapper::GetFirstPlatformUserWithNoInputDevice() const
{
	const FPlatformUserId UnpairedUser = GetUserForUnpairedInputDevices();
	
	for (const FPlatformUserId ExistingUser : AllocatedPlatformUserIds)
	{
		// Skip the unpaired user, they can't have devices mapped to them.
		if (ExistingUser == UnpairedUser)
		{
			continue;
		}
		
		// If this use has no input device's mapped them, return that
		const FInputDeviceId ExistingDevice = GetPrimaryInputDeviceForUser(ExistingUser);
		if (!ExistingDevice.IsValid())
		{
			return ExistingUser;
		}
	}

	// Nothing found
	return PLATFORMUSERID_NONE;
}

bool IPlatformInputDeviceMapper::IsUnpairedUserId(const FPlatformUserId PlatformId) const
{
	return PlatformId == GetUserForUnpairedInputDevices();
}

bool IPlatformInputDeviceMapper::IsInputDeviceMappedToUnpairedUser(const FInputDeviceId InputDevice) const
{
	if (const FPlatformInputDeviceState* DeviceState = MappedInputDevices.Find(InputDevice))
	{
		return IsUnpairedUserId(DeviceState->OwningPlatformUser);
	}
	return false;
}

FPlatformUserId IPlatformInputDeviceMapper::GetPlatformUserForNewlyConnectedDevice(const int32 InUserId /*= -1*/)
{
	const EInputDeviceMappingPolicy Policy = UE::Input::GetDeviceMappingPolicyFromConfig();
	const FPlatformUserId PrimaryUser = GetPrimaryPlatformUser();
	
	// If the policy is to always map to the primary user, then do so.
	if (Policy == EInputDeviceMappingPolicy::MapAllDevicesToPrimaryUser)
	{
		return PrimaryUser;
	}
	// If the primary user is supposed to share the first gamepad and keyboard,
	// then we will map this device to it if it only has one device connected to it. 
	else if (Policy == EInputDeviceMappingPolicy::PrimaryUserSharesKeyboardAndFirstGamepad)
	{
		TArray<FInputDeviceId> PrimaryUserDevices;
		GetAllInputDevicesForUser(PrimaryUser, OUT PrimaryUserDevices);

		if (PrimaryUserDevices.Num() <= 1)
		{
			return PrimaryUser;
		}
	}
	// If you have given an optional UserID and this platform is using controller ID as it's user id, then
	// we can return the platform user associated with that user.
	else if (InUserId != -1 && IsUsingControllerIdAsUserId())
	{
		return GetPlatformUserForUserIndex(InUserId);
	}

	// Return the first valid platform user which does not have any input devices mapped to it
	const FPlatformUserId ValidUserWithNoDevices = GetFirstPlatformUserWithNoInputDevice();
	if (ValidUserWithNoDevices.IsValid())
	{
		return ValidUserWithNoDevices;
	}

	// If all current platform users have a valid input device mapped to them, then we want to create a new one!
	return AllocateNewUserId();
}

FPlatformUserId IPlatformInputDeviceMapper::GetUserForInputDevice(FInputDeviceId DeviceId) const
{
	if (const FPlatformInputDeviceState* FoundState = MappedInputDevices.Find(DeviceId))
	{
		return FoundState->OwningPlatformUser;
	}
	return PLATFORMUSERID_NONE;
}

FInputDeviceId IPlatformInputDeviceMapper::GetPrimaryInputDeviceForUser(FPlatformUserId UserId) const
{
	FInputDeviceId FoundDevice = INPUTDEVICEID_NONE;

	// By default look for the lowest input device mapped to this user
	for (const TPair<FInputDeviceId, FPlatformInputDeviceState>& DeviceMapping : MappedInputDevices)
	{
		if (DeviceMapping.Value.OwningPlatformUser == UserId)
		{
			if (FoundDevice == INPUTDEVICEID_NONE || DeviceMapping.Key < FoundDevice)
			{
				FoundDevice = DeviceMapping.Key;
			}
		}
	}

	return FoundDevice;
}

bool IPlatformInputDeviceMapper::Internal_SetInputDeviceConnectionState(FInputDeviceId DeviceId, EInputDeviceConnectionState NewState)
{
	if (!DeviceId.IsValid())
	{
		UE_LOGF(LogInputDeviceMapper, Error, "IPlatformInputDeviceMapper::Internal_SetInputDeviceConnectionState was called with an invalid DeviceId of '%d'", DeviceId.GetId());
		return false;
	}

	// If the connection state hasn't changed, then there is no point to calling the map function below
	if (GetInputDeviceConnectionState(DeviceId) == NewState)
	{
		return false;
	}

	// Determine the owning user for this input device
	FPlatformUserId OwningUser = GetUserForInputDevice(DeviceId);

	// If the user is invalid, then fallback to being "Unpaired" user on this platform (which may still be PLATFORMUSERID_NONE)
	if (!OwningUser.IsValid())
	{
		OwningUser = GetUserForUnpairedInputDevices();
	}

	// Mapping the input device to the user will ensure that it is correctly mapped to the given user.
	// This covers the case where someone has called this function with a new input device that is not
	// yet mapped, as well as broadcasting the delegates we want.
	return Internal_MapInputDeviceToUser(DeviceId, OwningUser, NewState);
}

EInputDeviceConnectionState IPlatformInputDeviceMapper::GetInputDeviceConnectionState(const FInputDeviceId DeviceId) const
{
	EInputDeviceConnectionState State = EInputDeviceConnectionState::Unknown;

	if (!DeviceId.IsValid())
	{
		State = EInputDeviceConnectionState::Invalid;
	}
	else if (const FPlatformInputDeviceState* MappedDeviceState = MappedInputDevices.Find(DeviceId))
	{
		State = MappedDeviceState->ConnectionState;
	}

	return State;
}

bool IPlatformInputDeviceMapper::Internal_MapInputDeviceToUser(FInputDeviceId DeviceId, FPlatformUserId UserId, EInputDeviceConnectionState ConnectionState)
{
	if (!DeviceId.IsValid())
	{
		UE_LOGF(LogInputDeviceMapper, Error, "IPlatformInputDeviceMapper::Internal_MapInputDeviceToUser was called with an invalid DeviceId of '%d'", DeviceId.GetId());
		return false;
	}

	// Some platforms could validate it had been allocated before, but we allocate on demand if needed
	{
		if (DeviceId > LastInputDeviceId)
		{
			LastInputDeviceId = DeviceId;
		}
		
		if (UserId > LastPlatformUserId)
		{
			LastPlatformUserId = UserId;
		}
	}
	
	// Store the connection state of the input device
	FPlatformInputDeviceState& InputDeviceState = MappedInputDevices.FindOrAdd(DeviceId);
	InputDeviceState.OwningPlatformUser = UserId;
	InputDeviceState.ConnectionState = ConnectionState;

#if 0	// todo_bh: Not sure if we actually need this here since we have updated all platforms already
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Forward compatibility for FInputDeviceScope with the newer FInputDeviceRegistry system. 
	// If we have an input device scope, and there is not yet a device descriptor set for this device,
	// then populate it with the FInputDeviceScope data if possible.
	if (IsInGameThread())
	{
		if(!FInputDeviceRegistry::FindDescriptor(DeviceId).IsSet())
		{
			if (const FInputDeviceScope* Scope = FInputDeviceScope::GetCurrent())
			{
				UE_LOGF(LogInputDeviceMapper, Log, "Registering device based on the FInputDeviceScope %d", DeviceId.GetId());
			
				FInputDeviceRegistry::RegisterDevice(
					FInputDeviceDescriptor
					{
						.HardwareDeviceHandle     = DeviceId,
						.InputDeviceName          = Scope->InputDeviceName,
						.HardwareDeviceIdentifier = FName(*Scope->HardwareDeviceIdentifier),
					});
			}
		}	
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	// If there is a valid FInputDeviceScope at the time of connection, or the device registry is aware of this device,
	// then the hardware info is available.
	if (FInputDeviceRegistry::FindDescriptor(DeviceId).IsSet())
	{
		OnInputDeviceHardwareInfoAvailable.Broadcast(ConnectionState, UserId, DeviceId);	
	}
	
	// Broadcast delegates to let listeners know that the platform user has had an input device change
	OnInputDeviceConnectionChange.Broadcast(ConnectionState, UserId, DeviceId);
	
	return true;
}

bool IPlatformInputDeviceMapper::Internal_ChangeInputDeviceUserMapping(FInputDeviceId DeviceId, FPlatformUserId NewUserId, FPlatformUserId OldUserId)
{
	if (!DeviceId.IsValid())
	{
		UE_LOGF(LogInputDeviceMapper, Error, "IPlatformInputDeviceMapper::Internal_ChangeInputDeviceUserMapping was called with an invalid DeviceId of '%d'", DeviceId.GetId());
		return false;
	}

	if (NewUserId == OldUserId)
	{
		UE_LOGF(LogInputDeviceMapper, Log, "[%s] DeviceId of '%d' is already mapped to platform user '%d'.", __func__, DeviceId.GetId(), OldUserId.GetInternalId());
		return false;
	}

	// Update the existing device state to be the new owning platform user
	if (FPlatformInputDeviceState* ExistingDeviceState = MappedInputDevices.Find(DeviceId))
	{
		// Only change the platform user of this device if the old user matches up with the one that was given
		if (ExistingDeviceState->OwningPlatformUser == OldUserId)
		{
			ExistingDeviceState->OwningPlatformUser = NewUserId;
		}
	}
	else
	{
		UE_LOGF(LogInputDeviceMapper, Error, "IPlatformInputDeviceMapper::Internal_ChangeInputDeviceUserMapping: DeviceID '%d' is not mapped! Call Internal_MapInputDeviceToUser to map it to a user first!", DeviceId.GetId());
		return false;
	}

	// Broadcast the delegates letting listeners know that the input device has changed owners
	OnInputDevicePairingChange.Broadcast(DeviceId, NewUserId, OldUserId);
	
	return true;
}


#if WITH_EDITOR
void IPlatformInputDeviceMapper::HandleInputDevicePolicyChanged(const EInputDeviceMappingPolicy NewPolicy)
{
	if (NewPolicy == UE::Input::CachedDevicePolicy)
	{
		UE_LOGF(LogInputDeviceMapper, Log, "[%s] Device Mapping Policy is already set to '%ls'...",
			__func__,
			*LexToString(UE::Input::CachedDevicePolicy));
		
		return;
	}

	UE_LOGF(LogInputDeviceMapper, Log, "[%s] Changing device mapping policy from '%ls' to '%ls'...",
		__func__,
		*LexToString(UE::Input::CachedDevicePolicy),
		*LexToString(NewPolicy));
	
	const FPlatformUserId PrimaryUser = GetPrimaryPlatformUser();
	const FPlatformUserId UnpairedUser = GetUserForUnpairedInputDevices();

	TArray<FInputDeviceId> AllConnectedDevices;
	GetAllConnectedInputDevices(OUT AllConnectedDevices);
		
	switch(NewPolicy)
	{
	case EInputDeviceMappingPolicy::UseManagedPlatformLogin:
		// Since this would be handled by the platform... no need to do anything
		break;
	case EInputDeviceMappingPolicy::PrimaryUserSharesKeyboardAndFirstGamepad:
		{
			// Start off by remapping all input devices to the invalid platform user id
			// except for the primary device
			for (FInputDeviceId Device : AllConnectedDevices)
			{
				if (Device == GetDefaultInputDevice())
				{
					continue;
				}
				
				const FPlatformUserId OldUserId = GetUserForInputDevice(Device);
				Internal_ChangeInputDeviceUserMapping(Device, UnpairedUser, OldUserId);
			}
		
			TArray<FInputDeviceId> PrimaryUserDevices;
			
			// Ensure that the primary user has the primary input devices (keyboard)
			// and a gamepad assigned to them
			while (!AllConnectedDevices.IsEmpty())
			{
				const FInputDeviceId Device = AllConnectedDevices.Pop();
				
				const FPlatformUserId OldUserId = GetUserForInputDevice(Device);
				
				GetAllInputDevicesForUser(PrimaryUser, OUT PrimaryUserDevices);

				// If the primary user only has 1 device then map this gamepad to it
				// We also want to ensure that the default input device is always mapped to the primary user
				if (PrimaryUserDevices.Num() <= 1 || Device == GetDefaultInputDevice())
				{
					Internal_ChangeInputDeviceUserMapping(Device, PrimaryUser, OldUserId);
					continue;
				}

				// Otherwise, map to the next available platform user or a new one if necessary
				FPlatformUserId NextUser = GetFirstPlatformUserWithNoInputDevice();
				if (!NextUser.IsValid())
				{
					NextUser = AllocateNewUserId();
				}

				Internal_ChangeInputDeviceUserMapping(Device, NextUser, OldUserId);
			}
		}
		break;
	case EInputDeviceMappingPolicy::CreateUniquePlatformUserForEachDevice:
		{
			// Start off by remapping all input devices to the invalid platform user id
			// except for the primary device
			for (FInputDeviceId Device : AllConnectedDevices)
			{
				if (Device == GetDefaultInputDevice())
				{
					continue;
				}
				
				const FPlatformUserId OldUserId = GetUserForInputDevice(Device);
				Internal_ChangeInputDeviceUserMapping(Device, UnpairedUser, OldUserId);
			}

			while (!AllConnectedDevices.IsEmpty())
			{
				const FInputDeviceId Device = AllConnectedDevices.Pop();

				// Map to the next available platform user ID, or create a new one if necessary
				FPlatformUserId NextUser = GetFirstPlatformUserWithNoInputDevice();
				if (!NextUser.IsValid())
				{
					NextUser = AllocateNewUserId();
				}

				Internal_ChangeInputDeviceUserMapping(Device, NextUser, PLATFORMUSERID_NONE);
			}
		}
		break;
	case EInputDeviceMappingPolicy::MapAllDevicesToPrimaryUser:
	{
		// Map all connected input devices to the primary platform user
		for (const FInputDeviceId Device : AllConnectedDevices)
		{
			const FPlatformUserId OldUserId = GetUserForInputDevice(Device);
			Internal_ChangeInputDeviceUserMapping(Device, PrimaryUser, OldUserId);
		}
	}
	break;
	case EInputDeviceMappingPolicy::Invalid:
	default:
		checkNoEntry();
		break;
	}

	// Update the cached value to use this new one
	UE::Input::CachedDevicePolicy = NewPolicy;
}
#endif	// #if WITH_EDITOR

int32 IPlatformInputDeviceMapper::GetMaxPlatformUserCount() const
{
	// By default, return the cached value from the input config.
	return UE::Input::GetCachedMaxUserCount();
}

void IPlatformInputDeviceMapper::BindCoreDelegates()
{
	FCoreDelegates::OnUserLoginChangedEvent.AddRaw(this, &IPlatformInputDeviceMapper::OnUserLoginChangedEvent);
}

void IPlatformInputDeviceMapper::UnbindCoreDelegates()
{
	FCoreDelegates::OnUserLoginChangedEvent.RemoveAll(this);
}

bool IPlatformInputDeviceMapper::ShouldCreateUniqueUserForEachDevice() const
{
	const EInputDeviceMappingPolicy Policy = UE::Input::GetDeviceMappingPolicyFromConfig();

	return 
		Policy == EInputDeviceMappingPolicy::CreateUniquePlatformUserForEachDevice || 
		Policy == EInputDeviceMappingPolicy::UseManagedPlatformLogin;
}

const EInputDeviceMappingPolicy IPlatformInputDeviceMapper::GetCurrentDeviceMappingPolicy() const
{
	return UE::Input::GetDeviceMappingPolicyFromConfig();
}

///////////////////////////////////////////////////////////////////////////
// FGenericPlatformInputDeviceMapper

FGenericPlatformInputDeviceMapper::FGenericPlatformInputDeviceMapper(const bool InbUsingControllerIdAsUserId, const bool InbShouldBroadcastLegacyDelegates)
	: IPlatformInputDeviceMapper()
	, bUsingControllerIdAsUserId(InbUsingControllerIdAsUserId)
	, bShouldBroadcastLegacyDelegates(InbShouldBroadcastLegacyDelegates)
{
	// Set the last input device id to be the default of 0, that way any new devices will have
	// an index of 1 or higher and we can use the Default Input Device as a fallback for any
	// unpaired input devices without an owning PlatformUserId
	LastInputDeviceId = GetDefaultInputDevice();
	LastPlatformUserId = GetPrimaryPlatformUser();

	// By default map the Default Input device to the Primary platform user in a connected state. This ensures that the SlateApplication has a 
	// "Default" user to deal with representing the keyboard and mouse
	Internal_MapInputDeviceToUser(GetDefaultInputDevice(), GetPrimaryPlatformUser(), EInputDeviceConnectionState::Connected);
	
	// Keep track of any allocated platform users so that we can utilize them when remapping input devices
	AllocatedPlatformUserIds.AddUnique(GetPrimaryPlatformUser());
}

FPlatformUserId FGenericPlatformInputDeviceMapper::GetUserForUnpairedInputDevices() const
{
	// Not supported by default. If a platform wanted to support this, then it is recommended that
	// you create a static const FPlatformUserID with a value of 0 to start out as the "Unpaired"
	// user that input devices can then map to
	return PLATFORMUSERID_NONE;
}

FPlatformUserId FGenericPlatformInputDeviceMapper::GetPrimaryPlatformUser() const
{
	// Most platforms will want the primary user to be 0
	static const FPlatformUserId PrimaryPlatformUser = FPlatformUserId::CreateFromInternalId(0);
	return PrimaryPlatformUser;
}

FInputDeviceId FGenericPlatformInputDeviceMapper::GetDefaultInputDevice() const
{
	static const FInputDeviceId DefaultInputDeviceId = FInputDeviceId::CreateFromInternalId(0);
	return DefaultInputDeviceId;
}

bool FGenericPlatformInputDeviceMapper::RemapControllerIdToPlatformUserAndDevice(int32 ControllerId, FPlatformUserId& InOutUserId, FInputDeviceId& OutInputDeviceId)
{
	if (IsUsingControllerIdAsUserId())
	{
		if (InOutUserId.GetInternalId() >= 0 && ControllerId >= 0 && InOutUserId.GetInternalId() != ControllerId)
		{
			// Both are valid so use them
			OutInputDeviceId = FInputDeviceId::CreateFromInternalId(ControllerId);
			return true;
		}
		else if (ControllerId >= 0)
		{
			//if (!OutInputDeviceId.IsValid())
			{
				// Just use controller id, and copy over device
				OutInputDeviceId = FInputDeviceId::CreateFromInternalId(ControllerId);	
			}
			
			// If we were already given a valid platform user then we can just stop now.
			// This will be the case on platforms with an existing concept of "User Logins" from platform itself
			if (InOutUserId.IsValid())
			{
				return true;
			}

			// If it wasn't valid, then check for a valid known existing user to use
			FPlatformUserId ExistingUser = GetUserForInputDevice(OutInputDeviceId);
			if (ExistingUser.IsValid())
			{
				InOutUserId = ExistingUser;
				return true;
			}

			// Otherwise this is a fresh input device, and we need to create a new PlatformUserId for it.
			// Some platforms do not have the concept of "User ID"s (i.e. platforms that don't allow having multiple users logged in at once)
			// For those platforms, they may want to create a new platform user ID for each additional input device that is connected. This would
			// allow them to create the facade that there is a separation between the connected input devices and their platform users, allowing
			// gameplay code to differentiate between platform users in a consistent manner.
			if (ShouldCreateUniqueUserForEachDevice())
			{
				InOutUserId = AllocateNewUserId();
			}
			else
			{
				// Otherwise just have a 1:1 mapping of input device to user id's
				InOutUserId = FPlatformUserId::CreateFromInternalId(ControllerId);
				
				AllocatedPlatformUserIds.AddUnique(InOutUserId);
			}
			return true;
		}
		else if (InOutUserId.GetInternalId() >= 0)
		{
			// Ignore controller id
			OutInputDeviceId = FInputDeviceId::CreateFromInternalId(InOutUserId.GetInternalId());
			return true;
		}
	}
	
	return false;
}

FPlatformUserId FGenericPlatformInputDeviceMapper::GetPlatformUserForUserIndex(int32 LocalUserIndex)
{
	// The platform user index is equivalent to ControllerId in most legacy code
	if (IsUsingControllerIdAsUserId())
	{
		return FPlatformUserId::CreateFromInternalId(LocalUserIndex);
	}

	checkNoEntry();
	return PLATFORMUSERID_NONE;
}

bool FGenericPlatformInputDeviceMapper::RemapUserAndDeviceToControllerId(FPlatformUserId UserId, int32& OutControllerId, FInputDeviceId OptionalDeviceId /* = INPUTDEVICEID_NONE */)
{
	// It's just a 1:1 mapping of the old ControllerId to PlatformId if this is true
	if (IsUsingControllerIdAsUserId())
	{
		OutControllerId = UserId;
		return true;
	}
	return false;
}

int32 FGenericPlatformInputDeviceMapper::GetUserIndexForPlatformUser(FPlatformUserId UserId)
{
	// The platform user index is equivalent to ControllerId in most legacy code
	int32 OutControllerId = INDEX_NONE;
	if (RemapUserAndDeviceToControllerId(UserId, OutControllerId))
	{
		return OutControllerId;
	}
	return INDEX_NONE;
}

bool FGenericPlatformInputDeviceMapper::IsUsingControllerIdAsUserId() const
{
	return bUsingControllerIdAsUserId;
}

bool FGenericPlatformInputDeviceMapper::ShouldBroadcastLegacyDelegates() const
{
	return bShouldBroadcastLegacyDelegates;
}

void FGenericPlatformInputDeviceMapper::OnUserLoginChangedEvent(bool bLoggedIn, int32 RawPlatformUserId, int32 UserIndex)
{
	// Attain the FPlatformUserId from the user index that will be given by the Platform code
	FPlatformUserId LoggedOutPlatformUserId = GetPlatformUserForUserIndex(UserIndex);
	
	// A new user has logged in
	if (bLoggedIn)
	{
		// TODO: As of right now there is no logic that needs to run when a new platform user logs in, but there may be in the future
	}
	// A platform user has logged out
	else if (bUnpairInputDevicesWhenLoggingOut)
	{
		// Remap any input devices that the logged out platform user had to the "Unpaired" user
		FPlatformUserId UnkownUserId = GetUserForUnpairedInputDevices();
		if (LoggedOutPlatformUserId != UnkownUserId)
		{
			TArray<FInputDeviceId> InputDevices;
			GetAllInputDevicesForUser(LoggedOutPlatformUserId, InputDevices);

			for (FInputDeviceId DeviceId : InputDevices)
			{
				Internal_ChangeInputDeviceUserMapping(DeviceId, UnkownUserId, LoggedOutPlatformUserId);
			}
		}
	}
}

FPlatformUserId FGenericPlatformInputDeviceMapper::AllocateNewUserId()
{
	// Create a new platform user ID that is 1 higher than the last one
	LastPlatformUserId = FPlatformUserId::CreateFromInternalId(LastPlatformUserId.GetInternalId() + 1);

	AllocatedPlatformUserIds.AddUnique(LastPlatformUserId);

	UE_LOGF(LogInputDeviceMapper, Log, "[%s] Allocating a new PlatformUserId %d", __func__, LastPlatformUserId.GetInternalId());

	// We want to warn about this state happening, but not crash the application or anything. 
	// Each platform has a specific number of platform users which can be signed on at any given time
	// so going over that amount would cause undefined behavior such as your input no being routed correctly
	// to the active local player.
	ensureAlwaysMsgf(AllocatedPlatformUserIds.Num() <= GetMaxPlatformUserCount(), TEXT("Requested more then the max number of supported platform users! Undefined behavior may occur."));
	
	return LastPlatformUserId;
}

FInputDeviceId FGenericPlatformInputDeviceMapper::AllocateNewInputDeviceId()
{
	// Create a new platform user ID that is 1 higher than the last one
	LastInputDeviceId = FInputDeviceId::CreateFromInternalId(LastInputDeviceId.GetId() + 1);
	
	return LastInputDeviceId;
}