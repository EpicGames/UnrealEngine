// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformSoftwareCursor.h"
#include "GenericPlatform/GenericApplication.h"
#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "IOS/IOSSystemIncludes.h"
#endif

// Windows has special needs with RECT, and we don't need this class on Windows anyway, so just skip it
#if !PLATFORM_MICROSOFT

FGenericPlatformSoftwareCursor::FGenericPlatformSoftwareCursor() 
	: CurrentType(EMouseCursor::None)
	, CurrentPosition(FVector2D::ZeroVector)
	, CursorClipRect()
	, bShow(false)
{	
}

void FGenericPlatformSoftwareCursor::SetPosition( const int32 X, const int32 Y )
{
	FVector2D NewPosition(X, Y);
	UpdateCursorClipping(NewPosition);
	
	CurrentPosition = NewPosition;
}

void FGenericPlatformSoftwareCursor::SetType( const EMouseCursor::Type InNewCursor )
{
	CurrentType = InNewCursor;
}

void FGenericPlatformSoftwareCursor::GetSize(int32& Width, int32& Height) const
{
	Width = 32;
	Height = 32;
}

void FGenericPlatformSoftwareCursor::Show(bool bInShow)
{
	bShow = bInShow;
}

void FGenericPlatformSoftwareCursor::Lock(const RECT* const Bounds)
{
	if (Bounds == NULL)
	{
		FDisplayMetrics DisplayMetrics;
		FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);

		// The PS4 cursor should never leave the screen
		CursorClipRect.Min = FIntPoint::ZeroValue;
		CursorClipRect.Max.X = DisplayMetrics.PrimaryDisplayWidth - 1;
		CursorClipRect.Max.Y = DisplayMetrics.PrimaryDisplayHeight - 1;
	}
	else
	{
		CursorClipRect.Min.X = Bounds->left;
		CursorClipRect.Min.Y = Bounds->top;
		CursorClipRect.Max.X = Bounds->right - 1;
		CursorClipRect.Max.Y = Bounds->bottom - 1;
	}

	FVector2D Position = GetPosition();
	if (UpdateCursorClipping(Position))
	{
		SetPosition((int32)Position.X, (int32)Position.Y);
	}
}

bool FGenericPlatformSoftwareCursor::UpdateCursorClipping(FVector2D& CursorPosition)
{
	bool bAdjusted = false;

	if (CursorClipRect.Area() > 0)
	{
		if (CursorPosition.X < CursorClipRect.Min.X)
		{
			CursorPosition.X = CursorClipRect.Min.X;
			bAdjusted = true;
		}
		else if (CursorPosition.X > CursorClipRect.Max.X)
		{
			CursorPosition.X = CursorClipRect.Max.X;
			bAdjusted = true;
		}

		if (CursorPosition.Y < CursorClipRect.Min.Y)
		{
			CursorPosition.Y = CursorClipRect.Min.Y;
			bAdjusted = true;
		}
		else if (CursorPosition.Y > CursorClipRect.Max.Y)
		{
			CursorPosition.Y = CursorClipRect.Max.Y;
			bAdjusted = true;
		}
	}

	return bAdjusted;
}

#endif