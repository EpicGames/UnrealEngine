// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformSplash.h"

#include "Containers/UnrealString.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformSplash.h"
#include "Internationalization/Text.h"
#include "Misc/Paths.h"

// Supported file extension for splash image
const TCHAR* SupportedSplashImageExt[] = 
{
	TEXT(".png"),
	TEXT(".jpg"),
	nullptr
};


static FString GCustomSplashScreenFileName;

/**
* Return the filename found (look for PNG, JPG and BMP in that order, try to avoid BMP, use more space...)
*/
FString GetSplashFilename(const FString& ContentDir, const FString& Filename)
{
	int index = 0;
	const FString ImageName = ContentDir / Filename;
	FString Path;

	while (SupportedSplashImageExt[index])
	{
		Path = ImageName + SupportedSplashImageExt[index++];

		if (FPaths::FileExists(Path))
			return Path;
	}

	// if no image was found, we assume it's a BMP (default)
	return ImageName + TEXT(".bmp");
}

void FGenericPlatformSplash::SetCustomSplashImage(const TCHAR* SplashFilename)
{
	GCustomSplashScreenFileName = SplashFilename;
}

void FGenericPlatformSplash::SetSplashText(const SplashTextType::Type InType, const TCHAR* InText)
{
	FPlatformSplash::SetSplashText(InType, FText::AsCultureInvariant(InText));
}

/**
* Finds a usable splash pathname for the given filename
*
* @param SplashFilename Name of the desired splash name("Splash")
* @param IconFilename Name of the desired icon name("Splash")
* @param OutPath String containing the path to the file, if this function returns true
* @param OutIconPath String containing the path to the icon, if this function returns true
*
* @return true if a splash screen was found
*/
bool FGenericPlatformSplash::GetSplashPath(const TCHAR* SplashFilename, FString& OutPath, bool& OutIsCustom,
	const bool bAllowCustom)
{
	FString Filename = FString("Splash/") + SplashFilename;

	if(GCustomSplashScreenFileName.IsEmpty() || !bAllowCustom)
	{
		// let's check if maybe the incoming path is already a fully qualified file name
		if (FPaths::FileExists(SplashFilename))
		{
			OutPath = SplashFilename;
			OutIsCustom = false;

			// if this was found, then we're done
			return true;
		}

		// first look in game's splash directory
		OutPath = FPaths::ConvertRelativePathToFull(GetSplashFilename(FPaths::ProjectContentDir(), Filename));
		OutIsCustom = true;

		// if this was found, then we're done
		if (FPaths::FileExists(*OutPath))
		{
			return true;
		}

		// next look in Engine/Splash
		OutPath = FPaths::ConvertRelativePathToFull(GetSplashFilename(FPaths::EngineContentDir(), Filename));
		OutIsCustom = false;

		// if this was found, then we're done
		if (FPaths::FileExists(*OutPath))
		{
			return true;
		}

	}
	else
	{ 
		OutPath = GCustomSplashScreenFileName;
		OutIsCustom = true;
		// if this was found, then we're done
		if (FPaths::FileExists(*OutPath))
		{
			return true;
		}
	}


	// if not found yet, then return failure
	return false;
}

bool FGenericPlatformSplash::GetSplashPath(const TCHAR* SplashFilename, const TCHAR* IconFilename, FString& OutPath, FString& OutIconPath, bool& OutIsCustom)
{
	FString IconName = FString("Splash/") + IconFilename;

	// Get the path for the splash image
	bool bGetSplashSucceeded = GetSplashPath(SplashFilename, OutPath, OutIsCustom);

	// first look in game's splash directory
	OutIconPath = GetSplashFilename(FPaths::ProjectContentDir(), IconName);

	// if the icon was found, then we're done
	if (IFileManager::Get().FileSize(*OutIconPath) != -1)
	{
		return bGetSplashSucceeded;
	}

	// next look in Engine/Splash
	OutIconPath = GetSplashFilename(FPaths::EngineContentDir(), IconName);
	OutIsCustom = false;

	// if this was found, then we're done
	if (IFileManager::Get().FileSize(*OutIconPath) != -1)
	{
		return bGetSplashSucceeded;
	}

	// if not found yet, then return failure
	return false;
}
