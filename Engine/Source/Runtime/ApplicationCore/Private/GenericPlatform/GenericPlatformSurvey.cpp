// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformSurvey.h"
#include "Containers/Array.h"

DEFINE_LOG_CATEGORY(LogSynthBenchmark);

// 100: avg good CPU, <100:slower, >100:faster
float FSynthBenchmarkResults::ComputeCPUPerfIndex(TArray<float>* OutIndividualResults) const
{
	const uint32 NumStats = sizeof(CPUStats) / sizeof(CPUStats[0]);
	if (OutIndividualResults != nullptr)
	{
		OutIndividualResults->Empty();
	}

	float Ret = 0.0f;
	float TotalWeight = 0.0f;
	for (uint32 i = 0; i < NumStats; ++i)
	{
		TotalWeight += CPUStats[i].GetWeight();
	}

	for (uint32 i = 0; i < NumStats; ++i)
	{
		const float PerfIndex = CPUStats[i].ComputePerfIndex();
		if (OutIndividualResults != nullptr)
		{
			OutIndividualResults->Add(PerfIndex);
		}

		const float NormalizedWeight = (TotalWeight > 0.f) ? (CPUStats[i].GetWeight() / TotalWeight) : 0.f;
		Ret += PerfIndex * NormalizedWeight;
	}

	return Ret;
}

// 100: avg good GPU, <100:slower, >100:faster
float FSynthBenchmarkResults::ComputeGPUPerfIndex(TArray<float>* OutIndividualResults) const
{
	const uint32 NumStats = sizeof(GPUStats) / sizeof(GPUStats[0]);
	if (OutIndividualResults != nullptr)
	{
		OutIndividualResults->Empty(NumStats);
	}

	float Ret = 0.0f;
	float TotalWeight = 0.0f;
	for (uint32 i = 0; i < NumStats; ++i)
	{
		TotalWeight += GPUStats[i].GetWeight();
	}

	for (uint32 i = 0; i < NumStats; ++i)
	{
		const float PerfIndex = GPUStats[i].ComputePerfIndex();
		if (OutIndividualResults != nullptr)
		{
			OutIndividualResults->Add(PerfIndex);
		}

		const float NormalizedWeight = (TotalWeight > 0.f) ? (GPUStats[i].GetWeight() / TotalWeight) : 0.f;
		Ret += PerfIndex * NormalizedWeight;
	}

	return Ret;
}
