// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericWindow.h"
#include "Misc/App.h"

#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"

DEFINE_LOG_CATEGORY_STATIC(LogGenericPlatformWindow, Log, All);

FGenericWindow::FGenericWindow()
{

}

FGenericWindow::~FGenericWindow()
{

}

void FGenericWindow::ReshapeWindow( int32 X, int32 Y, int32 Width, int32 Height )
{
	// empty default functionality
}

bool FGenericWindow::GetFullScreenInfo( int32& X, int32& Y, int32& Width, int32& Height ) const
{
	if (!FParse::Param(FCommandLine::Get(), TEXT("RenderOffScreen")))
	{
		// this function cannot return valid results, and should not be needed on consoles, etc
		UE_LOGF(LogGenericPlatformWindow, Fatal, "GetFullScreenInfo is not expected to be called on this platform");
	}
	return false;
}

void FGenericWindow::MoveWindowTo ( int32 X, int32 Y )
{
	// empty default functionality
}

void FGenericWindow::BringToFront( bool bForce )
{
	// empty default functionality
}

void FGenericWindow::HACK_ForceToFront()
{
	// empty default functionality
}

void FGenericWindow::Destroy()
{
	// empty default functionality
}

void FGenericWindow::Minimize()
{
	// empty default functionality
}

void FGenericWindow::Maximize()
{
	// empty default functionality
}

void FGenericWindow::Restore()
{
	// empty default functionality
}

void FGenericWindow::Show()
{
	// empty default functionality
}

void FGenericWindow::Hide()
{
	// empty default functionality
}

void FGenericWindow::SetWindowMode( EWindowMode::Type InNewWindowMode )
{
	// empty default functionality
}

EWindowMode::Type FGenericWindow::GetWindowMode() const
{
	// default functionality
	return EWindowMode::Windowed;
}

bool FGenericWindow::IsMaximized() const
{
	// empty default functionality
	return true;
}

bool FGenericWindow::IsMinimized() const
{
	return false;
}


bool FGenericWindow::IsVisible() const
{
	// empty default functionality
	return true;
}

bool FGenericWindow::GetRestoredDimensions(int32& X, int32& Y, int32& Width, int32& Height)
{
	if (!FParse::Param(FCommandLine::Get(), TEXT("RenderOffScreen")) && FApp::CanEverRender())
	{
		// this function cannot return valid results, and should not be needed on consoles, etc
		UE_LOGF(LogGenericPlatformWindow, Fatal, "GetRestoredDimensions is not expected to be called on this platform");
	}
	return false;
}

void FGenericWindow::SetWindowFocus()
{
	// empty default functionality
}

void FGenericWindow::SetOpacity( const float InOpacity )
{
	// empty default functionality
}

void FGenericWindow::Enable( bool bEnable )
{
	// empty default functionality
}

bool FGenericWindow::IsPointInWindow( int32 X, int32 Y ) const
{
	// empty default functionality
	return true;
}
	
int32 FGenericWindow::GetWindowBorderSize() const
{
	// empty default functionality
	return 0;
}

int32 FGenericWindow::GetWindowTitleBarSize() const
{
	// empty default functionality
	return 0;
}

void* FGenericWindow::GetOSWindowHandle() const
{
	return nullptr;
}

bool FGenericWindow::IsForegroundWindow() const 
{
	// empty default functionality
	return true;
}

bool FGenericWindow::SupportsDynamicCursorLock() const
{
	return false;
}

bool FGenericWindow::IsFullscreenSupported() const
{
	// empty default functionality
	return true;
}

void FGenericWindow::SetText(const TCHAR* const Text)
{
	// empty default functionality
}

const FGenericWindowDefinition& FGenericWindow::GetDefinition() const
{
	return *Definition.Get();
}

bool FGenericWindow::IsDefinitionValid() const
{
	return Definition.IsValid();
}

void FGenericWindow::AdjustCachedSize( FVector2D& Size ) const
{
}

float FGenericWindow::GetDPIScaleFactor() const
{
	return 1.0f;
}

void FGenericWindow::SetDPIScaleFactor(const float Factor)
{
	// empty default functionality
}

bool FGenericWindow::IsManualManageDPIChanges() const
{
	// returns false by default so the application can auto-manage the size of its windows, in response to DPI variations
	return false;
}

void FGenericWindow::SetManualManageDPIChanges(const bool bAutoHandle)
{
	// empty default functionality
}

void FGenericWindow::DrawAttention(const FWindowDrawAttentionParameters& Parameters)
{
	// empty default functionality
}

void FGenericWindow::SetNativeWindowButtonsVisibility(bool bVisible)
{
	// empty default functionality
}

bool FGenericWindow::IsRenderingOffScreen() const
{
	return false;
}
