// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/InputDeviceRegistry.h"
#include "Logging/LogMacros.h"
#include "HAL/IConsoleManager.h"

DEFINE_LOG_CATEGORY_STATIC(LogInputDeviceRegistry, Log, All);

FInputDeviceRegistry& FInputDeviceRegistry::Get()
{
	static FInputDeviceRegistry Instance;
	return Instance;
}

bool FInputDeviceRegistry::RegisterDevice(const FInputDeviceDescriptor& Descriptor)
{
	if (!Descriptor.HardwareDeviceHandle.IsValid())
	{
		UE_LOGF(LogInputDeviceRegistry, Warning, "RegisterDevice failed: HardwareDeviceHandle is not valid. %ls", *Descriptor.ToString());
		return false;
	}

	FInputDeviceRegistry& Registry = Get();
	bool bDescriptorChanged = false;
	{
		FWriteScopeLock ScopeLock(Registry.Lock);

		// First registration of this DeviceId counts as a change. A re-registration is a change only
		// if any of the descriptor's fields differs from what was previously stored.
		const FInputDeviceDescriptor* PreviousDescriptor = Registry.Descriptors.Find(Descriptor.HardwareDeviceHandle);
		bDescriptorChanged = (PreviousDescriptor == nullptr) || (*PreviousDescriptor != Descriptor);

		UE_LOGF(LogInputDeviceRegistry, Log, "RegisterDevice: %ls (changed=%ls)",
			*Descriptor.ToString(),
			bDescriptorChanged ? TEXT("true") : TEXT("false"));

		Registry.Descriptors.Add(Descriptor.HardwareDeviceHandle, Descriptor);
	}

	// Broadcast outside the write lock so subscribers can safely call back into the registry without
	// risking a deadlock on the same thread. Only fire when the descriptor actually changed — a
	// re-registration with identical fields is a no-op for downstream consumers.
	if (bDescriptorChanged)
	{
		Registry.DescriptorChangedDelegate.Broadcast(Descriptor);
	}

	return true;
}

void FInputDeviceRegistry::RemoveDevice(const FInputDeviceId DeviceId)
{
	UE_LOGF(LogInputDeviceRegistry, Log, "RemoveDevice: DeviceId=%d", DeviceId.GetId());
	
	FInputDeviceRegistry& Registry = Get();
	FWriteScopeLock ScopeLock(Registry.Lock);
	Registry.Descriptors.Remove(DeviceId);
}

FString FInputDeviceDescriptor::ToString() const
{
	return FString::Printf(TEXT("DeviceId=%d InputDeviceName=%s HardwareDeviceIdentifier=%s"),
		HardwareDeviceHandle.GetId(),
		*InputDeviceName.ToString(),
		*HardwareDeviceIdentifier.ToString());
}

TOptional<FInputDeviceDescriptor> FInputDeviceRegistry::FindDescriptor(const FInputDeviceId DeviceId)
{
	FInputDeviceRegistry& Registry = Get();
	FReadScopeLock ScopeLock(Registry.Lock);

#if !UE_BUILD_SHIPPING
	// Simulated descriptors take priority so that non-shipping tooling can override what
	// FindDescriptor returns without touching the real runtime registry.
	if (const FInputDeviceDescriptor* Simulated = Registry.SimulatedDescriptors.Find(DeviceId))
	{
		return *Simulated;
	}
#endif // !UE_BUILD_SHIPPING

	if (const FInputDeviceDescriptor* Found = Registry.Descriptors.Find(DeviceId))
	{
		return *Found;
	}
	return TOptional<FInputDeviceDescriptor>();
}

#if !UE_BUILD_SHIPPING

void FInputDeviceRegistry::SetSimulatedDescriptor(const FInputDeviceId DeviceId, const FInputDeviceDescriptor& SimulatedDescriptor)
{
	UE_LOGF(LogInputDeviceRegistry, Log, "SetSimulatedDescriptor: DeviceId=%d %ls", DeviceId.GetId(), *SimulatedDescriptor.ToString());

	FInputDeviceRegistry& Registry = Get();
	FWriteScopeLock ScopeLock(Registry.Lock);
	Registry.SimulatedDescriptors.Add(DeviceId, SimulatedDescriptor);
}

void FInputDeviceRegistry::ClearSimulatedDescriptor(const FInputDeviceId DeviceId)
{
	UE_LOGF(LogInputDeviceRegistry, Log, "ClearSimulatedDescriptor: DeviceId=%d", DeviceId.GetId());

	FInputDeviceRegistry& Registry = Get();
	FWriteScopeLock ScopeLock(Registry.Lock);
	Registry.SimulatedDescriptors.Remove(DeviceId);
}

void FInputDeviceRegistry::ClearAllSimulatedDescriptors()
{
	UE_LOGF(LogInputDeviceRegistry, Log, "ClearAllSimulatedDescriptors");

	FInputDeviceRegistry& Registry = Get();
	FWriteScopeLock ScopeLock(Registry.Lock);
	Registry.SimulatedDescriptors.Reset();
}
#endif // !UE_BUILD_SHIPPING

FOnInputDeviceDescriptorChanged& FInputDeviceRegistry::OnInputDeviceDescriptorChanged()
{
	return Get().DescriptorChangedDelegate;
}

void FInputDeviceRegistry::LogRegistryContents()
{
	FInputDeviceRegistry& Registry = Get();
	FReadScopeLock ScopeLock(Registry.Lock);

	UE_LOGF(LogInputDeviceRegistry, Display, "Input Device Registry contents:");
	UE_LOGF(LogInputDeviceRegistry, Display, "  Registered descriptors: %d", Registry.Descriptors.Num());
	for (const TPair<FInputDeviceId, FInputDeviceDescriptor>& Pair : Registry.Descriptors)
	{
#if !UE_BUILD_SHIPPING
		const bool bSimulated = Registry.SimulatedDescriptors.Contains(Pair.Key);
#else
		const bool bSimulated = false;
#endif
		UE_LOGF(LogInputDeviceRegistry, Display, "    [%ls] %ls",
			bSimulated ? TEXT("OVERRIDDEN") : TEXT("ACTIVE    "),
			*Pair.Value.ToString());
	}

#if !UE_BUILD_SHIPPING
	UE_LOGF(LogInputDeviceRegistry, Display, "  Simulated descriptors: %d", Registry.SimulatedDescriptors.Num());
	for (const TPair<FInputDeviceId, FInputDeviceDescriptor>& Pair : Registry.SimulatedDescriptors)
	{
		const bool bShadowsReal = Registry.Descriptors.Contains(Pair.Key);
		UE_LOGF(LogInputDeviceRegistry, Display, "    [%ls] %ls",
			bShadowsReal ? TEXT("SHADOWS  ") : TEXT("STANDALONE"),
			*Pair.Value.ToString());
	}
#endif // !UE_BUILD_SHIPPING
}

#if !UE_BUILD_SHIPPING

//-----------------------------------------------------------------------------
// Non-shipping console commands
//-----------------------------------------------------------------------------

// Input.Devices.SetDeviceDesc <DeviceId> <InputDeviceName>::<HardwareDeviceIdentifier>
//   Example: Input.Devices.SetDeviceDesc 1 XInputInterface::XInputController
static FAutoConsoleCommand CmdSetSimulatedDescriptor(
	TEXT("Input.Devices.SetDeviceDesc"),
	TEXT("(Non-shipping) Simulate a device descriptor for a given device ID.\n")
	TEXT("Usage: Input.Devices.SetDeviceDesc <DeviceId> <InputDeviceName>::<HardwareDeviceIdentifier>\n")
	TEXT("Example: Input.Devices.SetDeviceDesc 1 XInputInterface::XInputController"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		if (Args.Num() < 2)
		{
			UE_LOGF(LogInputDeviceRegistry, Warning,
				"Input.Devices.SetDeviceDesc: expected 2 arguments: <DeviceId> <InputDeviceName>::<HardwareDeviceIdentifier>");
			return;
		}

		const int32 RawDeviceId = FCString::Atoi(*Args[0]);
		const FInputDeviceId DeviceId = FInputDeviceId::CreateFromInternalId(RawDeviceId);
		if (!DeviceId.IsValid())
		{
			UE_LOGF(LogInputDeviceRegistry, Warning,
				"Input.Devices.SetDeviceDesc: DeviceId '%ls' is not valid.", *Args[0]);
			return;
		}

		// Parse "InputDeviceName::HardwareDeviceIdentifier"
		FString InputDeviceName;
		FString HardwareDeviceIdentifier;
		if (!Args[1].Split(TEXT("::"), &InputDeviceName, &HardwareDeviceIdentifier))
		{
			UE_LOGF(LogInputDeviceRegistry, Warning,
				"Input.Devices.SetDeviceDesc: argument '%ls' is not in the expected format <InputDeviceName>::<HardwareDeviceIdentifier>",
				*Args[1]);
			return;
		}

		FInputDeviceDescriptor Desc;
		Desc.HardwareDeviceHandle     = DeviceId;
		Desc.InputDeviceName          = FName(*InputDeviceName);
		Desc.HardwareDeviceIdentifier = FName(*HardwareDeviceIdentifier);
		FInputDeviceRegistry::SetSimulatedDescriptor(DeviceId, Desc);
	})
);

// Input.Devices.ClearSimulatedDescriptor <DeviceId>
static FAutoConsoleCommand CmdClearSimulatedDescriptor(
	TEXT("Input.Devices.ClearSimulatedDescriptor"),
	TEXT("(Non-shipping) Remove the simulated descriptor for a given device ID, restoring the real one.\n")
	TEXT("Usage: Input.Devices.ClearSimulatedDescriptor <DeviceId>"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		if (Args.Num() < 1)
		{
			UE_LOGF(LogInputDeviceRegistry, Warning,
				"Input.Devices.ClearSimulatedDescriptor: expected 1 argument: <DeviceId>");
			return;
		}

		const int32 RawDeviceId = FCString::Atoi(*Args[0]);
		const FInputDeviceId DeviceId = FInputDeviceId::CreateFromInternalId(RawDeviceId);
		if (!DeviceId.IsValid())
		{
			UE_LOGF(LogInputDeviceRegistry, Warning,
				"Input.Devices.ClearSimulatedDescriptor: DeviceId '%ls' is not valid.", *Args[0]);
			return;
		}

		FInputDeviceRegistry::ClearSimulatedDescriptor(DeviceId);
	})
);

// Input.Devices.ClearAllSimulatedDescriptors
static FAutoConsoleCommand CmdClearAllSimulatedDescriptors(
	TEXT("Input.Devices.ClearAllSimulatedDescriptors"),
	TEXT("(Non-shipping) Remove all simulated device descriptors, restoring real ones for all devices."),
	FConsoleCommandDelegate::CreateLambda([]()
	{
		FInputDeviceRegistry::ClearAllSimulatedDescriptors();
	})
);

#endif // !UE_BUILD_SHIPPING

// Input.Devices.List
static FAutoConsoleCommand CmdListDevices(
	TEXT("Input.Devices.List"),
	TEXT("Log the current contents of the input device registry. In non-shipping builds this also includes any simulated descriptors."),
	FConsoleCommandDelegate::CreateLambda([]()
	{
		FInputDeviceRegistry::LogRegistryContents();
	})
);