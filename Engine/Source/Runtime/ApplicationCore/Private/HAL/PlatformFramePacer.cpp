// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/PlatformFramePacer.h"
#include "HAL/IConsoleManager.h"

DEFINE_LOG_CATEGORY_STATIC(LogFramePacer, Log, All);

static FAutoConsoleCommand CFramePace(
	TEXT("r.SetFramePace"),
	TEXT("Set a target frame rate for the frame pacer.")
	TEXT("To set 30fps: \"r.SetFramePace 30\""),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
{
	if (Args.Num() > 0)
	{
		int32 RequestFramePace = (int32)FCString::Atod(*Args[0]);
		int32 ResultFramePace = FPlatformRHIFramePacer::SetFramePace(RequestFramePace);
		UE_LOGF(LogFramePacer, Display, "r.FramePace : requesting %d, set as %d", RequestFramePace, ResultFramePace);
	}
	else
	{
		UE_LOGF(LogFramePacer, Display, "r.FramePace : current frame pace is %d", FPlatformRHIFramePacer::GetFramePace());
	}
}),
ECVF_Default);