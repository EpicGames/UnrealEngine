// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_ACCESSIBILITY

#include "IOS/Accessibility/IOSAccessibilityCache.h"
#include "IOS/Accessibility/IOSAccessibilityElement.h"
#include "IOS/IOSApplication.h"
#include "IOS/IOSAppDelegate.h"
#include "IOS/IOSAsyncTask.h"
#include "IOS/IOSView.h"
#include "HAL/IConsoleManager.h"
#include "Apple/AppleStringUtils.h"
#include "Async/TaskGraphInterfaces.h"

@implementation FIOSAccessibilityCache
@synthesize RootWindowId;

#if !UE_BUILD_SHIPPING
static void DumpAccessibilityStatsForwarder()
{
	[[FIOSAccessibilityCache AccessibilityElementCache] DumpAccessibilityStats];
}
#endif

-(id)init
{
	Cache = [[NSMutableDictionary alloc] init];

#if !UE_BUILD_SHIPPING
	IConsoleManager::Get().RegisterConsoleCommand
	(
		TEXT("Accessibility.DumpStatsIOS"),
		TEXT("Writes to LogAccessibility the memory stats for the platform-level accessibility data (AccessibilityElements) required for IOS support."),
		FConsoleCommandDelegate::CreateStatic(&DumpAccessibilityStatsForwarder),
		ECVF_Default
	);
#endif

	return self;
}

-(void)dealloc
{
	[Cache release];
	[super dealloc];
}

-(FIOSAccessibilityLeaf*)GetAccessibilityElement:(AccessibleWidgetId)Id
{
	if (Id == IAccessibleWidget::InvalidAccessibleWidgetId)
	{
		return nil;
	}
	
	FIOSAccessibilityLeaf* ExistingElement = [Cache objectForKey:[NSNumber numberWithInt:Id]];
	if (ExistingElement == nil)
	{
		ExistingElement = [[[FIOSAccessibilityLeaf alloc] initWithId:Id] autorelease];
		[Cache setObject:ExistingElement forKey:[NSNumber numberWithInt:Id]];
	}
	return ExistingElement;
}

-(bool)AccessibilityElementExists:(AccessibleWidgetId)Id
{
	return [Cache objectForKey:[NSNumber numberWithInt:Id]] != nil;
}

-(void)RemoveAccessibilityElement:(AccessibleWidgetId)Id
{
	[Cache removeObjectForKey:[NSNumber numberWithInt:Id]];
}

-(void)Clear
{
	[Cache removeAllObjects];
}

+(id)AccessibilityElementCache
{
	static FIOSAccessibilityCache* Cache = nil;
	if (Cache == nil)
	{
		Cache = [[self alloc] init];
	}
	return Cache;
}

-(void)UpdateAllCachedProperties
{
	TArray<AccessibleWidgetId> Ids;
	for (NSString* Key in Cache)
	{
		Ids.Add(Key.intValue);
	}

	if (Ids.Num() > 0)
	{
		// All IAccessibleWidget functions must be run on Game Thread
		FFunctionGraphTask::CreateAndDispatchWhenReady([Ids]()
		{
			for (const AccessibleWidgetId Id : Ids)
			{
				TSharedPtr<IAccessibleWidget> Widget = [IOSAppDelegate GetDelegate].IOSApplication->GetAccessibleMessageHandler()->GetAccessibleWidgetFromId(Id);
				if (Widget.IsValid())
				{
					// Children
					TArray<int32> ChildIds;
					for (int32 i = 0; i < Widget->GetNumberOfChildren(); ++i)
					{
						TSharedPtr<IAccessibleWidget> Child = Widget->GetChildAt(i);
						if (Child.IsValid())
						{
							ChildIds.Add(Child->GetId());
						}
					}

					// Bounding rect
					FBox2D Bounds = Widget->GetBounds();
					const double Scale = [IOSAppDelegate GetDelegate].IOSView.contentScaleFactor;
					Bounds.Min /= Scale;
					Bounds.Max /= Scale;

					// Visibility
					const bool bIsEnabled = Widget->IsEnabled();
					const bool bIsVisible = !Widget->IsHidden();

					// All UIKit functions must be run on Main Thread
					dispatch_async(dispatch_get_main_queue(), ^
					{
						FIOSAccessibilityLeaf* Element = [[FIOSAccessibilityCache AccessibilityElementCache] GetAccessibilityElement:Id];
						Element.ChildIds = ChildIds;
						Element.Bounds = Bounds;
						[Element SetAccessibilityTrait:UIAccessibilityTraitNotEnabled Set:!bIsEnabled];
						Element.bIsVisible = bIsVisible;
						if ([Element ShouldEmptyCachedStrings])
						{
							[Element EmptyCachedStrings];
						}
					});
				}
			}
		}, TStatId(), NULL, ENamedThreads::GameThread);
	}
}

#if !UE_BUILD_SHIPPING
-(void)DumpAccessibilityStats
{
	// @TODO: Update stats with current data
//	const uint32 NumContainers = [Cache count];
//	uint32 SizeOfContainer = 0;
//	uint32 SizeOfLeaf = 0;
//	uint32 CacheSize = 0;
//
//	NSArray* Keys = [Cache allKeys];
//	for (NSString* Key in Keys)
//	{
//		FIOSAccessibilityLeaf* Leaf = [Cache objectForKey : Key];
//		FIOSAccessibilityContainer* Container = Nil;
//		SizeOfContainer = malloc_size(Container);
//		SizeOfLeaf = malloc_size(Leaf);
//		CacheSize += sizeof(NSString*) + sizeof(FIOSAccessibilityContainer*) + malloc_size(Key) + SizeOfContainer + SizeOfLeaf
//			+ malloc_size([NSString stringWithFString : Leaf.Label])
//			+ malloc_size([NSString stringWithFString : Leaf.Hint])
//			+ malloc_size([NSString stringWithFString : Leaf.Value]);
//	}
//
//	UE_LOGF(LogAccessibility, Log, "Number of Accessibility Elements: %i", NumContainers * 2);
//	UE_LOGF(LogAccessibility, Log, "Size of FIOSAccessibilityContainer: %u", SizeOfContainer);
//	UE_LOGF(LogAccessibility, Log, "Size of FIOSAccessibilityLeaf: %u", SizeOfLeaf);
//	UE_LOGF(LogAccessibility, Log, "Memory stored in cache: %u kb", CacheSize / 1000);
}
#endif

@end

#endif
