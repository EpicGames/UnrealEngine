// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_ACCESSIBILITY

#include "IOS/Accessibility/IOSAccessibilityElement.h"

#include "Apple/AppleStringUtils.h"
#include "Async/TaskGraphInterfaces.h"
#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"
#include "IOS/Accessibility/IOSAccessibilityCache.h"
#include "IOS/IOSApplication.h"
#include "IOS/IOSAppDelegate.h"
#include "IOS/IOSView.h"

/**
 * The iOS Accessibility Hierarchy
 *
 *
 * The [FIOSAccessibilityContainer accessibilityContainer] and [FIOSAccessibilityLeaf accessibilityContainer] methods
 * are what enforce the accessibility hierarchy to allow iOS to navigate the accessibility tree using VoiceOver gestures.
 * When [UIAccessibilityElement isAccessibilityElement] returns NO, only accessibility container functions will be called
 * to retrieve the next/previous accessible element.
 * If it returns YES, the element can be focused on and its accessibility attributes can be retrieved.
 * Notice how we can't have a UIAccessibilityElement that both supports children, focus and retrieving accessibility attributes.
 *  Thus, every FIOSAccessibilityLeaf that has children will have a
 * corresponding FIOSAccessibilityContainer. The FIOSAccessibilityContainer will
 * have the leaf as the first child and all the children of the leaf as subsequent children. I.e the leaf and all its children are siblings all parented to the container.
 * This allows us to focus on the FIOSAccessibilityElements while allowing for next/previous navigation for children elements.
 *
 * Below shows how we need to transform the original Slate hierarchy into a form that iOS recognizes
 * Slate Hierarchy:
 * IOSView
 *  	WindowLeaf
 *  			Leaf1
 *  			 			Leaf2
 *  			 			 			Leaf3
 * IOS Accessibility Hierarchy:
 * IOSView
 *  	WindowContainer
 *  	 		WindowLeaf
 *  	 		Container1
 *  	 		 		Leaf1
 *  	 		 		Container2
 *  	 		 		 		Leaf2
 *  	 		 		 		Leaf3
 *
 * Working Hypothesis On How Next/Previous Gestures Work
 * When you perform the next/previous gesture, the iOS accessibility framework will call accessibilityContainer to
 * retrieve the container of the currently focused FIOSAccessibilityLeaf. iOS will figure out the index of the focused FIOSAccessibilityLeaf
 * among the accessibilityChildren of the container using [FIOSAccessibilityContainer accessibilityElementAtIndex] and/or [FIOSAccessibilityContainer indexOfAccessibilityElement].
 * Once it has the index, then it'll increment/decrement it accordingly and call [FIOSAccessibilityContainer accessibilityElementAtIndex] to find the next/previous sibling.
 * If an FIOSAccessibilityLeaf is found, it returns the leaf to be focus .
 * If an FIOSAccessibilityContainer is found, since containers can't be focused, it searches
 * for the first FIOSAccessibilityLeaf among its children and returns that to be focused.
 */

@implementation FIOSAccessibilityContainer
@synthesize Leaf;

-(id)initWithLeaf:(FIOSAccessibilityLeaf*)InLeaf
{
	// IOSView just used for initialization.
	// actual accessibility container will be returned by accessibilityContainer method
	if (self = [super initWithAccessibilityContainer : [IOSAppDelegate GetDelegate].IOSView])
	{
	  self.Leaf = InLeaf;
	}
	return self;
}

-(void)dealloc
{
	// This should have been cleaned up in [FIOSAccessibilityLeaf dealloc]
	check(!Leaf);
	if (_AccessibilityChildren)
	{
		[_AccessibilityChildren removeAllObjects];
		[_AccessibilityChildren release];
	}
	[super dealloc];
}

/**
 * Having isAccessibilityElement return NO ensures that only functions
 *  to do with accessibility elements will be called. Returning NO also
 *  means the element will never be focused on.
 *  @see accessibilityElements, accessibilityElementCount, accessibilityElementAtIndex, indexOfAccessibilityElement
 */
-(BOOL)isAccessibilityElement
{
	return NO;
}

/**
 * Used to enforce The iOS Accessibility Hierarchy.
 * Due to the nature of the iOS Accessibility Hierarchy, the returned array
 * can contain both FIOSAccessibilityContainer and FIOSAccessibilityLeaf elements.
 * @see The iOS Accessibility Hierarchy
 */
-(NSArray*)accessibilityElements
{
	if (self.Leaf.ChildIds.Num() < 1)
	{
		return Nil;
	}
if (!_AccessibilityChildren)
{
	_AccessibilityChildren = [[NSMutableArray alloc] init];
}
	else
	{
		[_AccessibilityChildren removeAllObjects];
	}
	// We always treat the Leaf as the first element in our accessibilityElements array
	[_AccessibilityChildren addObject: self.Leaf];
	for (const AccessibleWidgetId ChildId : self.Leaf.ChildIds)
	{
		FIOSAccessibilityLeaf* Child = [[FIOSAccessibilityCache AccessibilityElementCache] GetAccessibilityElement:ChildId];
		// Retrieved child should never be Nil
		check(Child);
		// If the child has children, we want to add the corresponding accessibilityContainer to enforce the correct accessibility hierarchy
		if (Child.ChildIds.Num() > 0)
		{
			[_AccessibilityChildren addObject: Child.accessibilityContainer];
		}
		else
		{
			[_AccessibilityChildren addObject: Child];
		}
	}
	return _AccessibilityChildren;
}

-(NSInteger)accessibilityElementCount
{
	// The extra +1 is from the Leaf element
	return self.Leaf.ChildIds.Num() + 1;
}

/**
 * Used in conjunction with accessibilityElements method to determine which
 * the next/previous accessibility element should be focused on with VoiceOver gestures.
 * The accessibilityElements of an FIOSAccessibilityContainer consists of a leaf as the first element and the children of the leaf as subsequent elements.
 * As such, the index will need to be offset to return the correct element.
 * @see accessibilityElements
 */
-(id)accessibilityElementAtIndex:(NSInteger)index
{
	if (index < 0 || index >= self.accessibilityElementCount)
	{
		return Nil;
	}
	// The leaf is always the first element in accessibilityElements
	if (index == 0)
	{
		return self.Leaf;
	}
	else
	{
		// We subtract 1 as the first element is the leaf and offsets everything else
		FIOSAccessibilityLeaf* ChildAtIndex = [[FIOSAccessibilityCache AccessibilityElementCache] GetAccessibilityElement:self.Leaf.ChildIds[index - 1]];
		// if the child has children, we need to return the accessibilityContainer to enforce accessibility hierarchy
		if (ChildAtIndex.ChildIds.Num() > 0)
		{
			return ChildAtIndex.accessibilityContainer;
		}
		return ChildAtIndex;
	}
}

/**
 * Used in conjunction with accessibilityElements method to determine which
 * the next/previous accessibility element should be focused on with VoiceOver gestures.
 * The accessibilityElements of an FIOSAccessibilityContainer consists of a leaf as the first element and the children of the leaf as subsequent elements.
 * As such, the index will need to be offset.
 * @see accessibilityElements
*/
-(NSInteger)indexOfAccessibilityElement:(id)element
{
	if (self.Leaf == element)
	{
		// If it's a Leaf, it is the first child of the container's accessibilityElements array
		return 0;
	}

	for (int32 i = 0; i < self.Leaf.ChildIds.Num(); ++i)
	{
		AccessibleWidgetId ChildId = self.Leaf.ChildIds[i];
		FIOSAccessibilityLeaf* Child = [[FIOSAccessibilityCache AccessibilityElementCache] GetAccessibilityElement: ChildId];
		// The element could either be a leaf or container based whether the
		// element has children in the accessibility hierarchy
		if (Child == element || Child.accessibilityContainer == element)
		{
			// +1 is from the additional leaf as the first element of accessibilityElements
			return i + 1;
		}
	}
	return NSNotFound;
}

/**
 * Together with [FIOSAccessibilityLeaf accessibilityContainer],
 * enforces an accessibility hierarchy thatt iOS can recognize and navigate with VoiceOver gestures.
 * @see [FIOSAccessibilityLeaf accessibilityContainer], The iOS Accessibility Hierarchy
 */
-(id)accessibilityContainer
{
	// If we're the root IAccessibleWindow, our container is the IOSView
	if (self.Leaf.Id == [FIOSAccessibilityCache AccessibilityElementCache].RootWindowId)
	{
		return [IOSAppDelegate GetDelegate].IOSView;
	}
	// The leaf could have an invalid parent Id when the parent is being removed.
	// It's very rare for this to happen as [FIOSAccessibilityLeaf accessibilityContainer]
	// should already handle the case where its parent Id is invalid.
	// But just to be safe.
	if (self.Leaf.ParentId == IAccessibleWidget::InvalidAccessibleWidgetId)
	{
		return [IOSAppDelegate GetDelegate].IOSView;
	}
	FIOSAccessibilityLeaf* LeafParent = [[FIOSAccessibilityCache AccessibilityElementCache] GetAccessibilityElement: self.Leaf.ParentId];
	// see The iOS Accessibility Hierarchy to understand why this works
	return LeafParent.accessibilityContainer;
}

-(CGRect)accessibilityFrame
{
	return self.Leaf.accessibilityFrame;
}

-(id)accessibilityHitTest:(CGPoint)point
{
	AccessibleWidgetId FoundId = IAccessibleWidget::InvalidAccessibleWidgetId;
	const double Scale = [IOSAppDelegate GetDelegate].IOSView.contentScaleFactor;
	const int32 X = FMath::TruncToInt(point.x * Scale);
	const int32 Y = FMath::TruncToInt(point.y * Scale);
	// Update the labels while we're on the game thread, since IOS is going to request them immediately.
	FString TempLabel, TempHint, TempValue;
	const AccessibleWidgetId TempId = self.Leaf.Id;
	[IOSAppDelegate WaitAndRunOnGameThread:[TempId, X, Y, &FoundId, &TempLabel, &TempHint, &TempValue]()
	{
		const TSharedPtr<IAccessibleWidget> Widget = [IOSAppDelegate GetDelegate].IOSApplication->GetAccessibleMessageHandler()->GetAccessibleWidgetFromId(TempId);
		if (Widget.IsValid())
		{
			const TSharedPtr<IAccessibleWidget> HitWidget = Widget->GetWindow()->AsWindow()->GetChildAtPosition(X, Y);
			if (HitWidget.IsValid())
			{
				FoundId = HitWidget->GetId();
				TempLabel = HitWidget->GetWidgetName();
				TempHint = HitWidget->GetHelpText();
				if (HitWidget->AsProperty())
				{
					TempValue = HitWidget->AsProperty()->GetValue();
				}
			}
		}
	}];

	if (FoundId != IAccessibleWidget::InvalidAccessibleWidgetId)
	{
		FIOSAccessibilityLeaf* FoundLeaf = [[FIOSAccessibilityCache AccessibilityElementCache] GetAccessibilityElement:FoundId];
		if ([FoundLeaf ShouldCacheStrings])
		{
			FoundLeaf.Label = MoveTemp(TempLabel);
			FoundLeaf.Hint = MoveTemp(TempHint);
			FoundLeaf.Value = MoveTemp(TempValue);
			FoundLeaf.LastCachedStringTime = FPlatformTime::Seconds();
		}
		return FoundLeaf;
	}
	else
	{
		return Leaf;
	}
}

@end

@implementation FIOSAccessibilityLeaf
@synthesize Label;
@synthesize Hint;
@synthesize Value;
@synthesize Traits;
@synthesize LastCachedStringTime;
@synthesize Id;
@synthesize ChildIds;
@synthesize Bounds;
@synthesize bIsVisible;
@synthesize ParentId;

-(id)initWithId:(AccessibleWidgetId)InId
{
	// initialize with IOS View as container, but the actual hierarchy will be determined by calls to accessibilityContainer
	if (self = [super initWithAccessibilityContainer : [IOSAppDelegate GetDelegate].IOSView])
	{
		self.Id = InId;
		self.bIsVisible = true;
		// All IAccessibleWidget functions must be run on Game Thread
		FFunctionGraphTask::CreateAndDispatchWhenReady([InId]()
		{
			TSharedPtr<IAccessibleWidget> Widget = [IOSAppDelegate GetDelegate].IOSApplication->GetAccessibleMessageHandler()->GetAccessibleWidgetFromId(InId);
			if (Widget.IsValid())
			{
				// Most accessibility traits cannot be changed after setting, so initialize them here
				UIAccessibilityTraits InitialTraits = UIAccessibilityTraitNone;
				if (Widget->AsProperty() && !FMath::IsNearlyZero(Widget->AsProperty()->GetStepSize()))
				{
					InitialTraits |= UIAccessibilityTraitAdjustable;
				}
				if (Widget->AsActivatable())
				{
					InitialTraits |= UIAccessibilityTraitButton;
				}
				if (Widget->GetWidgetType() == EAccessibleWidgetType::Image)
				{
					InitialTraits |= UIAccessibilityTraitImage;
				}
				if (Widget->GetWidgetType() == EAccessibleWidgetType::Hyperlink)
				{
					InitialTraits |= UIAccessibilityTraitLink;
				}
				if (!Widget->IsEnabled())
				{
					InitialTraits |= UIAccessibilityTraitNotEnabled;
				}
				FString InitialLabel = Widget->GetWidgetName();
				FString InitialHint = Widget->GetHelpText();
				FString InitialValue;
				if (Widget->AsProperty())
				{
					InitialValue = Widget->AsProperty()->GetValue();
				}
				AccessibleWidgetId InitialParentId = IAccessibleWidget::InvalidAccessibleWidgetId;
				TSharedPtr<IAccessibleWidget> InitialParent = Widget->GetParent();
				if (InitialParent.IsValid())
				{
					InitialParentId = InitialParent->GetId();
				}
				// All UIKit functions must be run on Main Thread
				dispatch_async(dispatch_get_main_queue(), ^
				{
					FIOSAccessibilityLeaf* Self = [[FIOSAccessibilityCache AccessibilityElementCache] GetAccessibilityElement:InId];
					Self.Traits = InitialTraits;
					Self.Label = InitialLabel;
					Self.Hint = InitialHint;
					Self.Value = InitialValue;
					Self.LastCachedStringTime = FPlatformTime::Seconds();
					[Self SetParent:InitialParentId];
				});
			}
		}, TStatId(), NULL, ENamedThreads::GameThread);
	}
	return self;
}

-(void)dealloc
{
	if (_Container)
	{
		// removes self from the container
		_Container.Leaf = Nil;
		[_Container release];
	}
	[super dealloc];
}

-(void)SetParent:(AccessibleWidgetId)InParentId
{
	self.ParentId = InParentId;
}

/**
 * Returning YES for isAccessibilityElement allows the
 * UIAccessibilityElement to be focused on with VoiceOver, be found by hit tests and calls the methods to get accessibility data from the element
 * e.g accessibilityLabel, accessibilityHint etc
 * Returning YES also prevents accessibility container methods from being called
 * E.g accessibilityElements
 */
-(BOOL)isAccessibilityElement
{
	// We ignore the root window so it doesn't keep getting announced during hit tests
	// @TODO: Remove this when we have cross platform ability to selectively ignore accessible windows
	return (self.Id != [FIOSAccessibilityCache AccessibilityElementCache].RootWindowId);
}

/**
 * In conjunction with [FIOSAccessibilityContainer accessibilityContainer],
 * enforces The iOS Accessibility Hierarchy.
 * We create and return an FIOSAccessibilityContainer if the leaf has any children.
 * @see [FIOSAccessibilityContainer accessibilityContainer], The iOS Accessibility Hierarchy
 */
-(id)accessibilityContainer
{
	if ((self.ChildIds.Num() > 0) || ((self.Id == [FIOSAccessibilityCache AccessibilityElementCache].RootWindowId)))
	{
		if (_Container == Nil)
		{
			_Container = [[FIOSAccessibilityContainer alloc] initWithLeaf:self];
		}
		return _Container;
	}
	// iOS seems to still hold on to an FIOSAccessibilityLeaf for a while after its corresponding Slate widget has been removed
	// in FIOSApplication::OnAccessibleEventRaised
	// before dealloc is called. We guard against that here
	if (self.ParentId == IAccessibleWidget::InvalidAccessibleWidgetId)
	{
		return Nil;
	}
	FIOSAccessibilityLeaf* Parent = [[FIOSAccessibilityCache AccessibilityElementCache] GetAccessibilityElement: self.ParentId];
	// Parent should always be valid at this point
	check(Parent);
	// See The iOS Accessibility Hierarchy to understand why we do this
	return Parent.accessibilityContainer;
}

-(CGRect)accessibilityFrame
{
	// This function will be called less than the function to cache the bounds, so make
	// the IOS rect here. If we refactor the code to not using a polling-based cache,
	// it may make more sense to change the Bounds property itself to a CGRect.
	// @TODO: Investigate returning super's accessibility frame if the leaf is not visible
	return CGRectMake(Bounds.Min.X, Bounds.Min.Y, Bounds.Max.X - Bounds.Min.X, Bounds.Max.Y - Bounds.Min.Y);
}

-(bool)ShouldCacheStrings
{
	return FPlatformTime::Seconds() - LastCachedStringTime > 1.0;
}

-(bool)ShouldEmptyCachedStrings
{
	return FPlatformTime::Seconds() - LastCachedStringTime > 5.0;
}

-(void)EmptyCachedStrings
{
	Label.Empty();
	Hint.Empty();
	Value.Empty();
}

-(void)CacheStrings
{
	FString TempLabel, TempHint, TempValue;
	const AccessibleWidgetId SelfId = self.Id;
	[IOSAppDelegate WaitAndRunOnGameThread:[SelfId, &TempLabel, &TempHint, &TempValue]()
	{
		const TSharedPtr<IAccessibleWidget> Widget = [IOSAppDelegate GetDelegate].IOSApplication->GetAccessibleMessageHandler()->GetAccessibleWidgetFromId(SelfId);
		if (Widget.IsValid())
		{
			TempLabel = Widget->GetWidgetName();
			TempHint = Widget->GetHelpText();
			if (Widget->AsProperty())
			{
				TempValue = Widget->AsProperty()->GetValue();
			}
		}
	}];
	self.Label = MoveTemp(TempLabel);
	self.Hint = MoveTemp(TempHint);
	self.Value = MoveTemp(TempValue);
	self.LastCachedStringTime = FPlatformTime::Seconds();
}

-(NSString*)accessibilityLabel
{
	// when we focus on an accessibility element via swiping, this function is always called.  ,
	// label, hint and value are most likely all requested in quick succession of each other.
	// So we only cache the strings here as an optimization
	// Assumption: hint and value will  never be called standalone.
	if ([self ShouldCacheStrings])
	{
		[self CacheStrings];
	}
	if (!self.Label.IsEmpty())
	{
		return [NSString stringWithFString:self.Label];
	}
	return nil;
}

-(NSString*)accessibilityHint
{
	if (!self.Hint.IsEmpty())
	{
		return [NSString stringWithFString:self.Hint];
	}
	return nil;
}

-(NSString*)accessibilityValue
{
	if (!self.Value.IsEmpty())
	{
		return [NSString stringWithFString:self.Value];
	}
	return nil;
}

-(void)SetAccessibilityTrait:(UIAccessibilityTraits)Trait Set:(bool)IsEnabled
{
	if (IsEnabled)
	{
		self.Traits |= Trait;
	}
	else
	{
		self.Traits &= ~Trait;
	}
}

-(UIAccessibilityTraits)accessibilityTraits
{
	return self.Traits;
}

-(void)accessibilityIncrement
{
	const AccessibleWidgetId TempId = self.Id;
	// All IAccessibleWidget functions must be run on Game Thread
	FFunctionGraphTask::CreateAndDispatchWhenReady([TempId]()
	{
		TSharedPtr<IAccessibleWidget> Widget = [IOSAppDelegate GetDelegate].IOSApplication->GetAccessibleMessageHandler()->GetAccessibleWidgetFromId(TempId);
		if (Widget.IsValid())
		{
			IAccessibleProperty* Property = Widget->AsProperty();
			if (Property && !FMath::IsNearlyZero(Property->GetStepSize()))
			{
				const float CurrentValue = FCString::Atof(*Property->GetValue());
				Property->SetValue(FString::SanitizeFloat(CurrentValue + Property->GetStepSize()));
			}
		}
	}, TStatId(), NULL, ENamedThreads::GameThread);
}

-(void)accessibilityDecrement
{
	const AccessibleWidgetId TempId = self.Id;
	// All IAccessibleWidget functions must be run on Game Thread
	FFunctionGraphTask::CreateAndDispatchWhenReady([TempId]()
	{
		TSharedPtr<IAccessibleWidget> Widget = [IOSAppDelegate GetDelegate].IOSApplication->GetAccessibleMessageHandler()->GetAccessibleWidgetFromId(TempId);
		if (Widget.IsValid())
		{
			IAccessibleProperty* Property = Widget->AsProperty();
			if (Property && !FMath::IsNearlyZero(Property->GetStepSize()))
			{
				const float CurrentValue = FCString::Atof(*Property->GetValue());
				Property->SetValue(FString::SanitizeFloat(CurrentValue - Property->GetStepSize()));
			}
		}
	}, TStatId(), NULL, ENamedThreads::GameThread);
}

-(BOOL)accessibilityActivate
{
	// @TODO: Verify if its an activatable before calling super
	return [super accessibilityActivate];
}

@end

#endif
