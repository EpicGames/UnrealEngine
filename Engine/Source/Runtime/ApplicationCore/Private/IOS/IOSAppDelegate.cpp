// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOS/IOSAppDelegate.h"
#include "Apple/AppleStringUtils.h"
#include "IOS/IOSCommandLineHelper.h"
#include "HAL/ExceptionHandling.h"
#include "Modules/Boilerplate/ModuleBoilerplate.h"
#include "Misc/CallbackDevice.h"
#include "IOS/IOSView.h"
#include "IOS/IOSWindow.h"
#include "Async/TaskGraphInterfaces.h"
#include "GenericPlatform/GenericPlatformChunkInstall.h"
#include "IOS/IOSPlatformMisc.h"
#include "BackgroundURLSessionHandler.h"
#include "HAL/PlatformStackWalk.h"
#include "HAL/PlatformProcess.h"
#include "Misc/OutputDeviceError.h"
#include "Misc/CommandLine.h"
#include "IOS/IOSPlatformFramePacer.h"
#include "IOS/IOSApplication.h"
#include "IOS/IOSAsyncTask.h"
#include "Misc/ConfigCacheIni.h"
#include "IOS/IOSPlatformCrashContext.h"
#include "Misc/OutputDeviceError.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/FeedbackContext.h"
#include "Misc/App.h"
#include "Algo/AllOf.h"
#include "Misc/App.h"
#include "Misc/EmbeddedCommunication.h"
#include "Containers/Ticker.h"
#include "UObject/GarbageCollection.h"
#include "Templates/Function.h"
#if USE_MUTE_SWITCH_DETECTION
#include "SharkfoodMuteSwitchDetector.h"
#include "SharkfoodMuteSwitchDetector.m"
#endif

#include <AudioToolbox/AudioToolbox.h>
#include <AVFAudio/AVAudioApplication.h>
#include <AVFoundation/AVAudioSession.h>
#import <AVFoundation/AVFoundation.h>
#include "HAL/IConsoleManager.h"
#if WITH_ENGINE
#include "RHICommandList.h"
#include "UnrealClient.h"
#endif

#if WITH_ACCESSIBILITY
#include "IOS/Accessibility/IOSAccessibilityCache.h"
#endif

// this is the size of the game thread stack, it must be a multiple of 4k
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
#define GAME_THREAD_STACK_SIZE 2 * 1024 * 1024
#else
#define GAME_THREAD_STACK_SIZE 16 * 1024 * 1024
#endif

DEFINE_LOG_CATEGORY(LogIOSAudioSession);

int GAudio_ForceAmbientCategory = 1;
//static FAutoConsoleVariableRef CVar_ForceAmbientCategory(
//															  TEXT("audio.ForceAmbientCategory"),
//															  GAudio_ForceAmbientCategory,
//															  TEXT("Force the iOS AVAudioSessionCategoryAmbient category over AVAudioSessionCategorySoloAmbient")
//															  );

extern bool GShowSplashScreen;

FIOSCoreDelegates::FOnOpenURL FIOSCoreDelegates::OnOpenURL;
FIOSCoreDelegates::FOnWillResignActive FIOSCoreDelegates::OnWillResignActive;
FIOSCoreDelegates::FOnDidBecomeActive FIOSCoreDelegates::OnDidBecomeActive;
TArray<FIOSCoreDelegates::FFilterDelegateAndHandle> FIOSCoreDelegates::PushNotificationFilters;

static uint GEnabledAudioFeatures[(uint8)EAudioFeature::NumFeatures];

/*
	From: https://developer.apple.com/library/ios/documentation/UIKit/Reference/UIApplicationDelegate_Protocol/#//apple_ref/occ/intfm/UIApplicationDelegate/applicationDidEnterBackground:
	"In practice, you should return from applicationDidEnterBackground: as quickly as possible. If the method does not return before time runs out your app is terminated and purged from memory."
*/

static float GOverrideThreadWaitTime = 0.0;
static float GMaxThreadWaitTime = 2.0;	// Setting this to be 2 seconds since this wait has to be done twice (once for sending the enter background event to the game thread, and another for waiting on the suspend msg
										// I could not find a reference for this but in the past I believe the timeout was 5 seconds
FAutoConsoleVariableRef CVarThreadBlockTime(
		TEXT("ios.lifecycleblocktime"),
		GMaxThreadWaitTime,
		TEXT("How long to block main IOS thread to make sure gamethread gets time.\n"),
		ECVF_Default);

FString GNotificationActivationProtocolKey = "activationProtocol";
FAutoConsoleVariableRef CVarNotificationActivationProtocolKey(
		TEXT("ios.notification.activationProtocolKey"),
		GNotificationActivationProtocolKey,
		TEXT("Notification data payload key that's associated with the activation protocol"),
		ECVF_Default
);

#if !PLATFORM_TVOS
bool GIOSDelayRotationUntilPresent = 1;
FAutoConsoleVariableRef CVarDelayRotationUntilPresent(
		TEXT("ios.DelayRotationUntilPresent"),
		GIOSDelayRotationUntilPresent,
		TEXT("Whether to delay device rotation handling until we have a frame available at the new orientation (default=1).\n"),
		ECVF_Default);

#if !UE_BUILD_SHIPPING
static FAutoConsoleCommand CForceRotation(
	TEXT("ios.ForceRotation"),
	TEXT("Force a rotation with the integer representing a EDeviceScreenOrientation value passed as a parameter."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		if (Args.Num() > 0)
		{
			int32 RequestRotation = (int32)FCString::Atod(*Args[0]);
			FPlatformMisc::SetAllowedDeviceOrientation((EDeviceScreenOrientation)RequestRotation);
		}
	}),
	ECVF_Default);
#endif
#endif


static void SignalHandler(int32 Signal, struct __siginfo* Info, void* Context)
{
	static int32 bHasEntered = 0;
	if (FPlatformAtomics::InterlockedCompareExchange(&bHasEntered, 1, 0) == 0)
	{
		const SIZE_T StackTraceSize = 65535;
		ANSICHAR* StackTrace = (ANSICHAR*)FMemory::Malloc(StackTraceSize);
		StackTrace[0] = 0;
		
		// Walk the stack and dump it to the allocated memory.
		FPlatformStackWalk::StackWalkAndDump(StackTrace, StackTraceSize, 0, (ucontext_t*)Context);
		UE_LOGF(LogIOS, Error, "%ls", ANSI_TO_TCHAR(StackTrace));
		FMemory::Free(StackTrace);
		
		GError->HandleError();
		FPlatformMisc::RequestExit(true);
	}
}

void InstallSignalHandlers()
{
	struct sigaction Action;
	FMemory::Memzero(&Action, sizeof(struct sigaction));
	
	Action.sa_sigaction = SignalHandler;
	sigemptyset(&Action.sa_mask);
	Action.sa_flags = SA_SIGINFO | SA_RESTART | SA_ONSTACK;
	
	sigaction(SIGQUIT, &Action, NULL);
	sigaction(SIGILL, &Action, NULL);
	sigaction(SIGEMT, &Action, NULL);
	sigaction(SIGFPE, &Action, NULL);
	sigaction(SIGBUS, &Action, NULL);
	sigaction(SIGSEGV, &Action, NULL);
	sigaction(SIGSYS, &Action, NULL);
}

void LexFromString(EAudioFeature& OutFeature, const TCHAR* String)
{
	OutFeature = EAudioFeature::NumFeatures;
	if (FCString::Stricmp(String, TEXT("Playback")) == 0)
	{
		OutFeature = EAudioFeature::Playback;
	}
	else if (FCString::Stricmp(String, TEXT("Record")) == 0)
	{
		OutFeature = EAudioFeature::Record;
	}
	else if (FCString::Stricmp(String, TEXT("DoNotMixWithOthers")) == 0)
	{
		OutFeature = EAudioFeature::DoNotMixWithOthers;
	}
	else if (FCString::Stricmp(String, TEXT("VoiceChat")) == 0)
	{
		OutFeature = EAudioFeature::VoiceChat;
	}
	else if (FCString::Stricmp(String, TEXT("UseReceiver")) == 0)
	{
		OutFeature = EAudioFeature::UseReceiver;
	}
	else if (FCString::Stricmp(String, TEXT("DisableBluetoothSpeaker")) == 0)
	{
		OutFeature = EAudioFeature::DisableBluetoothSpeaker;
	}
	else if (FCString::Stricmp(String, TEXT("BluetoothMicrophone")) == 0)
	{
		OutFeature = EAudioFeature::BluetoothMicrophone;
	}
	else if (FCString::Stricmp(String, TEXT("BackgroundAudio")) == 0)
	{
		OutFeature = EAudioFeature::BackgroundAudio;
	}
	else if (FCString::Stricmp(String, TEXT("AllowHapticsAndSystemSoundsDuringRecording")) == 0)
	{
		OutFeature = EAudioFeature::AllowHapticsAndSystemSoundsDuringRecording;
	}
}

FString LexToString(EAudioFeature Feature)
{
	switch (Feature)
	{
	case EAudioFeature::Playback: return TEXT("Playback");
	case EAudioFeature::Record: return TEXT("Record");
	case EAudioFeature::DoNotMixWithOthers: return TEXT("DoNotMixWithOthers");
	case EAudioFeature::VoiceChat: return TEXT("VoiceChat");
	case EAudioFeature::UseReceiver: return TEXT("UseReceiver");
	case EAudioFeature::DisableBluetoothSpeaker: return TEXT("DisableBluetoothSpeaker");
	case EAudioFeature::BluetoothMicrophone: return TEXT("BluetoothMicrophone");
	case EAudioFeature::BackgroundAudio: return TEXT("BackgroundAudio");
	case EAudioFeature::AllowHapticsAndSystemSoundsDuringRecording: return TEXT("AllowHapticsAndSystemSoundsDuringRecording");
	default: return FString();
	}
}

FDelegateHandle FIOSCoreDelegates::AddPushNotificationFilter(const FPushNotificationFilter& FilterDel)
{
	FDelegateHandle NewHandle(FDelegateHandle::EGenerateNewHandleType::GenerateNewHandle);
	PushNotificationFilters.Push({FilterDel, NewHandle});
	return NewHandle;
}

void FIOSCoreDelegates::RemovePushNotificationFilter(FDelegateHandle Handle)
{
	PushNotificationFilters.RemoveAll([Handle](const FFilterDelegateAndHandle& Entry) {
		return Entry.Handle == Handle;
	});
}

bool FIOSCoreDelegates::PassesPushNotificationFilters(NSDictionary* Payload)
{
	return Algo::AllOf(PushNotificationFilters, [Payload](const FFilterDelegateAndHandle& Entry) {
		return Entry.Filter.Execute(Payload);
	});
}

@interface IOSNotificationHandler : NSObject
+(void)handleWhenReady:(UNNotification*)notification withCompletionHandler:(void (^)()) completionHandler;
@end

@interface IOSAppDelegate()

// move private things from header to here

@end

@implementation IOSAppDelegate

#if !UE_BUILD_SHIPPING && !PLATFORM_TVOS
    @synthesize ConsoleAlertController;
	@synthesize ConsoleHistoryValues;
	@synthesize ConsoleHistoryValuesIndex;
#endif

@synthesize AlertResponse;
@synthesize bDeviceInPortraitMode;
@synthesize bEngineInit;
@synthesize bPlatformInit;
@synthesize OSVersion;

@synthesize Window;
@synthesize IOSView;
@synthesize timer;
@synthesize IdleTimerEnableTimer;
@synthesize IdleTimerEnablePeriod;
#if WITH_ACCESSIBILITY
@synthesize AccessibilityCacheTimer;
#endif
@synthesize savedOpenUrlParameters;


static IOSAppDelegate* CachedDelegate = nil;

+ (IOSAppDelegate*)GetDelegate
{
#if BUILD_EMBEDDED_APP
	if (CachedDelegate == nil)
	{
		UE_LOGF(LogIOS, Fatal, "Currently, a native app embedding Unreal must have the AppDelegate subclass from IOSAppDelegate.");

		// if we are embedded, but CachedDelegate is nil, then that means the delegate was not an IOSAppDelegate subclass,
		// so we need to do a switcheroo - but this is unlikely to work well
#if 0 // ALLOW_NATIVE_APP_DELEGATE
		SCOPED_BOOT_TIMING("Delegate Switcheroo");

		id<UIApplicationDelegate> ExistingDelegate = [UIApplication sharedApplication].delegate;
		
		CachedDelegate = [[IOSAppDelegate alloc] init];
		CachedDelegate.Window = ExistingDelegate.window;
		
		// @todo critical: The IOSAppDelegate needs to save the old delegate, override EVERY UIApplication function, and call the old delegate's functions for each one
		[UIApplication sharedApplication].delegate = CachedDelegate;
		
		// we will have missd the didFinishLaunchingWithOptions call, so we need to call it now, but we don't have the original launchOptions, nothing we can do now!
		[CachedDelegate application:[UIApplication sharedApplication] didFinishLaunchingWithOptions:nil];
#endif
	}
#endif

	return CachedDelegate;
}

-(id)init
{
#if UE_USE_SWIFT_UI_MAIN

	NSArray* Arguments = [[NSProcessInfo processInfo] arguments];
	Arguments = [Arguments subarrayWithRange:NSMakeRange(1, [Arguments count] - 1)];
	FString CmdLine = [Arguments componentsJoinedByString:@" "];

	FIOSCommandLineHelper::InitCommandArgs(*CmdLine);

#endif
	self = [super init];
	CachedDelegate = self;
	memset(GEnabledAudioFeatures, 0, sizeof(GEnabledAudioFeatures));
	return self;
}


-(void)dealloc
{
#if !UE_BUILD_SHIPPING && !PLATFORM_TVOS
	[ConsoleAlertController release];
	[ConsoleHistoryValues release];
#endif
	[Window release];
	[IOSView release];
	[timer release];
#if WITH_ACCESSIBILITY
	if (AccessibilityCacheTimer != nil)
	{
		[AccessibilityCacheTimer invalidate];
		[AccessibilityCacheTimer release];
	}
#endif
	[super dealloc];
}

-(void)MainAppThread:(NSDictionary*)launchOptions
{
	// make sure this thread has an auto release pool setup
	NSAutoreleasePool* AutoreleasePool = [[NSAutoreleasePool alloc] init];

	{
		SCOPED_BOOT_TIMING("[IOSAppDelegate MainAppThread setup]");

		self.bHasStarted = true;
		GIsGuarded = false;
		GStartTime = FPlatformTime::Seconds();


		while(!self.bCommandLineReady)
		{
			usleep(100);
		}
	}

    FTaskTagScope Scope(ETaskTag::EGameThread);

	while(self.bIsSuspended || [UIApplication sharedApplication].applicationState != UIApplicationStateActive)
	{
		usleep(100);
	}
	
	FAppEntry::Init();

	// check for update on app store if cvar is enabled
/*	dispatch_async(dispatch_get_main_queue(), ^{
		[[IOSAppDelegate GetDelegate] DoUpdateCheck];
	});*/
	
	// now that GConfig has been loaded, load the EnabledAudioFeatures from ini
	TArray<FString> EnabledAudioFeatures;
	GConfig->GetArray(TEXT("Audio"), TEXT("EnabledAudioFeatures"), EnabledAudioFeatures, GEngineIni);
	for (const FString& EnabledAudioFeature : EnabledAudioFeatures)
	{
		EAudioFeature Feature = EAudioFeature::NumFeatures;
		LexFromString(Feature, *EnabledAudioFeature);
		if (Feature != EAudioFeature::NumFeatures)
		{
			GEnabledAudioFeatures[static_cast<uint8>(Feature)] = 1;
		}
	}
	[self ToggleAudioSession:true];

#if !BUILD_EMBEDDED_APP
	[self InitIdleTimerSettings];
#endif

	self.bEngineInit = true;

#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
	// Ask iOS to re-evaluate allowed orientations if we change the ios.DelayRotationUntilPresent
	CVarDelayRotationUntilPresent->OnChangedDelegate().AddLambda([](IConsoleVariable* Variable)
	{
		[IOSAppDelegate UpdateSupportedInterfaceOrientations];
	});
	
	dispatch_async(dispatch_get_main_queue(), ^{
		[[IOSAppDelegate GetDelegate].IOSController setNeedsUpdateOfScreenEdgesDeferringSystemGestures];
	});
#endif

	// put a render thread job to turn off the splash screen after the first render flip
	if (GShowSplashScreen)
	{
		FGraphEventRef SplashTask = FFunctionGraphTask::CreateAndDispatchWhenReady([]()
		{
			GShowSplashScreen = false;
		}, TStatId(), NULL, ENamedThreads::ActualRenderingThread);
	}

	for (NSDictionary* openUrlParameter in self.savedOpenUrlParameters)
	{
		UIApplication* application = [openUrlParameter valueForKey : @"application"];
		NSURL* url = [openUrlParameter valueForKey : @"url"];
		NSString * sourceApplication = [openUrlParameter valueForKey : @"sourceApplication"];
		id annotation = [openUrlParameter valueForKey : @"annotation"];
		FIOSCoreDelegates::OnOpenURL.Broadcast(application, url, sourceApplication, annotation);
	}
	self.savedOpenUrlParameters = nil; // clear after saved openurl delegate running

#if BUILD_EMBEDDED_APP
	// tell the embedded app that the while 1 loop is going
	FEmbeddedCallParamsHelper Helper;
	Helper.Command = TEXT("engineisrunning");
	FEmbeddedDelegates::GetEmbeddedToNativeParamsDelegateForSubsystem(TEXT("native")).Broadcast(Helper);
#endif

#if WITH_ACCESSIBILITY
	// Initialize accessibility code if VoiceOver is enabled. This must happen after Slate has been initialized.
	dispatch_async(dispatch_get_main_queue(), ^{
		if (UIAccessibilityIsVoiceOverRunning())
		{
			[[IOSAppDelegate GetDelegate] OnVoiceOverStatusChanged];
		}
	});
#endif

    while( !IsEngineExitRequested() )
	{
        if (self.bIsSuspended)
        {
            FAppEntry::SuspendTick();
            
            self.bHasSuspended = true;
        }
        else
        {
			bool bOtherAudioPlayingNow = [self IsBackgroundAudioPlaying];
			if (bOtherAudioPlayingNow != self.bLastOtherAudioPlaying || self.bForceEmitOtherAudioPlaying)
			{
				FGraphEventRef UserMusicInterruptTask = FFunctionGraphTask::CreateAndDispatchWhenReady([bOtherAudioPlayingNow]()
				   {
					   //NSLog(@"UserMusicInterrupt Change: %s", bOtherAudioPlayingNow ? "playing" : "paused");
					   FCoreDelegates::UserMusicInterruptDelegate.Broadcast(bOtherAudioPlayingNow);
				   }, TStatId(), NULL, ENamedThreads::GameThread);
				
				self.bLastOtherAudioPlaying = bOtherAudioPlayingNow;
				self.bForceEmitOtherAudioPlaying = false;
			}
			
			int OutputVolume = [self GetAudioVolume];
			bool bMuted = false;
			
#if USE_MUTE_SWITCH_DETECTION
			SharkfoodMuteSwitchDetector* MuteDetector = [SharkfoodMuteSwitchDetector shared];
			bMuted = MuteDetector.isMute;
			if (bMuted != self.bLastMutedState || self.bForceEmitMutedState)
			{
				FGraphEventRef AudioMuteTask = FFunctionGraphTask::CreateAndDispatchWhenReady([bMuted, OutputVolume]()
					{
						//NSLog(@"Audio Session %s", bMuted ? "MUTED" : "UNMUTED");
						FCoreDelegates::AudioMuteDelegate.Broadcast(bMuted, OutputVolume);
					}, TStatId(), NULL, ENamedThreads::GameThread);
				
				self.bLastMutedState = bMuted;
				self.bForceEmitMutedState = false;
			}
#endif

			if (static_cast<float>(OutputVolume) != self.LastVolume || self.bForceEmitVolume)
			{
				FGraphEventRef AudioMuteTask = FFunctionGraphTask::CreateAndDispatchWhenReady([bMuted, OutputVolume]()
					{
						//NSLog(@"Audio Volume: %d", OutputVolume);
						FCoreDelegates::AudioMuteDelegate.Broadcast(bMuted, OutputVolume);
					}, TStatId(), NULL, ENamedThreads::GameThread);

				self.LastVolume = static_cast<float>(OutputVolume);
				self.bForceEmitVolume = false;
			}

            FAppEntry::Tick();
        
            // free any autoreleased objects every once in awhile to keep memory use down (strings, splash screens, etc)
            if (((GFrameCounter) & 31) == 0)
            {
				// If you crash upon release, turn on Zombie Objects (Edit Scheme... | Diagnostics | Zombie Objects)
				// This will list the last object sent the release message, which will help identify the double free
                [AutoreleasePool release];
                AutoreleasePool = [[NSAutoreleasePool alloc] init];
            }
        }

        // drain the async task queue from the game thread
        [FAppleAsyncTask ProcessAsyncTasks];
	}

    dispatch_sync(dispatch_get_main_queue(),^
    {
        [UIApplication sharedApplication].idleTimerDisabled = NO;
    });

	[AutoreleasePool release];
	FAppEntry::Shutdown();
    
    self.bHasStarted = false;
    
    if(bForceExit || FApp::IsUnattended())
    {
        _Exit(0);
        //exit(0);  // As far as I can tell we run into a lot of trouble trying to run static destructors, so this is a no go :(
    }
}

-(void)timerForSplashScreen
{
    if (!GShowSplashScreen)
    {
        if ([self.Window viewWithTag:200] != nil)
        {
            [[self.Window viewWithTag:200] removeFromSuperview];
            [self.viewController release];
        }
        [timer invalidate];
    }
}

-(void)RecordPeakMemory
{
    FPlatformMemoryStats MemoryStats = FIOSPlatformMemory::GetStats();

#if ENABLE_LOW_LEVEL_MEM_TRACKER
	FLowLevelMemTracker::Get().SetTagAmountForTracker(ELLMTracker::Platform, ELLMTag::PlatformVM, MemoryStats.UsedVirtual, false);
#endif
}

-(void)InitIdleTimerSettings
{
	float TimerDuration = 0.0F;
	GConfig->GetFloat(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("IdleTimerEnablePeriod"), TimerDuration, GEngineIni);
    IdleTimerEnablePeriod = TimerDuration;
	self.IdleTimerEnableTimer = nil;
	bool bEnableTimer = YES;
	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bEnableIdleTimer"), bEnableTimer, GEngineIni);
	[self EnableIdleTimer : bEnableTimer];
}

-(bool)IsIdleTimerEnabled
{
	__block BOOL Result;
	dispatch_sync(dispatch_get_main_queue(), ^{
		Result = ([UIApplication sharedApplication].idleTimerDisabled == NO);
	});
	return Result;
}

-(void)DeferredEnableIdleTimer
{
	[UIApplication sharedApplication].idleTimerDisabled = NO;
	self.IdleTimerEnableTimer = nil;
}

-(void)EnableIdleTimer:(bool)bEnabled
{
	dispatch_async(dispatch_get_main_queue(),^
	{
		if (bEnabled)
		{
			// Nothing needs to be done, if the enable timer is already running.
			if (self.IdleTimerEnableTimer == nil)
			{
				self.IdleTimerEnableTimer = [NSTimer scheduledTimerWithTimeInterval:IdleTimerEnablePeriod target:self selector:@selector(DeferredEnableIdleTimer) userInfo:nil repeats:NO];
			}
		}
		else
		{
			// Ensure pending attempts to enable the idle timer are cancelled.
			if (self.IdleTimerEnableTimer != nil)
			{
				[self.IdleTimerEnableTimer invalidate];
				self.IdleTimerEnableTimer = nil;
			}

			[UIApplication sharedApplication].idleTimerDisabled = NO;
			[UIApplication sharedApplication].idleTimerDisabled = YES;
		}
	});
}

-(void)NoUrlCommandLine
{
	//Since it is non-repeating, the timer should kill itself.
	self.bCommandLineReady = true;
}

- (void)InitializeAudioSession
{
	[[NSNotificationCenter defaultCenter] addObserverForName:AVAudioSessionInterruptionNotification object:nil queue:nil usingBlock:^(NSNotification *notification)
	{
		NSUInteger InterruptionType = [[[notification userInfo] objectForKey:AVAudioSessionInterruptionTypeKey] unsignedIntegerValue];
		NSNumber* InterruptionOption = [[notification userInfo] objectForKey:AVAudioSessionInterruptionOptionKey];
		UE_CLOGF(InterruptionType == AVAudioSessionInterruptionTypeBegan, LogIOSAudioSession, Log, "AudioSession interruption notification: Began");
		UE_CLOGF(InterruptionType == AVAudioSessionInterruptionTypeEnded, LogIOSAudioSession, Log, "AudioSession interruption notification: Ended");

		[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
		{
			// the audio context should resume immediately after interrupt, if suspended
			FAppEntry::ResetAudioContextResumeTime();
			
			switch (InterruptionType)
			{
				case AVAudioSessionInterruptionTypeBegan:
					// Notification that our audio was stopped by the OS. No action needed.
					FCoreDelegates::AudioInterruptionDelegate.Broadcast(true);
					break;
					
				case AVAudioSessionInterruptionTypeEnded:
					FCoreDelegates::AudioInterruptionDelegate.Broadcast(false);
					
					if (InterruptionOption != nil && InterruptionOption.unsignedIntegerValue > 0)
					{
						FAppEntry::RestartAudio();
					}
					[self ToggleAudioSession:true];
					break;
			}
			return true;
		}];
	}];

	[[NSNotificationCenter defaultCenter] addObserverForName:AVAudioSessionRouteChangeNotification object : nil queue : nil usingBlock : ^ (NSNotification *notification)
	{
		UE_LOGF(LogIOSAudioSession, Log, "AVAudioSessionRouteChange: UserInfo: [%ls] CurrentRoute: [%ls]", *FString([[notification userInfo]description]), *FString([[[AVAudioSession sharedInstance] currentRoute] description]));

		NSUInteger RouteChangeReason = [[[notification userInfo] objectForKey:AVAudioSessionRouteChangeReasonKey] unsignedIntegerValue];
		[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
		{
			switch (RouteChangeReason)
			{
				case AVAudioSessionRouteChangeReasonNewDeviceAvailable:
					// headphones plugged in
					FCoreDelegates::AudioRouteChangedDelegate.Broadcast(true);
					break;
					
				case AVAudioSessionRouteChangeReasonOldDeviceUnavailable:
					// headphones unplugged
					FCoreDelegates::AudioRouteChangedDelegate.Broadcast(false);
					break;
			}
			return true;
		}];
	}];

	self.bAudioSessionInitialized = true;

	self.bUsingBackgroundMusic = [self IsBackgroundAudioPlaying];
	self.bForceEmitOtherAudioPlaying = true;

#if USE_MUTE_SWITCH_DETECTION
	// Initialize the mute switch detector.
	[SharkfoodMuteSwitchDetector shared];
	self.bForceEmitMutedState = true;
#endif
	
	self.bForceEmitVolume = true;
	
	FCoreDelegates::ApplicationRequestAudioState.AddLambda([self]()
		{
			self.bForceEmitOtherAudioPlaying = true;
#if USE_MUTE_SWITCH_DETECTION
			self.bForceEmitMutedState = true;
#endif
			self.bForceEmitVolume = true;
		});
	
	[self ToggleAudioSession:true];
}

- (void)ToggleAudioSession:(bool)bActive
{
	if(!self.bAudioSessionInitialized)
	{
		return;
	}
	
	self.bAudioActive = bActive;
	
	// get the category and settings to use
	NSString* Category = AVAudioSessionCategoryAmbient;
	if([self IsFeatureActive:EAudioFeature::DoNotMixWithOthers])
	{
		Category = AVAudioSessionCategorySoloAmbient;
	}
#if !PLATFORM_TVOS
	bool bSupportsBackgroundAudio = false;
	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsBackgroundAudio"), bSupportsBackgroundAudio, GEngineIni);
	if (bSupportsBackgroundAudio)
	{
		Category = AVAudioSessionCategoryPlayback;
	}
#endif
	
	NSString* Mode = AVAudioSessionModeDefault;
	AVAudioSessionCategoryOptions Options = 0;
	if (self.bAudioActive || [self IsBackgroundAudioPlaying] || [self IsFeatureActive:EAudioFeature::BackgroundAudio])
	{
		// if we are running unattended, do not enable record as this would bring up a mic dialog
		const bool bRecordActive = [self IsFeatureActive:EAudioFeature::Record] && !FApp::IsUnattended();
		if ([self IsFeatureActive:EAudioFeature::Playback] && bRecordActive)
		{
			Category = AVAudioSessionCategoryPlayAndRecord;

			if ([self IsFeatureActive:EAudioFeature::VoiceChat])
			{
				Mode = AVAudioSessionModeVoiceChat;
			}

			if (![self IsFeatureActive:EAudioFeature::DoNotMixWithOthers])
			{
				Options |= AVAudioSessionCategoryOptionMixWithOthers;
			}

#if !PLATFORM_TVOS
			if (![self IsFeatureActive:EAudioFeature::UseReceiver])
			{
				Options |= AVAudioSessionCategoryOptionDefaultToSpeaker;
			}

			if ([self IsFeatureActive:EAudioFeature::BluetoothMicrophone])
			{
				// iOS sdk 26 deprecated AVAudioSessionCategoryOptionAllowBluetooth in favor of AVAudioSessionCategoryOptionAllowBluetoothHFP
				// but AVAudioSessionCategoryOptionAllowBluetoothHFP is not present in sdk 16 despite what the docs/header say
#if (defined(__IPHONE_26_0) && __IPHONE_OS_VERSION_MAX_ALLOWED >= __IPHONE_26_0) || PLATFORM_VISIONOS
				Options |= AVAudioSessionCategoryOptionAllowBluetoothHFP;
#else
				Options |= AVAudioSessionCategoryOptionAllowBluetooth;
#endif
			}
#endif

			if (![self IsFeatureActive:EAudioFeature::DisableBluetoothSpeaker])
			{
				Options |= AVAudioSessionCategoryOptionAllowBluetoothA2DP;
			}
		}
		else if ([self IsFeatureActive:EAudioFeature::Playback])
		{
			Category = AVAudioSessionCategoryPlayback;
			
			if (![self IsFeatureActive:EAudioFeature::DoNotMixWithOthers])
			{
				Options |= AVAudioSessionCategoryOptionMixWithOthers;
			}
		}
		else if (bRecordActive)
		{
			Category = AVAudioSessionCategoryRecord;

#if !PLATFORM_TVOS
			if ([self IsFeatureActive:EAudioFeature::BluetoothMicrophone])
			{
				// iOS sdk 26 deprecated AVAudioSessionCategoryOptionAllowBluetooth in favor of AVAudioSessionCategoryOptionAllowBluetoothHFP
				// but AVAudioSessionCategoryOptionAllowBluetoothHFP is not present in sdk 16 despite what the docs/header say
#if (defined(__IPHONE_26_0) && __IPHONE_OS_VERSION_MAX_ALLOWED >= __IPHONE_26_0) || PLATFORM_VISIONOS
				Options |= AVAudioSessionCategoryOptionAllowBluetoothHFP;
#else
				Options |= AVAudioSessionCategoryOptionAllowBluetooth;
#endif
			}
#endif
		}
	}
	else
	{
		Category = AVAudioSessionCategoryAmbient;
	}

	
	AVAudioSessionCategoryOptions TestOptions = Options;
	if ([[[AVAudioSession sharedInstance] category] compare:AVAudioSessionCategoryAmbient] == NSOrderedSame)
	{
		// This option is implicitly set, so we need to compare the existing options with it
		TestOptions |= AVAudioSessionCategoryOptionMixWithOthers;
	}
	// set the category if anything has changed
	NSError* ActiveError = nil;
	if ([[[AVAudioSession sharedInstance] category] compare:Category] != NSOrderedSame ||
		[[[AVAudioSession sharedInstance] mode] compare:Mode] != NSOrderedSame ||
		[[AVAudioSession sharedInstance] categoryOptions] != TestOptions)
	{
		UE_LOGF(LogIOSAudioSession, Log, "Setting AVAudioSession category to Category:%ls Mode:%ls Options:%llx", *FString(Category), *FString(Mode), Options);

		[[AVAudioSession sharedInstance] setCategory:Category mode:Mode options:Options error:&ActiveError];
		if (ActiveError)
		{
			UE_LOGF(LogIOSAudioSession, Error, "Failed to set AVAudioSession category to Category:%ls (current %ls) Mode:%ls (current %ls) Options:%llx (current %llx)! [Error = %ls].", 
				   *FString(Category), *FString([[AVAudioSession sharedInstance] category]), 
				   *FString(Mode), *FString([[AVAudioSession sharedInstance] mode]), 
				   Options, [[AVAudioSession sharedInstance] categoryOptions] & ~AVAudioSessionCategoryOptionMixWithOthers, 
				   *FString([ActiveError description]));
		}
	}
	
	const bool bAllowHapticsDuringRecording = [self IsFeatureActive:EAudioFeature::AllowHapticsAndSystemSoundsDuringRecording];
	[[AVAudioSession sharedInstance] setAllowHapticsAndSystemSoundsDuringRecording:bAllowHapticsDuringRecording error:nil];
	
	// set the AVAudioSession active if necessary
	if (bActive)
	{
		[[AVAudioSession sharedInstance] setActive:bActive error:&ActiveError];
		if (ActiveError)
		{
			UE_LOGF(LogIOSAudioSession, Error, "Failed to set audio session to active = %d [Error = %ls]", bActive, *FString([ActiveError description]));
		}
	}
}

- (bool)IsBackgroundAudioPlaying
{
	AVAudioSession* Session = [AVAudioSession sharedInstance];
	return Session.otherAudioPlaying;
}

-(bool)HasRecordPermission
{
#if PLATFORM_TVOS
	// TVOS does not have sound recording capabilities.
	return false;
#elif (defined(__IPHONE_17_0) && __IPHONE_OS_VERSION_MAX_ALLOWED >= __IPHONE_17_0)
    return [[AVAudioApplication sharedInstance] recordPermission] == AVAudioApplicationRecordPermissionGranted;
#else
    return [[AVAudioSession sharedInstance] recordPermission] == AVAudioSessionRecordPermissionGranted;
#endif
}

-(void)RequestRecordPermission:(TFunction<void(bool)>)OnComplete
{
#if PLATFORM_TVOS
	// TVOS does not have sound recording capabilities.
	return;
#elif (defined(__IPHONE_17_0) && __IPHONE_OS_VERSION_MAX_ALLOWED >= __IPHONE_17_0)
	[AVAudioApplication requestRecordPermissionWithCompletionHandler:^(BOOL bGranted)
	{
		OnComplete(bGranted);
	}];
#else
	[[AVAudioSession sharedInstance] requestRecordPermission:^(BOOL bGranted)
	{
		OnComplete(bGranted);
	}];
#endif
}

-(FIOSPlatformMisc::EIOSAuthMicrophonePermissionStatus)GetMicrophonePermissionStatus
{
#if PLATFORM_TVOS
	// TVOS does not have sound recording capabilities.
	return FIOSPlatformMisc::EIOSAuthMicrophonePermissionStatus::Denied;
#elif (defined(__IPHONE_17_0) && __IPHONE_OS_VERSION_MAX_ALLOWED >= __IPHONE_17_0)
	switch ([[AVAudioApplication sharedInstance] recordPermission])
	{
	case AVAudioApplicationRecordPermissionUndetermined:
		return FIOSPlatformMisc::EIOSAuthMicrophonePermissionStatus::NotDetermined;
	case AVAudioApplicationRecordPermissionDenied:
		return FIOSPlatformMisc::EIOSAuthMicrophonePermissionStatus::Denied;
	case AVAudioApplicationRecordPermissionGranted:
		return FIOSPlatformMisc::EIOSAuthMicrophonePermissionStatus::Granted;
	default:
		return FIOSPlatformMisc::EIOSAuthMicrophonePermissionStatus::Unknown;
	}
#else
	switch ([[AVAudioSession sharedInstance] recordPermission])
	{
	case AVAudioSessionRecordPermissionUndetermined:
		return FIOSPlatformMisc::EIOSAuthMicrophonePermissionStatus::NotDetermined;
	case AVAudioSessionRecordPermissionDenied:
		return FIOSPlatformMisc::EIOSAuthMicrophonePermissionStatus::Denied;
	case AVAudioSessionRecordPermissionGranted:
		return FIOSPlatformMisc::EIOSAuthMicrophonePermissionStatus::Granted;
	default:
		return FIOSPlatformMisc::EIOSAuthMicrophonePermissionStatus::Unknown;
	}
#endif
}

-(void)EnableHighQualityVoiceChat:(bool)bEnable
{
	[self SetFeature:EAudioFeature::VoiceChat Active:bEnable];
}

- (void)EnableVoiceChat:(bool)bEnable
{
    // mobile will prompt for microphone access
    if (FApp::IsUnattended())
	{
		return;
	}
	if (bEnable != self.bVoiceChatEnabled)
	{
		[self SetFeature:EAudioFeature::Playback Active:bEnable];
		[self SetFeature:EAudioFeature::Record Active:bEnable];
		self.bVoiceChatEnabled = bEnable;
	}
}

- (bool)IsVoiceChatEnabled
{
	return self.bVoiceChatEnabled;
}


- (void)SetFeature:(EAudioFeature)Feature Active:(bool)bIsActive
{
	if (bIsActive)
	{
		++GEnabledAudioFeatures[static_cast<uint8>(Feature)];
		if (GEnabledAudioFeatures[static_cast<uint8>(Feature)] == 1)
		{
			// Setting changed from disabled to enabled, apply the change
			[self ToggleAudioSession:self.bAudioActive];
		}
	}
	else
	{
		if (GEnabledAudioFeatures[static_cast<uint8>(Feature)] == 0)
		{
			UE_LOGF(LogIOSAudioSession, Warning, "Attempted to disable audio feature that is already disabled");
		}
		else
		{
			--GEnabledAudioFeatures[static_cast<uint8>(Feature)];
			if (GEnabledAudioFeatures[static_cast<uint8>(Feature)] == 0)
			{
				// Setting changed from enabled to disabled, apply the change
				[self ToggleAudioSession:self.bAudioActive];
			}
		}
	}
}

- (bool)IsFeatureActive:(EAudioFeature)Feature
{
	return GEnabledAudioFeatures[static_cast<uint8>(Feature)] > 0;
}


- (int)GetAudioVolume
{
	float vol = [[AVAudioSession sharedInstance] outputVolume];
	int roundedVol = (int)((vol * 100.0f) + 0.5f);
	return roundedVol;
}

- (bool)AreHeadphonesPluggedIn
{
	AVAudioSessionRouteDescription *route = [[AVAudioSession sharedInstance] currentRoute];

	bool headphonesFound = false;
	for (AVAudioSessionPortDescription *portDescription in route.outputs)
	{
		//compare to the iOS constant for headphones
		if ([portDescription.portType isEqualToString : AVAudioSessionPortHeadphones])
		{
			headphonesFound = true;
			break;
		}
	}
	return headphonesFound;
}

- (int)GetBatteryLevel
{
#if PLATFORM_TVOS
	// TVOS does not have a battery, return fully charged
	return 100;
#else
    return self.BatteryLevel;
#endif
}

- (bool)IsRunningOnBattery
{
#if PLATFORM_TVOS
	// TVOS does not have a battery, return plugged in
	return false;
#else
    return self.bBatteryState;
#endif
}

- (void)LoadScreenResolutionModifiers
{
#if PLATFORM_VISIONOS
	self.ScreenScale = 1.0f;
	self.NativeScale = 1.0f;
#else
	// cache these UI thread sensitive vars for later use
	self.ScreenScale = (float)[[UIScreen mainScreen] scale];
	self.NativeScale = (float)[[UIScreen mainScreen] nativeScale];
#endif

	// need to cache the MobileContentScaleFactor for framebuffer creation.
	static IConsoleVariable* CVarScale = IConsoleManager::Get().FindConsoleVariable(TEXT("r.MobileContentScaleFactor"));
	check(CVarScale);
	self.MobileContentScaleFactor = CVarScale ? CVarScale->GetFloat() : 0;

	// Can also be overridden from the commandline using "mcsf="
	FString CmdLineCSF;
	if (FParse::Value(FCommandLine::Get(), TEXT("mcsf="), CmdLineCSF, false))
	{
		self.MobileContentScaleFactor = FCString::Atof(*CmdLineCSF);
	}

	static IConsoleVariable* CVarResX = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.DesiredResX"));
	static IConsoleVariable* CVarResY = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.DesiredResY"));
	check(CVarResX);
	check(CVarResY);
		
	self.RequestedResX = CVarResX ? CVarResX->GetInt() : 0;
	self.RequestedResY = CVarResY ? CVarResY->GetInt() : 0;
		
	static bool bOnFirstUse = true;
	if (bOnFirstUse)
	{
		FString CmdLineMDRes;
		if (FParse::Value(FCommandLine::Get(), TEXT("mobileresx="), CmdLineMDRes, false))
		{
				self.RequestedResX = FCString::Atoi(*CmdLineMDRes);
		}
		if (FParse::Value(FCommandLine::Get(), TEXT("mobileresy="), CmdLineMDRes, false))
		{
				self.RequestedResY = FCString::Atoi(*CmdLineMDRes);
		}

		CVarScale->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&FIOSWindow::OnScaleFactorChanged));
		CVarResX->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&FIOSWindow::OnConsoleResolutionChanged));
		CVarResY->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&FIOSWindow::OnConsoleResolutionChanged));
			
		bOnFirstUse = false;
	}
}

- (void)CheckForZoomAccessibility
{
#if !PLATFORM_TVOS
    // warn about zoom conflicting
    UIAccessibilityRegisterGestureConflictWithZoom();
#endif
}

- (float)GetBackgroundingMainThreadBlockTime
{
	return GOverrideThreadWaitTime > 0.0f ? GOverrideThreadWaitTime : GMaxThreadWaitTime;
}

-(void)OverrideBackgroundingMainThreadBlockTime:(float)BlockTime
{
	GOverrideThreadWaitTime = BlockTime;
}

-(NSString*)GetPendingActivationProtocol
{
	return self.PendingActivationProtocol;
}

-(void)SetPendingActivationProtocol:(NSString *)NewProtocol
{
	static FTSTicker::FDelegateHandle ProtocolActivationTickerHandle;
	self.PendingActivationProtocol = NewProtocol;

	if ([self.PendingActivationProtocol length] != 0)
	{
		if (!ProtocolActivationTickerHandle.IsValid())
		{
			ProtocolActivationTickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
				[self](float)
				{
					if (FCoreDelegates::OnActivatedByProtocol.IsBound())
					{
						FCoreDelegates::OnActivatedByProtocol.Broadcast(FString([[IOSAppDelegate GetDelegate] PendingActivationProtocol]), PLATFORMUSERID_NONE);

						self.PendingActivationProtocol = @"";
						ProtocolActivationTickerHandle.Reset();
						return false; // remove from ticker
					}

					return true; // try again next tick
				}
			));
		}
	}
	else if (ProtocolActivationTickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(ProtocolActivationTickerHandle);
		ProtocolActivationTickerHandle.Reset();
	}
}

- (NSProcessInfoThermalState)GetThermalState
{
    return self.ThermalState;
}

bool GIsSuspended = 0;
- (void)ToggleSuspend:(bool)bSuspend
{
    self.bHasSuspended = !bSuspend;
    self.bIsSuspended = bSuspend;
    GIsSuspended = self.bIsSuspended;

	if (bSuspend)
	{
		FAppEntry::Suspend();
	}
	else
	{
        FIOSPlatformRHIFramePacer::Resume();
		FAppEntry::Resume();

		// Check rotation state after returning from background
		[self didRotate:nil];
	}
    
	if (IOSView && IOSView->bIsInitialized)
	{
		// Don't deadlock here because a msg box may appear super early blocking the game thread and then the app may go into the background
		double	startTime = FPlatformTime::Seconds();

		// don't wait for FDefaultGameMoviePlayer::WaitForMovieToFinish(), crash with 0x8badf00d if "Wait for Movies to Complete" is checked
		FEmbeddedCommunication::KeepAwake(TEXT("Background"), false);
		while(!self.bHasSuspended && !FAppEntry::IsStartupMoviePlaying() &&  (FPlatformTime::Seconds() - startTime) < [self GetBackgroundingMainThreadBlockTime])
		{
            FIOSPlatformRHIFramePacer::Suspend();
			FPlatformProcess::Sleep(0.05f);
		}
		FEmbeddedCommunication::AllowSleep(TEXT("Background"));
	}
}

- (void)ForceExit
{
	RequestEngineExit(TEXT("IOS ForceExit"));
    bForceExit = true;
}

- (BOOL)application:(UIApplication *)application willFinishLaunchingWithOptions:(NSDictionary *)launchOptions
{
	self.bDeviceInPortraitMode = false;
	self.bEngineInit = false;
	self.bPlatformInit = false;

	return YES;
}

static int32 GEnableThermalsReport = 0;
static FAutoConsoleVariableRef CVarGEnableThermalsReport(
	TEXT("ios.EnableThermalsReport"),
	GEnableThermalsReport,
	TEXT("When set to 1, will enable on-screen thermals debug display.")
);

- (BOOL)application:(UIApplication *)application didFinishLaunchingWithOptions:(NSDictionary*)launchOptions
{
	// save launch options
	self.launchOptions = launchOptions;

#if PLATFORM_TVOS
	self.bDeviceInPortraitMode = false;
#else
	// use the status bar orientation to properly determine landscape vs portrait
	self.InterfaceOrientation = [self.Window.windowScene interfaceOrientation];
	self.bDeviceInPortraitMode = UIInterfaceOrientationIsPortrait(self.InterfaceOrientation);
	printf("========= This app is in %s mode\n", self.bDeviceInPortraitMode ? "PORTRAIT" : "LANDSCAPE");

#if !PLATFORM_VISIONOS
	// Register for device orientation changes
	[[UIDevice currentDevice] beginGeneratingDeviceOrientationNotifications];
    [[NSNotificationCenter defaultCenter] addObserver:self selector:@selector(didRotate:) name:UIDeviceOrientationDidChangeNotification object:nil];
#endif
#endif

	// check OS version to make sure we have the API
	OSVersion = [[[UIDevice currentDevice] systemVersion] floatValue];
	if (!FPlatformMisc::IsDebuggerPresent() || GAlwaysReportCrash)
	{
//        InstallSignalHandlers();
	}

	self.savedOpenUrlParameters = [[NSMutableArray alloc] init];
	self.PeakMemoryTimer = [NSTimer scheduledTimerWithTimeInterval:0.1f target:self selector:@selector(RecordPeakMemory) userInfo:nil repeats:YES];

#if !BUILD_EMBEDDED_APP
#if !UE_IOS_SCENE_LIFECYCLE
    // Legacy UIApplicationDelegate path: the app delegate owns the window and starts the game
    // thread here. In scene mode, IOSSceneDelegate.scene:willConnectToSession:options: takes
    // over this work because the UIWindow must be initialized against a UIWindowScene.

#if PLATFORM_VISIONOS
    CGRect MainFrame = CGRectMake(0, 0, 1000, 1000);
#else
	CGRect MainFrame = [[UIScreen mainScreen] bounds];
#endif
    self.Window = [[UIWindow alloc] initWithFrame:MainFrame];

    [self.Window makeKeyAndVisible];

    FAppEntry::PreInit(self, application);

#if !PLATFORM_VISIONOS
    UIStoryboard *storyboard = [UIStoryboard storyboardWithName:@"LaunchScreen" bundle:nil];
    if (storyboard != nil)
    {
        self.viewController = [storyboard instantiateViewControllerWithIdentifier:@"LaunchScreen"];
        self.viewController.view.tag = 200;
        [self.Window addSubview: self.viewController.view];
        GShowSplashScreen = true;
    }
#else
	GShowSplashScreen = false;
#endif

    timer = [NSTimer scheduledTimerWithTimeInterval: 0.05f target:self selector:@selector(timerForSplashScreen) userInfo:nil repeats:YES];

	[self StartGameThread];

	self.CommandLineParseTimer = [NSTimer scheduledTimerWithTimeInterval:0.01f target:self selector:@selector(NoUrlCommandLine) userInfo:nil repeats:NO];
#endif // !UE_IOS_SCENE_LIFECYCLE
#endif // !BUILD_EMBEDDED_APP
	
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
	UNUserNotificationCenter *Center = [UNUserNotificationCenter currentNotificationCenter];
	Center.delegate = self;

#if !UE_BUILD_SHIPPING
	// make a history buffer
	self.ConsoleHistoryValues = [[NSMutableArray alloc] init];

	// load saved history from disk
	NSArray* SavedHistory = [[NSUserDefaults standardUserDefaults] objectForKey:@"ConsoleHistory"];
	if (SavedHistory != nil)
	{
		[self.ConsoleHistoryValues addObjectsFromArray:SavedHistory];
	}
	self.ConsoleHistoryValuesIndex = -1;

    FCoreDelegates::OnGetOnScreenMessages.AddLambda([&EnableThermalsReport = GEnableThermalsReport](TMultiMap<FCoreDelegates::EOnScreenMessageSeverity, FText >& OutMessages)
    {
        if (EnableThermalsReport)
        {
            switch ([[NSProcessInfo processInfo] thermalState])
            {
                case NSProcessInfoThermalStateNominal:	OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(TEXT("Thermals are Nominal"))); break;
                case NSProcessInfoThermalStateFair:		OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(TEXT("Thermals are Fair"))); break;
                case NSProcessInfoThermalStateSerious:	OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Warning, FText::FromString(TEXT("Thermals are Serious"))); break;
                case NSProcessInfoThermalStateCritical:	OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Error, FText::FromString(TEXT("Thermals are Critical"))); break;
            }
        }
        
        // Uncomment to view the state of the AVAudioSession category, mode, and options.
        //#define VIEW_AVAUDIOSESSION_INFO
#if defined(VIEW_AVAUDIOSESSION_INFO)
        FString Message = FString::Printf(
                                          TEXT("Session Category: %s, Mode: %s, Options: %x"),
                                          UTF8_TO_TCHAR([[AVAudioSession sharedInstance].category UTF8String]),
                                          UTF8_TO_TCHAR([[AVAudioSession sharedInstance].mode UTF8String]),
                                          [AVAudioSession sharedInstance].categoryOptions);
        OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(Message));
#endif // defined(VIEW_AVAUDIOSESSION_INFO)
    });
    

#endif // UE_BUILD_SHIPPING
#endif // !TVOS

#if !PLATFORM_TVOS
    UIDevice* UiDevice = [UIDevice currentDevice];
    UiDevice.batteryMonitoringEnabled = YES;
    
    // Battery level is from 0.0 to 1.0, get it in terms of 0-100
    self.BatteryLevel = ((int)([UiDevice batteryLevel] * 100));
    UIDeviceBatteryState State = UiDevice.batteryState;
    self.bBatteryState = State == UIDeviceBatteryStateUnplugged || State == UIDeviceBatteryStateUnknown;
    self.ThermalState = [[NSProcessInfo processInfo] thermalState];
    
    [[NSNotificationCenter defaultCenter] addObserver:self selector:@selector(temperatureChanged:) name:NSProcessInfoThermalStateDidChangeNotification object:nil];
    [[NSNotificationCenter defaultCenter] addObserver:self selector:@selector(lowPowerModeChanged:) name:NSProcessInfoPowerStateDidChangeNotification object:nil];
    [[NSNotificationCenter defaultCenter] addObserver:self selector:@selector(batteryChanged:) name:UIDeviceBatteryLevelDidChangeNotification object:nil];
    [[NSNotificationCenter defaultCenter] addObserver:self selector:@selector(batteryStateChanged:) name:UIDeviceBatteryStateDidChangeNotification object:nil];
#endif
    
    self.bAudioSessionInitialized = false;
	
	// InitializeAudioSession is now called from FEngineLoop::AppInit after the config system is initialized
//	[self InitializeAudioSession];

#if WITH_ACCESSIBILITY
	[[NSNotificationCenter defaultCenter] addObserver:self selector:@selector(OnVoiceOverStatusChanged) name:UIAccessibilityVoiceOverStatusDidChangeNotification object:nil];
#endif

	return YES;
}

#if WITH_IOS_UNIVERSAL_LINKS
- (BOOL)handleContinueUserActivity:(NSUserActivity *)userActivity
{
	if ([userActivity.activityType isEqualToString:NSUserActivityTypeBrowsingWeb])
	{
		[self SetPendingActivationProtocol:userActivity.webpageURL.absoluteString];
		return YES;
	}
	return NO;
}

#if !UE_IOS_SCENE_LIFECYCLE
-(BOOL)application:(UIApplication*)application continueUserActivity:(NSUserActivity*)userActivity restorationHandler:(void(^)(NSArray<id<UIUserActivityRestoring>>* _Nullable)) restorationHandler
{
	return [self handleContinueUserActivity:userActivity];
}
#endif
#endif // WITH_IOS_UNIVERSAL_LINKS

#if !PLATFORM_TVOS
- (UIInterfaceOrientationMask)application:(UIApplication *)application supportedInterfaceOrientationsForWindow:(UIWindow*)window
{
	static bool bInitialized = false;
	static UIInterfaceOrientationMask Mask = 0;
	
	if (!bInitialized)
	{
		bInitialized = true;
		
		bool bSupportsPortrait;
		bool bSupportsPortraitUpsideDown;
		bool bSupportsLandscapeLeft;
		bool bSupportsLandscapeRight;
		
		// This is called during app startup and IOSRuntimeSettings may not have been loaded yet
		bool hasValue = GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsPortraitOrientation"), bSupportsPortrait, GEngineIni);
		
		NSArray<NSString*> *SupportedOrientations = [[NSBundle mainBundle] objectForInfoDictionaryKey:@"UISupportedInterfaceOrientations"];
		if (!hasValue && SupportedOrientations != NULL)
		{
			// Loop through the Info.plist UISupportedInterfaceOrientations array values looking for "Portrait", "Left" and "Right"
			NSPredicate *predicate = [NSPredicate predicateWithFormat:@"SELF == %@", @"UIInterfaceOrientationPortrait"];
			bSupportsPortrait = ([SupportedOrientations filteredArrayUsingPredicate:predicate].count > 0);
			
			NSPredicate *predicateDown = [NSPredicate predicateWithFormat:@"SELF == %@", @"UIInterfaceOrientationPortraitUpsideDown"];
			bSupportsPortraitUpsideDown = ([SupportedOrientations filteredArrayUsingPredicate:predicateDown].count > 0);
			
			NSPredicate *PredicateLeft = [NSPredicate predicateWithFormat:@"SELF == %@", @"UIInterfaceOrientationLandscapeLeft"];
			bSupportsLandscapeLeft = ([SupportedOrientations filteredArrayUsingPredicate:PredicateLeft].count > 0);
			
			NSPredicate *PredicateRight = [NSPredicate predicateWithFormat:@"SELF == %@", @"UIInterfaceOrientationLandscapeRight"];
			bSupportsLandscapeRight = ([SupportedOrientations filteredArrayUsingPredicate:PredicateRight].count > 0);
		}
		else
		{
			GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsUpsideDownOrientation"), bSupportsPortraitUpsideDown, GEngineIni);
			GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsLandscapeLeftOrientation"), bSupportsLandscapeLeft, GEngineIni);
			GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsLandscapeRightOrientation"), bSupportsLandscapeRight, GEngineIni);
		}
		
		if (bSupportsPortrait)
		{
			Mask |= UIInterfaceOrientationMaskPortrait;
		}
		if (bSupportsPortraitUpsideDown)
		{
			Mask |= UIInterfaceOrientationMaskPortraitUpsideDown;
		}
		if (bSupportsLandscapeLeft)
		{
			Mask |= UIInterfaceOrientationMaskLandscapeLeft;
		}
		if (bSupportsLandscapeRight)
		{
			Mask |= UIInterfaceOrientationMaskLandscapeRight;
		}
	}
	
	const FIOSView *View = [[IOSAppDelegate GetDelegate] IOSView];
	if (View != nil &&
		(View->SupportedInterfaceOrientations != UIInterfaceOrientationMaskAll) &&
		(View->SupportedInterfaceOrientations & Mask) != View->SupportedInterfaceOrientations)
	{
		// Return all possible allowed orientations, that includes all orientations from IOSRuntimeSettings plus
		// the orientations set at runtime via Blueprint. The iOS View Controller will then restrict to the desired
		// orientation(s) with the "supportedInterfaceOrientations" function.
		return Mask | View->SupportedInterfaceOrientations;
	}
	// If no orientation constraints are set, default to MaskAll
	return (Mask==0)?UIInterfaceOrientationMaskAll:Mask;
}
#endif

#if WITH_ACCESSIBILITY
-(void)OnVoiceOverStatusChanged
{
	if (UIAccessibilityIsVoiceOverRunning() && self.IOSApplication->GetAccessibleMessageHandler()->ApplicationIsAccessible())
	{
		// This must happen asynchronously because when the app activates from a suspended state,
		// the IOS notification will emit before the game thread wakes up. This does mean that the
		// accessibility element tree will probably not be 100% completed when the application
		// opens for the first time. If this is a problem we can add separate branches for startup
		// vs waking up.
		FFunctionGraphTask::CreateAndDispatchWhenReady([]()
		{
			FIOSApplication* Application = [IOSAppDelegate GetDelegate].IOSApplication;
			Application->GetAccessibleMessageHandler()->SetActive(true);
			AccessibleWidgetId WindowId = Application->GetAccessibleMessageHandler()->GetAccessibleWindowId(Application->FindWindowByAppDelegateView());
			dispatch_async(dispatch_get_main_queue(), ^{
                IOSAppDelegate* Delegate = [IOSAppDelegate GetDelegate];
				[Delegate.IOSView SetAccessibilityWindow:WindowId];
				if (Delegate.AccessibilityCacheTimer == nil)
				{
					// Start caching accessibility data so that it can be returned instantly to IOS. If not cached, the data takes too long
					// to retrieve due to cross-thread waiting and IOS will timeout.
					Delegate.AccessibilityCacheTimer = [NSTimer scheduledTimerWithTimeInterval:0.25f target:[FIOSAccessibilityCache AccessibilityElementCache] selector:@selector(UpdateAllCachedProperties) userInfo:nil repeats:YES];
				}
			});
		}, TStatId(), NULL, ENamedThreads::GameThread);
	}
	else if (AccessibilityCacheTimer != nil)
	{
		[AccessibilityCacheTimer invalidate];
		AccessibilityCacheTimer = nil;
		[[IOSAppDelegate GetDelegate].IOSView SetAccessibilityWindow : IAccessibleWidget::InvalidAccessibleWidgetId];
		[[FIOSAccessibilityCache AccessibilityElementCache] Clear];
		FFunctionGraphTask::CreateAndDispatchWhenReady([]()
		{
			[IOSAppDelegate GetDelegate].IOSApplication->GetAccessibleMessageHandler()->SetActive(false);
		}, TStatId(), NULL, ENamedThreads::GameThread);
	}
}
#endif

// checking for update on the app store
-(void)DoUpdateCheck
{
	static bool bInit = false;
	static NSString* CurrentVersion = [[NSBundle mainBundle] infoDictionary][@"CFBundleShortVersionString"];
	static NSString* BundleID = [[NSBundle mainBundle] infoDictionary][@"CFBundleIdentifier"];
	bool bEnableUpdateCheck = NO;
	if (!bInit)
	{
		bool bReadData = GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bEnableUpdateCheck"), bEnableUpdateCheck, GEngineIni);
		self.bUpdateAvailable = false;
		bInit = bReadData;
	}
	if (bEnableUpdateCheck && bInit)
	{
		// kick off a check on the app store for an update
		NSLocale* Locale = [NSLocale autoupdatingCurrentLocale];
		NSURL* StoreURL = [NSURL URLWithString: [NSString stringWithFormat: @"http://itunes.apple.com/%@/lookup?bundleId=%@", Locale.countryCode, BundleID]];
		
		// kick off an NSURLSession to read the data
		NSURLSession* Session = [NSURLSession sharedSession];
		NSURLSessionDataTask* SessionTask = [Session dataTaskWithRequest: [NSURLRequest requestWithURL: StoreURL cachePolicy: NSURLRequestReloadIgnoringLocalCacheData timeoutInterval: Session.configuration.timeoutIntervalForRequest] completionHandler:^(NSData* _Nullable data, NSURLResponse* _Nullable response, NSError* _Nullable error) {
			
			if (error == nil && data != nil)
			{
				NSDictionary* StoreDictionary = [NSJSONSerialization JSONObjectWithData: data options: 0 error: nil];
			
				if ([StoreDictionary[@"resultCount"] integerValue] == 1)
				{
					// get the store version
					NSString* StoreVersion = StoreDictionary[@"results"][0][@"version"];
					if ([StoreVersion compare: CurrentVersion options: NSNumericSearch] == NSOrderedDescending)
					{
						self.bUpdateAvailable = true;
					}
					else
					{
						self.bUpdateAvailable = false;
					}
				}
				else
				{
					self.bUpdateAvailable = false;
				}
			}
			else
			{
				self.bUpdateAvailable = false;
			}
		}];
		
		[SessionTask resume];
	}
}

-(bool)IsUpdateAvailable
{
	dispatch_async(dispatch_get_main_queue(), ^{
		[[IOSAppDelegate GetDelegate] DoUpdateCheck];
	});
	return self.bUpdateAvailable;
}

- (void) StartGameThread
{
	// create a new thread (the pointer will be retained forever)
	NSThread* GameThread = [[NSThread alloc] initWithTarget:self selector:@selector(MainAppThread:) object:self.launchOptions];
	[GameThread setStackSize:GAME_THREAD_STACK_SIZE];
	[GameThread start];
	[GameThread setName:@"GameThread"];

	// this can be slow (1/3 of a second!), so don't make the game thread stall loading for it
	// check to see if we are using the network file system, if so, disable the idle timer
	FString HostIP;
//	if (FParse::Value(FCommandLine::Get(), TEXT("-FileHostIP="), HostIP))
	{
		[UIApplication sharedApplication].idleTimerDisabled = YES;
	}
}

+(bool)WaitAndRunOnGameThread:(TUniqueFunction<void()>)Function
{
	FGraphEventRef Task = FFunctionGraphTask::CreateAndDispatchWhenReady(MoveTemp(Function), TStatId(), NULL, ENamedThreads::GameThread);

	const double MaxThreadWaitTime = 2.0;
	const double StartTime = FPlatformTime::Seconds();
	while ((FPlatformTime::Seconds() - StartTime) < MaxThreadWaitTime)
	{
		FPlatformProcess::Sleep(0.05f);
		if (Task->IsComplete())
		{
			return true;
		}
	}
	return false;
}

- (void) didRotate:(NSNotification *)notification
{   
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
	// get the interface orientation
	
	if(GIsSuspended)
	{
		return;
	}

	if(!self.bPlatformInit)
	{
#if IOS_ROTATION_DEBUG_LOGGING
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[rotation] didRotate ignoring call before bPlatformInit self.InterfaceOrientation=%d windowScene=%d device=%d\n"), self.InterfaceOrientation, (int)[self.Window.windowScene interfaceOrientation], (int)[UIDevice currentDevice].orientation);
#endif
		return;
	}

#if IOS_ROTATION_DEBUG_LOGGING
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[rotation] didRotate self.InterfaceOrientation=%d windowScene=%d device=%d\n"), self.InterfaceOrientation, (int)[self.Window.windowScene interfaceOrientation], (int)[UIDevice currentDevice].orientation);
#endif

	UIInterfaceOrientationMask PotentialMask;
	UIInterfaceOrientation NewOrientation = self.InterfaceOrientation;
	bool bForceUpdate = false;
	if (GIOSDelayRotationUntilPresent)
	{
		// Force a refresh if the current and cached orientations are not the same. This can happen at startup.
		bForceUpdate = [self.Window.windowScene interfaceOrientation] != self.InterfaceOrientation;

		UIInterfaceOrientationMask AllowedMask = [self.IOSController supportedInterfaceOrientations_Internal];

		// See if we support the current device orientation
		UIInterfaceOrientation PotentialOrientation = [IOSAppDelegate GetUIInterfaceOrientationFromUIDeviceOrientation: [UIDevice currentDevice].orientation];	
		PotentialMask = [IOSAppDelegate GetMaskFromUIInterfaceOrientation: PotentialOrientation];
		if (PotentialOrientation != UIInterfaceOrientationUnknown && (PotentialMask & AllowedMask) != 0)
		{
#if IOS_ROTATION_DEBUG_LOGGING
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[rotation] didRotate: using device orientation\n"));
#endif
			NewOrientation = PotentialOrientation;
		}
		else
		{
			// If not, check if we support the current windowScene interfaceOrientation. It could have been programmatically set.
			PotentialOrientation = [self.Window.windowScene interfaceOrientation];
			PotentialMask = [IOSAppDelegate GetMaskFromUIInterfaceOrientation: PotentialOrientation];
			if ((PotentialMask & AllowedMask) != 0)
			{
				NewOrientation = PotentialOrientation;
#if IOS_ROTATION_DEBUG_LOGGING
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[rotation] didRotate: using windowScene interfaceOrientation\n"));
#endif
			}
			else
			{			
				// Force rotation to the first allowed orientation
				if ((AllowedMask & UIInterfaceOrientationMaskLandscape) == UIInterfaceOrientationMaskLandscape)
				{
					FString PreferredLandscapeOrientation;
					GConfig->GetString(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("PreferredLandscapeOrientation"), PreferredLandscapeOrientation, GEngineIni);
					
					// Preferred default is LandscapeRight as per 
					if (PreferredLandscapeOrientation.Equals("LandscapeLeft"))
					{
						NewOrientation =  UIInterfaceOrientationLandscapeLeft;
						PotentialMask = UIInterfaceOrientationMaskLandscapeLeft;
					}
					else
					{
						NewOrientation =  UIInterfaceOrientationLandscapeRight;
						PotentialMask = UIInterfaceOrientationMaskLandscapeRight;
					}
				}
				else if (AllowedMask & UIInterfaceOrientationMaskLandscapeRight)
				{
					NewOrientation =  UIInterfaceOrientationLandscapeRight;
					PotentialMask = UIInterfaceOrientationMaskLandscapeRight;
				}
				else if (AllowedMask & UIInterfaceOrientationMaskLandscapeLeft)
				{
					NewOrientation =  UIInterfaceOrientationLandscapeLeft;
					PotentialMask = UIInterfaceOrientationMaskLandscapeLeft;
				}
				else if (AllowedMask & UIInterfaceOrientationMaskPortrait)
				{
					NewOrientation = UIInterfaceOrientationPortrait;
					PotentialMask = UIInterfaceOrientationMaskPortrait;
				}
				else if (AllowedMask & UIInterfaceOrientationMaskPortraitUpsideDown)
				{
					NewOrientation = UIInterfaceOrientationPortraitUpsideDown;
					PotentialMask = UIInterfaceOrientationMaskPortraitUpsideDown;
				}
#if IOS_ROTATION_DEBUG_LOGGING
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[rotation] didRotate: using first allowed orientation %d mask %d (AllowedMask was %d)\n"), NewOrientation, PotentialMask, AllowedMask);
#endif
			}
		}
	}
	else
	{
		NewOrientation = [self.Window.windowScene interfaceOrientation];
	}

	if (self.InterfaceOrientation != NewOrientation || bForceUpdate)
	{
		static int32 Count=0;
		int LocalCount = ++Count;
#if IOS_ROTATION_DEBUG_LOGGING
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[rotation] didRotate: rotating to new orientation %d, count %d\n"), (int32)NewOrientation, LocalCount);
#endif

		self.InterfaceOrientation = NewOrientation;
		FIOSApplication* Application = self.IOSApplication;
		
		FFunctionGraphTask::CreateAndDispatchWhenReady([NewOrientation, Application, LocalCount]()
		{
#if IOS_ROTATION_DEBUG_LOGGING
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[rotation] didRotate: calling OrientationChanged(%d) from GT, count %d"), (int32)NewOrientation, LocalCount);
#endif
			Application->OrientationChanged(NewOrientation);
		}, TStatId(), NULL, ENamedThreads::GameThread);
	}
#endif
}

- (BOOL)handleOpenURL:(NSURL *)url sourceApplication:(NSString *)sourceApplication annotation:(id)annotation
{
#if !NO_LOGGING
	NSLog(@"%s", "IOSAppDelegate openURL\n");
#endif

	if (FString(url.scheme) == FApp::GetName())
	{
		NSString* EncdodedURLString = [url absoluteString];
		NSString* URLString = [EncdodedURLString stringByRemovingPercentEncoding];
		FString CommandLineParameters(URLString);
		
		// Strip the "URL" part of the URL before treating this like args. It comes in looking like so:
		// "MyGame://arg1 arg2 arg3 ..."
		// So, we're going to make it look like:
		// "arg1 arg2 arg3 ..."
		int32 URLTerminator = CommandLineParameters.Find( TEXT("://"), ESearchCase::CaseSensitive);
		if ( URLTerminator > -1 )
		{
			CommandLineParameters.RightChopInline(URLTerminator + 3, EAllowShrinking::No);
		}
		
		FIOSCommandLineHelper::InitCommandArgs(CommandLineParameters);
		self.bCommandLineReady = true;
		[self.CommandLineParseTimer invalidate];
		self.CommandLineParseTimer = nil;
	}

	UIApplication* application = [UIApplication sharedApplication];

	//    Save openurl information before engine initialize.
	//    When engine is done ready, running like previous. ( if OnOpenUrl is bound on game source. )
	if (self.bEngineInit)
	{
		FIOSCoreDelegates::OnOpenURL.Broadcast(application, url, sourceApplication, annotation);
	}
	else
	{
#if !NO_LOGGING
		NSLog(@"%s", "Before Engine Init receive IOSAppDelegate openURL\n");
#endif
		NSDictionary* openUrlParameter = [NSDictionary dictionaryWithObjectsAndKeys :
			application, @"application",
			url, @"url",
			sourceApplication, @"sourceApplication",
			annotation, @"annotation",
			nil];

		[savedOpenUrlParameters addObject : openUrlParameter];
	}

	return YES;
}

#if !UE_IOS_SCENE_LIFECYCLE
- (BOOL)application:(UIApplication *)application openURL:(NSURL *)url sourceApplication:(NSString *)sourceApplication annotation:(id)annotation
{
	return [self handleOpenURL:url sourceApplication:sourceApplication annotation:annotation];
}
#endif

- (void)handleWillResignActive
{
    /*
		Sent when the application is about to move from active to inactive
		state. This can occur for certain types of temporary interruptions (such
		as an incoming phone call or SMS message) or when the user quits the
		application and it begins the transition to the background state.

		Use this method to pause ongoing tasks, disable timers, and throttle
	 	down OpenGL ES frame rates. Games should use this method to pause the
		game.
	 */
	UE::GC::Private::SetSkipDestroy(true);
    if (self.bEngineInit)
    {
 		FEmbeddedCommunication::KeepAwake(TEXT("Background"), false);
        FGraphEventRef ResignTask = FFunctionGraphTask::CreateAndDispatchWhenReady([]()
        {
            FCoreDelegates::ApplicationWillDeactivateDelegate.Broadcast();
            FEmbeddedCommunication::AllowSleep(TEXT("Background"));
        }, TStatId(), NULL, ENamedThreads::GameThread);
		
		// Do not wait forever for this task to complete since the game thread may be stuck on waiting for user input from a modal dialog box
		double	startTime = FPlatformTime::Seconds();
		while((FPlatformTime::Seconds() - startTime) < [self GetBackgroundingMainThreadBlockTime])
		{
			FPlatformProcess::Sleep(0.05f);
			if(ResignTask->IsComplete())
			{
				UE_LOGF(LogTemp, Display, "Task was completed before time.");
				break;
			}
		}
		UE_LOGF(LogTemp, Display, "Done with entering background tasks time.");
    }
    
// fix for freeze on tvOS, moving to applicationDidEnterBackground. Not making the changes for iOS platforms as the bug does not happen and could bring some side effets.
#if !PLATFORM_TVOS
	
	bool bSupportsBackgroundAudio = false;
	if (GConfig)
	{
		GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsBackgroundAudio"), bSupportsBackgroundAudio, GEngineIni);
	}
    if (!bSupportsBackgroundAudio)
    {
        [self ToggleSuspend:true];
        [self ToggleAudioSession:false];
    }
#else
    [self ToggleAudioSession:false];
#endif
	
#if WITH_ENGINE
	// Suspend all future rendering
	FViewport::SetGameRenderingEnabled(false);
#endif
	
#if !PLATFORM_TVOS
	FAppEntry::Suspend(true);
#endif
    
#if WITH_ENGINE
	if (FTaskGraphInterface::IsRunning() && self.bEngineInit)
	{
		FGraphEventRef RTBlockTask = FFunctionGraphTask::CreateAndDispatchWhenReady([]()
																					{
			FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
			if (!RHICmdList.Bypass()) 
			{
				RHICmdList.SubmitAndBlockUntilGPUIdle();
			}
		}, TStatId(), NULL, ENamedThreads::GetRenderThread());
		
		RTBlockTask->Wait();
	}
#endif
	
	__block UIBackgroundTaskIdentifier BackgroundTaskId = UIBackgroundTaskInvalid;	
	BackgroundTaskId = [[UIApplication sharedApplication] beginBackgroundTaskWithExpirationHandler:^{
		UE_LOGF(LogIOS, Warning, "Background task expired before completing FIOSCoreDelegates::OnWillResignActive broadcast");
		[[UIApplication sharedApplication] endBackgroundTask: BackgroundTaskId];
	}];
	
	FIOSCoreDelegates::OnWillResignActive.Broadcast();
	
	[[UIApplication sharedApplication] endBackgroundTask: BackgroundTaskId];
}

#if !UE_IOS_SCENE_LIFECYCLE
- (void)applicationWillResignActive:(UIApplication *)application
{
	[self handleWillResignActive];
}
#endif

- (void)handleDidEnterBackground
{
	/*
	 Use this method to release shared resources, save user data, invalidate
	 timers, and store enough application state information to restore your
	 application to its current state in case it is terminated later.
	 
	 If your application supports background execution, this method is called
	 instead of applicationWillTerminate: when the user quits.
	 */

    // fix for freeze on tvOS, moving to applicationDidEnterBackground. Not making the changes for iOS platforms as the bug does not happen and could bring some side effets.
#if PLATFORM_TVOS
    [self ToggleSuspend:true];
#endif

    self.bAudioActive = false;
#if PLATFORM_TVOS
	FAppEntry::Suspend(true);
#endif
	FBackgroundURLSessionHandler::HandleDidEnterBackground();

	FEmbeddedCommunication::KeepAwake(TEXT("Background"), false);

	__block UIBackgroundTaskIdentifier BackgroundTaskId = UIBackgroundTaskInvalid;	
	BackgroundTaskId = [[UIApplication sharedApplication] beginBackgroundTaskWithExpirationHandler:^{
		UE_LOGF(LogIOS, Warning, "Background task expired before completing FCoreDelegates::ApplicationWillEnterBackgroundDelegate broadcast");
        [[UIApplication sharedApplication] endBackgroundTask: BackgroundTaskId];
    }];
	
	[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
	{
		// the audio context should resume immediately after interrupt, if suspended
		FAppEntry::ResetAudioContextResumeTime();

		FCoreDelegates::ApplicationWillEnterBackgroundDelegate.Broadcast();
	
		FEmbeddedCommunication::AllowSleep(TEXT("Background"));
        [[UIApplication sharedApplication] endBackgroundTask: BackgroundTaskId];
		return true;
	}];
}

#if !UE_IOS_SCENE_LIFECYCLE
- (void)applicationDidEnterBackground:(UIApplication *)application
{
	[self handleDidEnterBackground];
}
#endif

- (void)handleWillEnterForeground
{
	FBackgroundURLSessionHandler::HandleWillEnterForeground();
	FEmbeddedCommunication::KeepAwake(TEXT("Background"), false);

	/*
	 Called as part of the transition from the background to the inactive state; here you can undo many of the changes made on entering the background.
	 */
	[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
	{
		// the audio context should resume immediately after interrupt, if suspended
		FAppEntry::ResetAudioContextResumeTime();

		FCoreDelegates::ApplicationHasEnteredForegroundDelegate.Broadcast();

		FEmbeddedCommunication::AllowSleep(TEXT("Background"));
		return true;
	}];
}

#if !UE_IOS_SCENE_LIFECYCLE
- (void)applicationWillEnterForeground:(UIApplication *)application
{
	[self handleWillEnterForeground];
}
#endif

extern double GCStartTime;
- (void)handleDidBecomeActive
{
#if !UE_BUILD_SHIPPING
	if (FParse::Param(FCommandLine::Get(), TEXT("metalshadervalidation")))
	{
		FPlatformMisc::SetEnvironmentVar(TEXT("MTL_SHADER_VALIDATION"), TEXT("1"));
		FPlatformMisc::SetEnvironmentVar(TEXT("MTL_SHADER_VALIDATION_DEFAULT_STATE"), TEXT("none"));
	}
	
	if (FParse::Param(FCommandLine::Get(), TEXT("metalapivalidation")))
	{
		FPlatformMisc::SetEnvironmentVar(TEXT("MTL_DEBUG_LAYER"), TEXT("1"));
	}
#endif
	
	FIOSCoreDelegates::OnDidBecomeActive.Broadcast();
	
	UE::GC::Private::SetSkipDestroy(false);

	// make sure a GC will not timeout because it was started before entering background
    GCStartTime = FPlatformTime::Seconds();
	/*
	 Restart any tasks that were paused (or not yet started) while the application was inactive. If the application was previously in the background, optionally refresh the user interface.
	 */
	
#if WITH_ENGINE
	// Resume Rendering
	FViewport::SetGameRenderingEnabled(true);
#endif
	
	[self ToggleSuspend : false];
    [self ToggleAudioSession:true];
	FAppEntry::RestartAudio();
    FAppEntry::Resume(true);


    if (self.bEngineInit)
    {
		FEmbeddedCommunication::KeepAwake(TEXT("Background"), false);

       FGraphEventRef ResignTask = FFunctionGraphTask::CreateAndDispatchWhenReady([]()
       {
		   
            FCoreDelegates::ApplicationHasReactivatedDelegate.Broadcast();
		
			FEmbeddedCommunication::AllowSleep(TEXT("Background"));
        }, TStatId(), NULL, ENamedThreads::GameThread);

		// Do not wait forever for this task to complete since the game thread may be stuck on waiting for user input from a modal dialog box
		double	startTime = FPlatformTime::Seconds();
 		while((FPlatformTime::Seconds() - startTime) < [self GetBackgroundingMainThreadBlockTime])
		{
			FPlatformProcess::Sleep(0.05f);
			if(ResignTask->IsComplete())
			{
				break;
			}
		}
    }
}

#if !UE_IOS_SCENE_LIFECYCLE
- (void)applicationDidBecomeActive:(UIApplication *)application
{
	[self handleDidBecomeActive];
}
#endif

- (void)applicationWillTerminate:(UIApplication *)application
{
	/*
	 Called when the application is about to terminate.
	 Save data if appropriate.
	 See also applicationDidEnterBackground:.
	 */
	if (!UE::IsGCLocked())
	{
		FCoreDelegates::GetApplicationWillTerminateDelegate().Broadcast();
	}
    
    // note that we are shutting down
    // TODO: fix the reason why we are hanging when asked to shutdown
/*    RequestEngineExit(TEXT("IOS applicationWillTerminate"));
    
    if (!self.bEngineInit)*/
    {
        // we haven't yet made it to the point where the engine is initialized, so just exit the app
        _Exit(0);
    }
/*    else
    {
        // wait for the game thread to shut down
        while (self.bHasStarted == true)
        {
            usleep(3);
        }
    }*/
}

- (void)applicationDidReceiveMemoryWarning:(UIApplication *)application
{
	/*
	Tells the delegate when the application receives a memory warning from the system
	*/
	FPlatformMisc::HandleLowMemoryWarning();
}

#if !PLATFORM_TVOS && BACKGROUNDFETCH_ENABLED // NOTE: TVOS can do this starting in tvOS 11
- (void)application:(UIApplication *)application performFetchWithCompletionHandler:(void(^)(UIBackgroundFetchResult result))completionHandler
{
    // NOTE: the completionHandler must be called within 30 seconds
    FCoreDelegates::ApplicationPerformFetchDelegate.Broadcast();
    completionHandler(UIBackgroundFetchResultNewData);
}

#endif

#if !PLATFORM_TVOS
- (void)application:(UIApplication *)application handleEventsForBackgroundURLSession:(NSString *)identifier completionHandler:(void(^)(void))completionHandler
{
    //Save off completionHandler so that a future call to FCoreDelegates::ApplicationBackgroundSessionEventsAllSentDelegate can execute it
	[FBackgroundURLSessionHandler::BackgroundSessionEventCompleteDelegateMap setObject:completionHandler forKey:identifier];
    
    // Invoke background session with this identifier if needed to handle these events
	FString Id(identifier);
	FBackgroundURLSessionHandler::HandleEventsForBackgroundURLSession(Id);

	FCoreDelegates::ApplicationBackgroundSessionEventDelegate.Broadcast(Id);
}
#endif

#if !PLATFORM_TVOS

-(void)application:(UIApplication *)application didReceiveRemoteNotification:(NSDictionary *)userInfo fetchCompletionHandler:(void (^)(UIBackgroundFetchResult))completionHandler
{
	UE_LOGF(LogTemp, Display, "Received RemoteNotification");
	completionHandler(UIBackgroundFetchResultNoData);
}

- (void)application:(UIApplication *)application didRegisterForRemoteNotificationsWithDeviceToken:(NSData *)deviceToken
{
	if (FApp::IsUnattended())
	{
		return;
	}

	TArray<uint8> Token;
	Token.AddUninitialized([deviceToken length]);
	memcpy(Token.GetData(), [deviceToken bytes], [deviceToken length]);

	const char *data = (const char*)([deviceToken bytes]);
	NSMutableString *token = [NSMutableString string];

	for (NSUInteger i = 0; i < [deviceToken length]; i++) {
		[token appendFormat : @"%02.2hhX", data[i]];
	}

	UE_LOGF(LogTemp, Display, "Device Token: %ls", *FString(token));

    FFunctionGraphTask::CreateAndDispatchWhenReady([Token]()
    {
		FCoreDelegates::ApplicationRegisteredForRemoteNotificationsDelegate.Broadcast(Token);
    }, TStatId(), NULL, ENamedThreads::GameThread);
}

-(void)application:(UIApplication *)application didFailToRegisterForRemoteNotificationsWithError:(NSError *)error
{
	FString errorDescription([error description]);
	
	UE_LOGF(LogTemp, Warning, "FAILED to register for remote notificaiton: %ls", *errorDescription);
    FFunctionGraphTask::CreateAndDispatchWhenReady([errorDescription]()
    {
		FCoreDelegates::ApplicationFailedToRegisterForRemoteNotificationsDelegate.Broadcast(errorDescription);
    }, TStatId(), NULL, ENamedThreads::GameThread);
}

#endif

#if !PLATFORM_TVOS && !PLATFORM_VISIONOS

+(EDeviceScreenOrientation) ConvertFromUIInterfaceOrientation:(UIInterfaceOrientation)Orientation
{
	switch(Orientation)
	{
		default:
		case UIInterfaceOrientationUnknown : return EDeviceScreenOrientation::Unknown;
		case UIInterfaceOrientationPortrait : return EDeviceScreenOrientation::Portrait;
		case UIInterfaceOrientationPortraitUpsideDown : return EDeviceScreenOrientation::PortraitUpsideDown;
		case UIInterfaceOrientationLandscapeLeft : return EDeviceScreenOrientation::LandscapeLeft;
		case UIInterfaceOrientationLandscapeRight : return EDeviceScreenOrientation::LandscapeRight;
	}
}

+(UIInterfaceOrientation) GetUIInterfaceOrientationFromUIDeviceOrientation:(UIDeviceOrientation)Orientation
{
	switch(Orientation)
	{
		default:
		case UIDeviceOrientationUnknown : return UIInterfaceOrientationUnknown;
		case UIDeviceOrientationPortrait : return UIInterfaceOrientationPortrait;
		case UIDeviceOrientationPortraitUpsideDown : return UIInterfaceOrientationPortraitUpsideDown;
		// Note UIDeviceOrientation and UIInterfaceOrientation have have lansdcape left and right switched
		// https://developer.apple.com/documentation/uikit/uiinterfaceorientation
		case UIDeviceOrientationLandscapeLeft : return UIInterfaceOrientationLandscapeRight;
		case UIDeviceOrientationLandscapeRight : return UIInterfaceOrientationLandscapeLeft;
	}
}

+(UIInterfaceOrientationMask) GetMaskFromUIInterfaceOrientation:(UIInterfaceOrientation)Orientation
{
	switch(Orientation)
	{
		default:
		case UIInterfaceOrientationUnknown : return UIInterfaceOrientationMaskAll;
		case UIInterfaceOrientationPortrait : return UIInterfaceOrientationMaskPortrait;
		case UIInterfaceOrientationPortraitUpsideDown : return UIInterfaceOrientationMaskPortraitUpsideDown;
		case UIInterfaceOrientationLandscapeLeft : return UIInterfaceOrientationMaskLandscapeLeft;
		case UIInterfaceOrientationLandscapeRight : return UIInterfaceOrientationMaskLandscapeRight;
	}
}

+(UIInterfaceOrientationMask) GetMaskFromEDeviceScreenOrientation:(EDeviceScreenOrientation)Orientation
{
	switch(Orientation)
	{
		default:
		case EDeviceScreenOrientation::Unknown : return UIInterfaceOrientationMaskAll;
		case EDeviceScreenOrientation::Portrait : return UIInterfaceOrientationMaskPortrait;
		case EDeviceScreenOrientation::PortraitUpsideDown : return UIInterfaceOrientationMaskPortraitUpsideDown;
		case EDeviceScreenOrientation::LandscapeLeft : return UIInterfaceOrientationMaskLandscapeLeft;
		case EDeviceScreenOrientation::LandscapeRight : return UIInterfaceOrientationMaskLandscapeRight;
	}
}

+(void)UpdateSupportedInterfaceOrientations;
{
	dispatch_async(dispatch_get_main_queue(),
	^{
		if (@available(iOS 16.0, visionOS 1.0, *))
		{
			[[IOSAppDelegate GetDelegate].IOSController setNeedsUpdateOfSupportedInterfaceOrientations];
		}
		else
		{
// Deprecated iOS16 - suppress warning if deployment target	>= iOS16
#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wdeprecated-declarations"
			[UIViewController attemptRotationToDeviceOrientation];
#pragma clang diagnostic pop
		}
	});
}

static bool HandleReceivedNotification(UNNotification* notification)
{
	bool IsLocal = false;
	bool IsReadyToHandle = [IOSAppDelegate GetDelegate].bEngineInit;
	if (IsReadyToHandle)
	{
		NSString* NotificationType = (NSString*)[notification.request.content.userInfo objectForKey: @"NotificationType"];
		if(NotificationType != nullptr)
		{
			FString LocalOrRemote(NotificationType);
			if(LocalOrRemote == FString("Local"))
			{
				IsLocal = true;
			}
		}
		
		int AppState;
		if ([UIApplication sharedApplication].applicationState == UIApplicationStateInactive)
		{
			AppState = 1; // EApplicationState::Inactive;
		}
		else if ([UIApplication sharedApplication].applicationState == UIApplicationStateBackground)
		{
			AppState = 2; // EApplicationState::Background;
		}
		else
		{
			AppState = 3; // EApplicationState::Active;
		}
		
		if(IsLocal)
		{
			NSString*	activationEvent = (NSString*)[notification.request.content.userInfo objectForKey: @"ActivationEvent"];
			if(activationEvent != nullptr)
			{
				FString	activationEventFString(activationEvent);
				int32	fireDate = FMath::TruncToInt([notification.date timeIntervalSince1970]);
				
				FFunctionGraphTask::CreateAndDispatchWhenReady([activationEventFString, fireDate, AppState]()
															   {
																   FCoreDelegates::ApplicationReceivedLocalNotificationDelegate.Broadcast(activationEventFString, fireDate, AppState);
															   }, TStatId(), NULL, ENamedThreads::GameThread);
			}
		}
		else
		{
			NSString* JsonString = @"{}";
			NSError* JsonError;
			NSData* JsonData = [NSJSONSerialization dataWithJSONObject : notification.request.content.userInfo
															   options : 0
																 error : &JsonError];
			
			if (JsonData)
			{
				JsonString = [[[NSString alloc] initWithData:JsonData encoding : NSUTF8StringEncoding] autorelease];
			}
			
			FString	jsonFString(JsonString);
			
			FFunctionGraphTask::CreateAndDispatchWhenReady([jsonFString, AppState]()
														   {
															   FCoreDelegates::ApplicationReceivedRemoteNotificationDelegate.Broadcast(jsonFString, AppState);
														   }, TStatId(), NULL, ENamedThreads::GameThread);
		}
	}
	return IsReadyToHandle;
}

- (void)userNotificationCenter:(UNUserNotificationCenter *)center
	   willPresentNotification:(UNNotification *)notification
		 withCompletionHandler:(void (^)(UNNotificationPresentationOptions options))completionHandler
{
	// Received notification while app is in the foreground
	[IOSNotificationHandler handleWhenReady:notification withCompletionHandler:^{ completionHandler(UNNotificationPresentationOptionNone); }];
}

- (void)userNotificationCenter:(UNUserNotificationCenter *)center
didReceiveNotificationResponse:(UNNotificationResponse *)response
		 withCompletionHandler:(void (^)())completionHandler
{
	// Received notification while app is in the background or closed
	
	// Save launch local notification so the app can check for it when it is ready
	NSDictionary* userInfo = response.notification.request.content.userInfo;
	if(userInfo != nullptr)
	{
		NSString*	activationEvent = (NSString*)[userInfo objectForKey: @"ActivationEvent"];
		
		if(activationEvent != nullptr)
		{
			FAppEntry::gAppLaunchedWithLocalNotification = true;
			FAppEntry::gLaunchLocalNotificationActivationEvent = FString(activationEvent);
			FAppEntry::gLaunchLocalNotificationFireDate = FMath::TruncToInt([response.notification.date timeIntervalSince1970]);
		}
		
		// Try to find the activation protocol
		id ActivationProtocolId = userInfo[(NSString*)FPlatformString::TCHARToCFString(*GNotificationActivationProtocolKey)];
		if ([ActivationProtocolId isKindOfClass:[NSString class]])
		{
			NSString* ActivationProtocol = ActivationProtocolId;
			[self SetPendingActivationProtocol:ActivationProtocol];
		}
	}
	
	[IOSNotificationHandler handleWhenReady:response.notification withCompletionHandler:completionHandler];
}

#else
static bool HandleReceivedNotification(UNNotification* notification) { return true; }
#endif

/**
 * Shows the given Game Center supplied controller on the screen
 *
 * @param Controller The Controller object to animate onto the screen
 */
-(void)ShowController:(UIViewController*)Controller
{
	// slide it onto the screen
	[[IOSAppDelegate GetDelegate].IOSController presentViewController : Controller animated : YES completion : nil];
	
	// stop drawing the 3D world for faster UI speed
	//FViewport::SetGameRenderingEnabled(false);
}

/**
 * Hides the given Game Center supplied controller from the screen, optionally controller
 * animation (sliding off)
 *
 * @param Controller The Controller object to animate off the screen
 * @param bShouldAnimate YES to slide down, NO to hide immediately
 */
-(void)HideController:(UIViewController*)Controller Animated : (BOOL)bShouldAnimate
{
	// slide it off
	[Controller dismissViewControllerAnimated : bShouldAnimate completion : nil];

	// stop drawing the 3D world for faster UI speed
	//FViewport::SetGameRenderingEnabled(true);
}

/**
 * Hides the given Game Center supplied controller from the screen
 *
 * @param Controller The Controller object to animate off the screen
 */
-(void)HideController:(UIViewController*)Controller
{
	// call the other version with default animation of YES
	[self HideController : Controller Animated : YES];
}

-(void)gameCenterViewControllerDidFinish:(GKGameCenterViewController*)GameCenterDisplay
{
	// close the view 
	[self HideController : GameCenterDisplay];
}

/**
 * Show the leaderboard interface (call from iOS main thread)
 */
-(void)ShowLeaderboard:(NSString*)Category
{
	// create the leaderboard display object 
	GKGameCenterViewController* GameCenterDisplay = [[[GKGameCenterViewController alloc] init] autorelease];
    [GameCenterDisplay initWithState:GKGameCenterViewControllerStateLeaderboards];
	GameCenterDisplay.gameCenterDelegate = self;

	// show it 
	[self ShowController : GameCenterDisplay];
}

/**
 * Show the achievements interface (call from iOS main thread)
 */
-(void)ShowAchievements
{
#if !PLATFORM_TVOS
	// create the leaderboard display object 
	GKGameCenterViewController* GameCenterDisplay = [[[GKGameCenterViewController alloc] init] autorelease];
    [GameCenterDisplay initWithState : GKGameCenterViewControllerStateAchievements];
	GameCenterDisplay.gameCenterDelegate = self;

	// show it 
	[self ShowController : GameCenterDisplay];
#endif // !PLATFORM_TVOS
}

/**
 * Show the dashboard interface (call from iOS main thread)
 */
-(void)ShowDashboard
{
	// create the dashboard display object
	GKGameCenterViewController* GameCenterDisplay = [[[GKGameCenterViewController alloc] initWithState:GKGameCenterViewControllerStateDashboard] autorelease];
	GameCenterDisplay.gameCenterDelegate = self;

	// show it
		[self ShowController : GameCenterDisplay];
}

/**
 * Show the leaderboard interface (call from game thread)
 */
CORE_API bool IOSShowLeaderboardUI(const FString& CategoryName)
{
	// route the function to iOS thread, passing the category string along as the object
	NSString* CategoryToShow = [NSString stringWithFString : CategoryName];
	[[IOSAppDelegate GetDelegate] performSelectorOnMainThread:@selector(ShowLeaderboard : ) withObject:CategoryToShow waitUntilDone : NO];

	return true;
}

/**
* Show the achievements interface (call from game thread)
*/
CORE_API bool IOSShowAchievementsUI()
{

	// route the function to iOS thread
	[[IOSAppDelegate GetDelegate] performSelectorOnMainThread:@selector(ShowAchievements) withObject:nil waitUntilDone : NO];

	return true;
}

/**
 * Show the dashboard interface (call from game thread)
 */
CORE_API bool IOSShowDashboardUI()
{
	// route the function to iOS thread
	[[IOSAppDelegate GetDelegate] performSelectorOnMainThread:@selector(ShowDashboard) withObject:nil waitUntilDone : NO];

	return true;
}

-(void)batteryChanged:(NSNotification*)notification
{
#if !PLATFORM_TVOS
    UIDevice* Device = [UIDevice currentDevice];
    
    // Battery level is from 0.0 to 1.0, get it in terms of 0-100
    self.BatteryLevel = ((int)([Device batteryLevel] * 100));
	[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
	{
		UE_LOGF(LogIOS, Display, "Battery Level Changed: %d", self.BatteryLevel);
		return true;
	}];
#endif
}

-(void)batteryStateChanged:(NSNotification*)notification
{
#if !PLATFORM_TVOS
    UIDevice* Device = [UIDevice currentDevice];
    UIDeviceBatteryState State = Device.batteryState;
    self.bBatteryState = State == UIDeviceBatteryStateUnplugged || State == UIDeviceBatteryStateUnknown;
	[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
	{
		UE_LOGF(LogIOS, Display, "Battery State Changed: %d", self.bBatteryState);
		return true;
	}];
#endif
}

-(void)temperatureChanged:(NSNotification *)notification
{
#if !PLATFORM_TVOS
    // send game callback with new temperature severity
    FCoreDelegates::ETemperatureSeverity Severity;
    FString Level = "Unknown";
    self.ThermalState = [[NSProcessInfo processInfo] thermalState];
    switch (self.ThermalState)
    {
        case NSProcessInfoThermalStateNominal:	Severity = FCoreDelegates::ETemperatureSeverity::Good; Level = TEXT("Good"); break;
        case NSProcessInfoThermalStateFair:		Severity = FCoreDelegates::ETemperatureSeverity::Bad; Level = TEXT("Bad"); break;
        case NSProcessInfoThermalStateSerious:	Severity = FCoreDelegates::ETemperatureSeverity::Serious; Level = TEXT("Serious"); break;
        case NSProcessInfoThermalStateCritical:	Severity = FCoreDelegates::ETemperatureSeverity::Critical; Level = TEXT("Critical"); break;
    }
    
    [FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
     {
        UE_LOGF(LogIOS, Display, "Temperature Changed: %ls", *Level);
        FCoreDelegates::OnTemperatureChange.Broadcast(Severity);
        return true;
    }];
#endif
}

-(void)lowPowerModeChanged:(NSNotification *)notification
{
#if !PLATFORM_TVOS
    [FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
     {
        bool bInLowPowerMode = [[NSProcessInfo processInfo] isLowPowerModeEnabled];
        UE_LOGF(LogIOS, Display, "Low Power Mode Changed: %d", bInLowPowerMode);
        FCoreDelegates::OnLowPowerMode.Broadcast(bInLowPowerMode);
        return true;
    }];
#endif
}

-(UIWindow*)window
{
	return Window;
}

@end

@implementation IOSNotificationHandler
{
	UNNotification* notification;
	void (^completionHandler)();
	int retryCount;
}

+(void)handleWhenReady:(UNNotification*)notification withCompletionHandler:(void (^)()) completionHandler
{
	[[[[IOSNotificationHandler alloc] initWithNotification:notification andCompletionHandler:completionHandler] autorelease] handle];
}

-(id)initWithNotification:(UNNotification*)_notification andCompletionHandler:(void (^)()) _completionHandler
{
	self = [super init];
	if(self)
	{
		completionHandler = [_completionHandler copy];
		notification = [_notification retain];
		retryCount = 0;
	}
	return self;
}

-(void)dealloc
{
	[completionHandler release];
	[notification release];
	[super dealloc];
}

-(void)handle
{
	if (HandleReceivedNotification(notification))
	{
		completionHandler();
	}
	// retry for up to 10s
	else if(++retryCount <= 100)
	{
		[self performSelector:@selector(handle) withObject:nil afterDelay:0.1];
	}
	else
	{
		UE_LOGF(LogIOS, Warning, "Failed to deliver notification after retrying for 10 seconds.");

	}
}
@end
