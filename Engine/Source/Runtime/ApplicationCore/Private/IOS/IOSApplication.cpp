// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOS/IOSApplication.h"

#include "Apple/AppleStringUtils.h"
#include "IOS/IOSInputInterface.h"
#include "IOS/IOSCursor.h"
#include "IOS/IOSView.h"
#include "IOSWindow.h"
#include "Misc/CoreDelegates.h"
#include "IOS/IOSAppDelegate.h"
#include "IInputDeviceModule.h"
#include "GenericPlatform/IInputInterface.h"
#include "IInputDevice.h"
#include "Apple/AppleMouseController.h"
#include "Misc/ScopeLock.h"
#include "HAL/IConsoleManager.h"
#include "IOS/IOSAsyncTask.h"
#include "Stats/Stats.h"
#if WITH_ACCESSIBILITY
#include "IOS/Accessibility/IOSAccessibilityCache.h"
#include "IOS/Accessibility/IOSAccessibilityElement.h"
#endif

FIOSApplication* FIOSApplication::CreateIOSApplication()
{
	SCOPED_BOOT_TIMING("FIOSApplication::CreateIOSApplication");
	return new FIOSApplication();
}

FIOSApplication::FIOSApplication()
	: GenericApplication(MakeShareable(new FIOSCursor()))
	, InputInterface( FIOSInputInterface::Create( MessageHandler ) )
	, bHasLoadedInputPlugins(false)
#if WITH_ACCESSIBILITY
	, AccessibilityAnnouncementDelayTimer(Nil)
#endif
{
	[IOSAppDelegate GetDelegate].IOSApplication = this;
	
	if (TSharedPtr<FAppleKeyboardController> KeyboardController = InputInterface->GetKeyboardController().Pin())
	{
		KeyboardController->OnAvailabilityChanged().AddRaw(this, &FIOSApplication::HandlePhysicalKeyboardConnectionChanged);
		
		if (KeyboardController->IsAnyKeyboardConnected())
		{
			HandlePhysicalKeyboardConnectionChanged();
		}
	}
	
	if (TSharedPtr<FAppleMouseController> MouseController = InputInterface->GetMouseController().Pin())
	{
		MouseController->OnAvailabilityChanged().AddRaw(this, &FIOSApplication::HandlePhysicalMouseConnectionChanged);
		MouseController->OnMovementDispatched().AddRaw(this, &FIOSApplication::HandleMouseMovement);
		
		if (MouseController->IsAnyMouseConnected())
		{
			HandlePhysicalMouseConnectionChanged();
		}
	}
}

FIOSApplication::~FIOSApplication()
{
	IModularFeatures::Get().OnModularFeatureRegistered().RemoveAll(this);

#if WITH_ACCESSIBILITY
	if (AccessibilityAnnouncementDelayTimer)
	{
		[AccessibilityAnnouncementDelayTimer invalidate ];
		[AccessibilityAnnouncementDelayTimer release ];
	}
#endif
	
	if (TSharedPtr<FAppleKeyboardController> KeyboardController = InputInterface->GetKeyboardController().Pin())
	{
		KeyboardController->OnAvailabilityChanged().RemoveAll(this);
	}
	
	if (TSharedPtr<FAppleMouseController> MouseController = InputInterface->GetMouseController().Pin())
	{
		MouseController->OnAvailabilityChanged().RemoveAll(this);
		MouseController->OnMovementDispatched().RemoveAll(this);
	}
}

void FIOSApplication::HandlePhysicalKeyboardConnectionChanged()
{
	IOSViewController* ViewController = [[IOSAppDelegate GetDelegate] IOSController];
	
	if (!ensure(ViewController))
	{
		return;
	}

	if (InputInterface->IsAnyPhysicalKeyboardConnected())
	{
		ViewController.controllerUserInteractionEnabled = false;
		[ViewController becomeFirstResponder];
		UE_LOGF(LogIOS, Log, "[%s] External keyboard connected.", __func__);
	}
	else
	{
		ViewController.controllerUserInteractionEnabled = true;
		[ViewController resignFirstResponder];

		UE_LOGF(LogIOS, Log, "[%s] All external keyboards disconnected", __func__);
	}
}

void FIOSApplication::HandlePhysicalMouseConnectionChanged()
{
	bool bWasAttached = bIsMouseAttached;
	TSharedPtr<FAppleMouseController> MouseController = InputInterface->GetMouseController().Pin();
	if (MouseController && MouseController->IsAnyMouseConnected())
	{	
		bIsMouseAttached = true;
		UE_LOGF(LogIOS, Log, "[%s] External Mouse connected.", __func__);
	}
	else
	{
		bIsMouseAttached = false;
		UE_LOGF(LogIOS, Log, "[%s] All external mice disconnected", __func__);
	}
	
	if (bWasAttached != bIsMouseAttached)
	{
		Cursor->Show(bIsMouseAttached);
		MessageHandler->OnCursorSet();
	}
}

void FIOSApplication::HandleMouseMovement(FIntVector2 MovementDelta)
{
	const FVector2D TargetPos = Cursor->GetPosition() + FVector2D(MovementDelta);
	Cursor->SetPosition(FMath::TruncToInt(TargetPos.X), FMath::TruncToInt(TargetPos.Y));
}

FModifierKeysState FIOSApplication::GetModifierKeys() const
{ 
	if (TSharedPtr<FAppleKeyboardController> KeyboardController = InputInterface->GetKeyboardController().Pin())
	{
		return KeyboardController->GetModifierKeys();
	}

	return FModifierKeysState(); 
}

bool FIOSApplication::IsMouseAttached() const
{
	return bIsMouseAttached;
}

void FIOSApplication::InitializeWindow( const TSharedRef< FGenericWindow >& InWindow, const TSharedRef< FGenericWindowDefinition >& InDefinition, const TSharedPtr< FGenericWindow >& InParent, const bool bShowImmediately )
{
	const TSharedRef< FIOSWindow > Window = StaticCastSharedRef< FIOSWindow >( InWindow );
	const TSharedPtr< FIOSWindow > ParentWindow = StaticCastSharedPtr< FIOSWindow >( InParent );

	Windows.Add( Window );
	
	dispatch_sync(dispatch_get_main_queue(), ^{
		Window->Initialize( this, InDefinition, ParentWindow, bShowImmediately );
	});
}

void FIOSApplication::SetMessageHandler( const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler )
{
	GenericApplication::SetMessageHandler(InMessageHandler);
	InputInterface->SetMessageHandler(InMessageHandler);

	TArray<IInputDeviceModule*> PluginImplementations = IModularFeatures::Get().GetModularFeatureImplementations<IInputDeviceModule>(IInputDeviceModule::GetModularFeatureName());
	for (auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt)
	{
		(*DeviceIt)->SetMessageHandler(InMessageHandler);
	}
}

#if WITH_ACCESSIBILITY
void FIOSApplication::SetAccessibleMessageHandler(const TSharedRef<FGenericAccessibleMessageHandler>& InAccessibleMessageHandler)
{
	GenericApplication::SetAccessibleMessageHandler(InAccessibleMessageHandler);
	// This user is what IOS Voiceover will interact with
	FGenericAccessibleUserRegistry& UserRegistry = InAccessibleMessageHandler->GetAccessibleUserRegistry();
	// We failed to register the primary user, this should only happen if another user with the 0th index has already been registered.
	ensure(UserRegistry.RegisterUser(MakeShared<FGenericAccessibleUser>(FGenericAccessibleUserRegistry::GetPrimaryUserIndex())));
	InAccessibleMessageHandler->SetAccessibleEventDelegate(FGenericAccessibleMessageHandler::FAccessibleEvent::CreateRaw(this, &FIOSApplication::OnAccessibleEventRaised));
	InAccessibleMessageHandler->SetActive(UIAccessibilityIsVoiceOverRunning());
}
#endif

void FIOSApplication::AddExternalInputDevice(TSharedPtr<IInputDevice> InputDevice)
{
	if (InputDevice.IsValid())
	{
		ExternalInputDevices.Add(MoveTemp(InputDevice));
	}
}

void FIOSApplication::PollGameDeviceState( const float TimeDelta )
{
	@autoreleasepool
	{
		// Create all externally-implemented input devices from loaded modules.
		if (!bHasLoadedInputPlugins)
		{
			TArray<IInputDeviceModule*> PluginImplementations = IModularFeatures::Get().GetModularFeatureImplementations<IInputDeviceModule>(IInputDeviceModule::GetModularFeatureName());
			for (auto InputPluginIt = PluginImplementations.CreateIterator(); InputPluginIt; ++InputPluginIt)
			{
				TSharedPtr<IInputDevice> Device = (*InputPluginIt)->CreateInputDevice(MessageHandler);
				AddExternalInputDevice(MoveTemp(Device));
			}

			// Modules loaded after this point are handled by this callback
			IModularFeatures::Get().OnModularFeatureRegistered().AddRaw(this, &FIOSApplication::OnInputDeviceModuleRegistered);

			bHasLoadedInputPlugins = true;
		}

		// Poll game device state and send new events
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_IOSApplication_InputInterface_Tick);
			InputInterface->Tick(TimeDelta);
		}
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_IOSApplication_InputInterface_SendControllerEvents);
			InputInterface->SendControllerEvents();
		}

		// Poll externally-implemented devices
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_IOSApplication_ExternalInputDevice);
			for (auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt)
			{
				(*DeviceIt)->Tick(TimeDelta);
				(*DeviceIt)->SendControllerEvents();
			}
		}
	}
}

FPlatformRect FIOSApplication::GetWorkArea( const FPlatformRect& CurrentWindow ) const
{
	__block FPlatformRect Rect;
	dispatch_sync(dispatch_get_main_queue(), ^{
		Rect = FIOSWindow::GetScreenRect();
	});
	return Rect;
}

static TAutoConsoleVariable<float> CVarSafeZone_Landscape_Left(TEXT("SafeZone.Landscape.Left"), -1.0f, TEXT("Safe Zone - Landscape - Left"));
static TAutoConsoleVariable<float> CVarSafeZone_Landscape_Top(TEXT("SafeZone.Landscape.Top"), -1.0f, TEXT("Safe Zone - Landscape - Top"));
static TAutoConsoleVariable<float> CVarSafeZone_Landscape_Right(TEXT("SafeZone.Landscape.Right"), -1.0f, TEXT("Safe Zone - Landscape - Right"));
static TAutoConsoleVariable<float> CVarSafeZone_Landscape_Bottom(TEXT("SafeZone.Landscape.Bottom"), -1.0f, TEXT("Safe Zone - Landscape - Bottom"));

#if !PLATFORM_TVOS
UIInterfaceOrientation FIOSApplication::CachedOrientation = UIInterfaceOrientationPortrait;
UIEdgeInsets CachedInsets;
#endif

void FDisplayMetrics::RebuildDisplayMetrics(FDisplayMetrics& OutDisplayMetrics)
{
	FPlatformRect Rect;
	CGFloat ContentScaleFactor;
	Rect = FIOSWindow::GetScreenRect();
	const FIOSView *View = [[IOSAppDelegate GetDelegate] IOSView];
	[View CalculateContentScaleFactor:Rect.Right ScreenHeight:Rect.Bottom];
	ContentScaleFactor = View.contentScaleFactor;
	
	// Get screen rect
	OutDisplayMetrics.PrimaryDisplayWorkAreaRect = Rect;
	OutDisplayMetrics.VirtualDisplayRect = OutDisplayMetrics.PrimaryDisplayWorkAreaRect;

    // Total screen size of the primary monitor
    OutDisplayMetrics.PrimaryDisplayWidth = OutDisplayMetrics.PrimaryDisplayWorkAreaRect.Right - OutDisplayMetrics.PrimaryDisplayWorkAreaRect.Left;
    OutDisplayMetrics.PrimaryDisplayHeight = OutDisplayMetrics.PrimaryDisplayWorkAreaRect.Bottom - OutDisplayMetrics.PrimaryDisplayWorkAreaRect.Top;
    
#if !PLATFORM_TVOS
    //we need to set these according to the orientation
    TAutoConsoleVariable<float>* CVar_Left = nullptr;
    TAutoConsoleVariable<float>* CVar_Top = nullptr;
    TAutoConsoleVariable<float>* CVar_Right = nullptr;
    TAutoConsoleVariable<float>* CVar_Bottom = nullptr;
    
    //making an assumption that the "normal" landscape mode is Landscape right
    if (FIOSApplication::CachedOrientation == UIInterfaceOrientationLandscapeLeft)
    {
        CVar_Left = &CVarSafeZone_Landscape_Left;
        CVar_Right = &CVarSafeZone_Landscape_Right;
        CVar_Top = &CVarSafeZone_Landscape_Top;
        CVar_Bottom = &CVarSafeZone_Landscape_Bottom;
    }
    else if (FIOSApplication::CachedOrientation == UIInterfaceOrientationLandscapeRight)
    {
        CVar_Left = &CVarSafeZone_Landscape_Right;
        CVar_Right = &CVarSafeZone_Landscape_Left;
        CVar_Top = &CVarSafeZone_Landscape_Top;
        CVar_Bottom = &CVarSafeZone_Landscape_Bottom;
    }
    
    // of the CVars are set, use their values. If not, use what comes from iOS
    const double Inset_Left = (!CVar_Left || CVar_Left->AsVariable()->GetFloat() < 0.0f) ? CachedInsets.left : CVar_Left->AsVariable()->GetFloat();
    const double Inset_Top = (!CVar_Top || CVar_Top->AsVariable()->GetFloat() < 0.0f) ? CachedInsets.top : CVar_Top->AsVariable()->GetFloat();
    const double Inset_Right = (!CVar_Right || CVar_Right->AsVariable()->GetFloat() < 0.0f) ? CachedInsets.right : CVar_Right->AsVariable()->GetFloat();
    const double Inset_Bottom = (!CVar_Bottom || CVar_Bottom->AsVariable()->GetFloat() < 0.0f) ? CachedInsets.bottom : CVar_Bottom->AsVariable()->GetFloat();
    
    //setup the asymmetrical padding
    OutDisplayMetrics.TitleSafePaddingSize.X = Inset_Left;
    OutDisplayMetrics.TitleSafePaddingSize.Y = Inset_Top;
    OutDisplayMetrics.TitleSafePaddingSize.Z = Inset_Right;
    OutDisplayMetrics.TitleSafePaddingSize.W = Inset_Bottom;
    
    //scale the thing
	OutDisplayMetrics.TitleSafePaddingSize *= ContentScaleFactor;
    
    OutDisplayMetrics.ActionSafePaddingSize = OutDisplayMetrics.TitleSafePaddingSize;
#endif
    
}

#if !PLATFORM_TVOS
void FIOSApplication::CacheDisplayMetrics(UIInterfaceOrientation Orientation)
{
	CachedOrientation = Orientation;
	
	// Insets are given in whatever actual orientation the window has
	UIWindow* Window = [[[UIApplication sharedApplication] delegate] window];
	UIEdgeInsets OrientatedInsets = [Window safeAreaInsets];

	if (GIOSDelayRotationUntilPresent)
	{
		// After queueing a new orientation, the actual rotation hasn't occurred yet. 
		// We can attempt to adjust our current insets to the new orientation, but they are not entirely accurate as iOS adds an inset for the home swipe bar to the current orientation bottom.
		UIEdgeInsets PortraitInsets;

		// Convert current orientation to portrait
		switch ([Window.windowScene interfaceOrientation])
		{
		case UIInterfaceOrientationPortraitUpsideDown:
			PortraitInsets = UIEdgeInsetsMake(OrientatedInsets.bottom, OrientatedInsets.right, OrientatedInsets.top, OrientatedInsets.left);
			break;
		case UIInterfaceOrientationLandscapeLeft:
			PortraitInsets = UIEdgeInsetsMake(OrientatedInsets.right, OrientatedInsets.top, OrientatedInsets.left, OrientatedInsets.bottom);
			break;
		case UIInterfaceOrientationLandscapeRight:
			PortraitInsets = UIEdgeInsetsMake(OrientatedInsets.left, OrientatedInsets.bottom, OrientatedInsets.right, OrientatedInsets.top);
			break;
		case UIInterfaceOrientationPortrait:
		default:
			PortraitInsets = OrientatedInsets;
			break;	
		}

		// Convert portrait to new orientation and cache
		switch (Orientation)
		{
		case UIInterfaceOrientationPortraitUpsideDown:
			CachedInsets = UIEdgeInsetsMake(PortraitInsets.bottom, PortraitInsets.right, PortraitInsets.top, PortraitInsets.left);
			break;
		case UIInterfaceOrientationLandscapeLeft:
			CachedInsets = UIEdgeInsetsMake(PortraitInsets.left, PortraitInsets.bottom, PortraitInsets.right, PortraitInsets.top);
			break;
		case UIInterfaceOrientationLandscapeRight:
			CachedInsets = UIEdgeInsetsMake(PortraitInsets.right, PortraitInsets.top, PortraitInsets.left, PortraitInsets.bottom);
			break;
		case UIInterfaceOrientationPortrait:
		default:
			CachedInsets = PortraitInsets;
			break;	
		}
	}
	else
	{
		CachedInsets = OrientatedInsets;
	}

#if IOS_ROTATION_DEBUG_LOGGING
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[rotation] CachedInsets top:%f left:%f bottom:%f right:%f\n"), CachedInsets.top, CachedInsets.left, CachedInsets.bottom, CachedInsets.right);
#endif
}
#endif

TSharedRef< FGenericWindow > FIOSApplication::MakeWindow()
{
	return FIOSWindow::Make();
}

#if !PLATFORM_TVOS && !PLATFORM_VISIONOS

void FIOSApplication::OrientationChanged(UIInterfaceOrientation NewOrientation)
{
#if IOS_ROTATION_DEBUG_LOGGING
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[rotation] FIOSApplication::OrientationChanged: %d\n"), NewOrientation);
#endif

	// queue up the size as we see it now all the way to the RHI 
	[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
	{
	 	IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];
		FIOSApplication* App = AppDelegate.IOSApplication;
		
		__block FPlatformRect WindowRect;
		dispatch_sync(dispatch_get_main_queue(), ^{
			App->CacheDisplayMetrics(NewOrientation);	// Requires UI Thread
			WindowRect = FIOSWindow::GetScreenRect();	// Requires UI Thread
		});

		int32 WindowWidth = WindowRect.Right - WindowRect.Left;
		int32 WindowHeight = WindowRect.Bottom - WindowRect.Top;

		// reize viewport
		App->GetMessageHandler()->OnSizeChanged(App->Windows[0], WindowWidth, WindowHeight, false);
		App->GetMessageHandler()->OnResizingWindow(App->Windows[0]);
		 	
		FDisplayMetrics DisplayMetrics;
		FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);
		App->BroadcastDisplayMetricsChanged(DisplayMetrics);
		FCoreDelegates::OnSafeFrameChangedEvent.Broadcast();

		FCoreDelegates::ApplicationReceivedScreenOrientationChangedNotificationDelegate.Broadcast((int32)[IOSAppDelegate ConvertFromUIInterfaceOrientation:NewOrientation]);
		return true;
	 }];
}

void FIOSApplication::UpdateSafeZoneAfterRotation()
{
	// this function is called from the UI thread
	UIWindow* Window = [[[UIApplication sharedApplication] delegate] window];
	CachedInsets = [Window safeAreaInsets];

	// notify the game thread of the safe zone change
	[FAppleAsyncTask CreateTaskWithBlock : ^ bool(void)
	{
	 	IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];
		FIOSApplication* App = AppDelegate.IOSApplication;

		FDisplayMetrics DisplayMetrics;
		FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);
		App->BroadcastDisplayMetricsChanged(DisplayMetrics);
		FCoreDelegates::OnSafeFrameChangedEvent.Broadcast();
#if IOS_ROTATION_DEBUG_LOGGING
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[rotation] UpdateSafeZoneAfterRotation top:%f left:%f bottom:%f right:%f\n"), CachedInsets.top, CachedInsets.left, CachedInsets.bottom, CachedInsets.right);
#endif
		return true;
	}];
}

#endif


bool FIOSApplication::IsGamepadAttached() const
{
    if (InputInterface.IsValid())
	{
		return InputInterface->IsGamepadAttached();
	}

	return false;
}

TSharedRef<FIOSWindow> FIOSApplication::FindWindowByAppDelegateView()
{
	for (const TSharedRef<FIOSWindow>& Window : Windows)
	{
		if ([IOSAppDelegate GetDelegate].Window == static_cast<UIWindow*>(Window->GetOSWindowHandle()))
		{
			return Window;
		}
	}
    check(false);
    return Windows[0];
}

void FIOSApplication::OnInputDeviceModuleRegistered(const FName& Type, IModularFeature* ModularFeature)
{
	if (Type == IInputDeviceModule::GetModularFeatureName())
	{
		IInputDeviceModule* Module = static_cast<IInputDeviceModule*>(ModularFeature);

		TSharedPtr<IInputDevice> Device = Module->CreateInputDevice(MessageHandler);
		AddExternalInputDevice(MoveTemp(Device));
	}
}

#if WITH_ACCESSIBILITY

float GIOSAccessibleAnnouncementDelay = 0.1f;
FAutoConsoleVariableRef IOSAccessibleAnnouncementDealyRef(
	TEXT("ios.AccessibleAnnouncementDelay"),
	GIOSAccessibleAnnouncementDelay,
	TEXT("We need to introduce a small delay to avoid iOS system accessibility announcements from stomping on our requested user announcement. Delays <= 0.05f are too short and result in the announcement being dropped. Dellays ~0.075f result in unstable delivery")
);


void FIOSApplication::OnAccessibleEventRaised(const FAccessibleEventArgs& Args)
{
	// This should only be triggered by the accessible message handler which initiates from the Slate thread.
	check(IsInGameThread());

	const AccessibleWidgetId Id = Args.Widget->GetId();
	switch (Args.Event)
	{
	case EAccessibleEvent::ParentChanged:
	{
		const AccessibleWidgetId NewParentId = Args.NewValue.GetValue<AccessibleWidgetId>();
		dispatch_async(dispatch_get_main_queue(), ^{
			[[[FIOSAccessibilityCache AccessibilityElementCache] GetAccessibilityElement:Id] SetParent:NewParentId];
		});
		// LayoutChanged is to indicate things like "a widget became visible or hidden" while
		// ScreenChanged is for large-scale UI changes. It can potentially take an NSString to read
		// to the user when this happens, if we choose to support that.
		UIAccessibilityPostNotification(UIAccessibilityLayoutChangedNotification, nil);
		break;
	}
	case EAccessibleEvent::WidgetRemoved:
		dispatch_async(dispatch_get_main_queue(), ^{
			// There is an ensure() in FSlateAccessibleMessageHandler::OnWidgetRemoved that makes sure the Slate Accessibility Tree
			// is up to date and the removed widget doesn't have an invalid parent.
			// Also, since we update the children of each FIOSAccessibilityLeaf in [FIOSAccessibilityCache  UpdateAllProperties], we don't need to
			// handle reparenting/orphaned accessibility children here
			FIOSAccessibilityLeaf* Element = [[FIOSAccessibilityCache AccessibilityElementCache] GetAccessibilityElement:Id];
			[Element SetParent: IAccessibleWidget::InvalidAccessibleWidgetId];
			[[FIOSAccessibilityCache AccessibilityElementCache] RemoveAccessibilityElement:Id];
		});
		break;
	case EAccessibleEvent::Notification:
		{
			NSString* Announcement = [NSString stringWithFString: Args.NewValue.GetValue<FString>()];
			dispatch_async(dispatch_get_main_queue(), ^{
				// If we don't sleep for a small period of time, system accessibility
				// announcements can stomp on this announcement and so it's never made
				AccessibilityAnnouncementDelayTimer = [NSTimer scheduledTimerWithTimeInterval:GIOSAccessibleAnnouncementDelay repeats:NO block:^(NSTimer* _Nonnull timer){
					UIAccessibilityPostNotification(UIAccessibilityAnnouncementNotification, Announcement);
					AccessibilityAnnouncementDelayTimer = Nil;
				}];
			});
		}
		break;
	}
}
#endif

