// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOS/IOSConfigRules.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Misc/Parse.h"
#include "zlib.h"
#include <CommonCrypto/CommonCryptor.h>
#include <CommonCrypto/CommonKeyDerivation.h>

#include "Apple/AppleStringUtils.h"
#include "String/ParseLines.h"
#include "String/ParseTokens.h"
#include "Containers/StringConv.h"
#include "Misc/ByteSwap.h"
#include "Containers/Map.h"
#include "Misc/Compression.h"
#include "String/RemoveFrom.h"
#include "Internationalization/Regex.h"

#define USE_ODS_LOGGING 1

#if USE_ODS_LOGGING
	#define CONF_LOG(A,B, ...) FPlatformMisc::LowLevelOutputDebugStringf(__VA_ARGS__);
#else
	#define CONF_LOG(...) UE_LOG(__VA_ARGS__);
#endif

@implementation iOSConfigRuleProviders
static NSMutableArray* ConfigRuleProviders = nil;
+ (void)load
{
	ConfigRuleProviders = [[NSMutableArray alloc]init];
}

+ (void)registerRuleProvider:(NSObject<iOSConfigRuleProvider>*)newProvider
{
	[ConfigRuleProviders addObject: newProvider];	
}

+ (NSArray *)getRuleProviders
{
	return [ConfigRuleProviders copy];
}

+ (void)empty
{
	[ConfigRuleProviders removeAllObjects];
}
@end

class FConfigRules
{
public:

    static const int ExpectedConfRuleSig = 0x39d8;

    FString Path;
	FString Key;
	FString IV;

	uint32 Version = -1;
	uint32 UncompressedSize = 0;
    TArray<uint8> FileBytes;

#pragma pack(push)
#pragma pack(1)
    struct FConfHeader
    {
        uint16 Sig;
        int32 Version;
        int32 UncompressedSize;
    };
#pragma pack(pop)
    
    FConfigRules()
    {
       
    }
    
    int OpenAndGetVersionNumber(const FString& PathIN, const FString& KeyIn, const FString& IVIn)
    {
        check(Path.IsEmpty());
        Path = PathIN;
		Key = KeyIn;
		IV = IVIn;
		
        FFileHelper::LoadFileToArray(FileBytes, *Path);

		if (FileBytes.Num() > sizeof(FConfHeader))
        {
            FConfHeader ConfHeader = *((FConfHeader*)FileBytes.GetData());
            if (ByteSwap(ConfHeader.Sig) == ExpectedConfRuleSig)
            {
                Version = ByteSwap(ConfHeader.Version);
                UncompressedSize = ByteSwap(ConfHeader.UncompressedSize);
            }
        }
		else
		{
			CONF_LOG(LogConfigRules, Log, TEXT("ConfigRules: %s was not found."), *Path);	
		}
        return Version;
    }
	
	TConstArrayView<uint8> GetData() const
	{
		return TConstArrayView<uint8>(FileBytes.GetData() + sizeof(FConfHeader), FileBytes.Num() - sizeof(FConfHeader));
	}
};


bool Decrypt(const TConstArrayView<uint8>& DataIn, TArray<uint8>& DataOut, FString& key, FString& iv )
{
	check(!DataOut.IsEmpty());
	
	FTCHARToUTF8 UTF8Key(*key);
	
	uint8 	GeneratedKey[16] = {0};
	{
		static const uint8 salt[] = { 0x23, 0x71, 0xd3, 0xa3, 0x30, 0x71, 0x63, 0xe3};
		uint    rounds  = 1000;
		uint    keySize = kCCKeySizeAES128;

		
		CCKeyDerivationPBKDF(CCPBKDFAlgorithm(kCCPBKDF2),
							 UTF8Key.Get(),
							 UTF8Key.Length(),
							 salt,
							 sizeof(salt),
							 kCCPRFHmacAlgSHA1,
							 rounds,
							 GeneratedKey,
							 sizeof(GeneratedKey));
	}
	
	size_t WrittenBytes = 0;
	
	char ivPtr[kCCBlockSizeAES128 + 1];
	FMemory::Memzero(ivPtr);
	if (!iv.IsEmpty())
	{
		FTCHARToUTF8 UTF8iv(*iv);
		if (UTF8iv.Length() + 1 != sizeof(ivPtr))
		{
			return false;
		}
		FMemory::Memcpy(ivPtr, UTF8iv.Get(), sizeof(ivPtr));
	}

	CCCryptorStatus ccStatus   = kCCSuccess;
	size_t          cryptBytes = 0;
	
	ccStatus = CCCrypt(kCCDecrypt,
						kCCAlgorithmAES128,
						kCCOptionECBMode | kCCOptionPKCS7Padding,
                         GeneratedKey,
                         kCCKeySizeAES128,
                         ivPtr,
                         DataIn.GetData(),
                         DataIn.Num(),
                         DataOut.GetData(),
                         DataOut.Num(),
                         &cryptBytes);
    
	if (ccStatus == kCCSuccess)
	{
		DataOut.SetNum(cryptBytes);
	}
	
	return ccStatus == kCCSuccess;
}

namespace ConfigRules
{
	TMap<FString, FString> ProcessConfigRules(FConfigRules& Rules, TMap<FString,FString>&& PredefinedVariables);
};

bool FIOSConfigRules::Init(TMap<FString,FString>&& PredefinedVariables)
{
	InitRules();

	// load the configrule with the highest version.
	int FoundVersion = -1;
	FConfigRules SelectedRules;
	for (int i = 0; i < ConfigRulesParams.Num(); i++)
	{
		FString Filename = ConfigRulesParams[i].Path;
		FConfigRules TestRules;
		int TestVersion = TestRules.OpenAndGetVersionNumber(Filename, ConfigRulesParams[i].Key, ConfigRulesParams[i].IV);
		if (TestVersion > FoundVersion)
		{
			SelectedRules = MoveTemp(TestRules);
			FoundVersion = TestVersion;
		}
	}

	if (FoundVersion > -1)
	{
		ConfigRuleVariablesMap = ConfigRules::ProcessConfigRules(SelectedRules, MoveTemp(PredefinedVariables));
	}
	else
	{
		ConfigRuleVariablesMap = PredefinedVariables;
	}

#if !UE_BUILD_SHIPPING
	// Allow commandline overrides of config rule variables
	FString CrVarsValue;
	if (FParse::Value(FCommandLine::Get(), TEXT("crvars="), CrVarsValue, false))
	{
		TArray<FString> Pairs;
		CrVarsValue.ParseIntoArray(Pairs, TEXT(","));
		for (const FString& Pair : Pairs)
		{
			FString Key, Value;
			if (Pair.Split(TEXT("="), &Key, &Value))
			{
				Value.TrimQuotesInline();
				CONF_LOG(LogConfigRules, Log, TEXT("crvars override: set %s = %s"), *Key, *Value);
				ConfigRuleVariablesMap.Add(Key, Value);
			}
			else if (Pair.StartsWith(TEXT("-")))
			{
				Key = Pair.RightChop(1);
				CONF_LOG(LogConfigRules, Log, TEXT("crvars override: remove %s"), *Key);
				ConfigRuleVariablesMap.Remove(Key);
			}
			else
			{
				CONF_LOG(LogConfigRules, Log, TEXT("crvars ignoring malformed parameter: %s"), *Pair);
			}
		}
	}
#endif

	return FoundVersion > -1;
}

void FIOSConfigRules::InitRules()
{
    check(ConfigRulesParams.IsEmpty());
   
	for (NSObject<iOSConfigRuleProvider>* Provider in [iOSConfigRuleProviders getRuleProviders])
	{
		for (NSArray* Params in [Provider getRuleData])
		{
			if([Params count] == 3)
			{
				const char * PathStr = [[Params objectAtIndex:0] UTF8String];
				const char * KeyStr = [[Params objectAtIndex:1] UTF8String];
				const char * PathIV = [[Params objectAtIndex:2] UTF8String];
				
				FConfigRuleParams ConfigRuleParams;
				ConfigRuleParams.Path = PathStr;
				ConfigRuleParams.Key = KeyStr;
				ConfigRuleParams.IV = PathIV;
				
				ConfigRuleParams.Path.ReplaceInline(TEXT("[[cache]]"), *FString([NSSearchPathForDirectoriesInDomains(NSCachesDirectory, NSUserDomainMask, YES) objectAtIndex:0]));
				ConfigRulesParams.Add(ConfigRuleParams);
			}
		}
	}
	
	[iOSConfigRuleProviders empty];
	
	if(ConfigRulesParams.IsEmpty())
	{
		FConfigRuleParams ConfigRuleParams;
		ConfigRuleParams.Path = TEXT("~/configrules");		
		ConfigRulesParams.Add(ConfigRuleParams);
	}
}

namespace ConfigRules
{
	TMap<FString, FString> ProcessConfigRules(FConfigRules& Rules, TMap<FString,FString>&& PredefinedVariables)
	{
		TArray<uint8> DecryptBytes;
		bool bSuccess = true;
		if (!Rules.Key.IsEmpty())
		{
			DecryptBytes.SetNum(Rules.FileBytes.Num());
			bSuccess = Decrypt(Rules.GetData(), DecryptBytes, Rules.Key, Rules.IV);
		}

		TArray<uint8> UncompressedBytes;
		if (bSuccess)
		{
			const TConstArrayView<uint8>& SourceBytes = Rules.Key.IsEmpty() ? Rules.GetData() : DecryptBytes;
			UncompressedBytes.SetNum(Rules.UncompressedSize);
			bSuccess = FCompression::UncompressMemory(NAME_Zlib, UncompressedBytes.GetData(), UncompressedBytes.Num(), SourceBytes.GetData(), SourceBytes.Num());
		}

		if (bSuccess)
		{
			return FGenericConfigRules::ParseConfigRules(UncompressedBytes, MoveTemp(PredefinedVariables));
		}
		else
		{
			CONF_LOG(LogConfigRules, Error, TEXT("ConfigRules: file read failed for %s!"), *Rules.Path);
		}
		return PredefinedVariables;
	}
}
