// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOS/IOSDebugOutputDevice.h"
#include "HAL/PlatformMisc.h"
#include "CoreGlobals.h"
#include "Misc/OutputDeviceHelper.h"

FIOSDebugOutputDevice::FIOSDebugOutputDevice() 
{
}

void FIOSDebugOutputDevice::Serialize( const TCHAR* Msg, ELogVerbosity::Type Verbosity, const class FName& Category )
{
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("%s%s"),*FOutputDeviceHelper::FormatLogLine(Verbosity, Category, Msg, GPrintLogTimes),LINE_TERMINATOR); 
}

