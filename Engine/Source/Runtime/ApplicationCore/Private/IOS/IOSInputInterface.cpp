// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOS/IOSInputInterface.h"
#include "IOS/IOSAppDelegate.h"
#include "IOS/IOSApplication.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeLock.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformTime.h"
#include "Misc/EmbeddedCommunication.h"

#import <AudioToolbox/AudioToolbox.h>

#include "Apple/AppleControllerInterface.h"
#include "Apple/AppleMouseController.h"
#include "IOS/IOSInputDefinitions.h"

#define APPLE_CONTROLLER_DEBUG 0

static TAutoConsoleVariable<float> CVarHapticsKickHeavy(TEXT("ios.VibrationHapticsKickHeavyValue"), 0.65f, TEXT("Vibation values higher than this will kick a haptics heavy Impact"));
static TAutoConsoleVariable<float> CVarHapticsKickMedium(TEXT("ios.VibrationHapticsKickMediumValue"), 0.5f, TEXT("Vibation values higher than this will kick a haptics medium Impact"));
static TAutoConsoleVariable<float> CVarHapticsKickLight(TEXT("ios.VibrationHapticsKickLightValue"), 0.3f, TEXT("Vibation values higher than this will kick a haptics light Impact"));
static TAutoConsoleVariable<float> CVarHapticsRest(TEXT("ios.VibrationHapticsRestValue"), 0.2f, TEXT("Vibation values lower than this will allow haptics to Kick again when going over ios.VibrationHapticsKickValue"));
static TAutoConsoleVariable<int32> CVarUnifyMotionSpace(TEXT("ios.UnifyMotionSpace"), 1, TEXT("If set to non-zero, acceleration, gravity, and rotation rate will all be in the same coordinate space. 0 for legacy behaviour. 1 (default as of 5.5) will match Unreal's coordinate space (left-handed, z-up, etc). 2 will be right-handed by swapping x and y. Non-zero also forces rotation rate units to be radians/s and acceleration units to be g."));

constexpr EIOSEventType operator+(EIOSEventType type, int Index) { return (EIOSEventType)((int)type + Index); }
// protects the input stack used on 2 threads
static FCriticalSection CriticalSection;
static TArray<TouchInput> TouchInputStack;
static TArray<GestureInput> GestureInputStack;
static TArray<GestureInput> PendingDiscreteGestureReleases;

FName FIOSInputInterface::InputClassName_DefaultMobileTouch;
FString FIOSInputInterface::HardwareDeviceIdentifier_DefaultMobileTouch;

TSharedRef< FIOSInputInterface > FIOSInputInterface::Create(  const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler )
{
    return MakeShareable( new FIOSInputInterface( InMessageHandler ) );
}

void FIOSInputInterface::MapKeyboardModifierKeys()
{
	constexpr int32 GCKeysToMapNum = 9;
	KeyboardController->IncreaseModifierKeyMapAllocation(GCKeysToMapNum);
	
	using namespace IOS;
	KeyboardController->AddModifierKeyMapping(static_cast<uint32>(EUnrealKeyCode::CapsLock), FAppleKeyboardController::EModifierKeyFlags::CapsLocked);
	KeyboardController->AddModifierKeyMapping(static_cast<uint32>(EUnrealKeyCode::LeftAlt), FAppleKeyboardController::EModifierKeyFlags::LeftAltDown);
	KeyboardController->AddModifierKeyMapping(static_cast<uint32>(EUnrealKeyCode::LeftCommand), FAppleKeyboardController::EModifierKeyFlags::LeftCommandDown);
	KeyboardController->AddModifierKeyMapping(static_cast<uint32>(EUnrealKeyCode::LeftControl), FAppleKeyboardController::EModifierKeyFlags::LeftControlDown);
	KeyboardController->AddModifierKeyMapping(static_cast<uint32>(EUnrealKeyCode::LeftShift), FAppleKeyboardController::EModifierKeyFlags::LeftShiftDown);
	KeyboardController->AddModifierKeyMapping(static_cast<uint32>(EUnrealKeyCode::RightAlt), FAppleKeyboardController::EModifierKeyFlags::RightAltDown);
	KeyboardController->AddModifierKeyMapping(static_cast<uint32>(EUnrealKeyCode::RightCommand), FAppleKeyboardController::EModifierKeyFlags::RightCommandDown);
	KeyboardController->AddModifierKeyMapping(static_cast<uint32>(EUnrealKeyCode::RightControl), FAppleKeyboardController::EModifierKeyFlags::RightControlDown);
	KeyboardController->AddModifierKeyMapping(static_cast<uint32>(EUnrealKeyCode::RightShift), FAppleKeyboardController::EModifierKeyFlags::RightShiftDown);
}

FIOSInputInterface::FIOSInputInterface( const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler )
	: FAppleControllerInterface( InMessageHandler )
	, bAllowRemoteRotation(false)
	, bGameSupportsMultipleActiveControllers(false)
    , LastHapticValue(0.0f)
	, KeyboardController(MakeShared<FAppleKeyboardController>(InMessageHandler, FAppleKeyboardController::EInitFlags::GCKeyboardKeyHandlerDisabled))
	, MouseController(MakeShared<FAppleMouseController>(InMessageHandler))
{
    SCOPED_BOOT_TIMING("FIOSInputInterface::FIOSInputInterface");
	
	MapKeyboardModifierKeys();

#if !PLATFORM_TVOS
    MotionManager = nil;
    ReferenceAttitude = nil;
#endif
    bPauseMotion = false;
    GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bDisableMotionData"), bPauseMotion, GEngineIni);
    
    GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bGameSupportsMultipleActiveControllers"), bGameSupportsMultipleActiveControllers, GEngineIni);
    GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bAllowRemoteRotation"), bAllowRemoteRotation, GEngineIni);
    GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bAllowControllers"), bAllowControllers, GEngineIni);
    GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bControllersBlockDeviceFeedback"), bControllersBlockDeviceFeedback, GEngineIni);
    
    
    NSNotificationCenter* notificationCenter = [NSNotificationCenter defaultCenter];
    NSOperationQueue* currentQueue = [NSOperationQueue currentQueue];

    if (!bGameSupportsMultipleActiveControllers)
    {
        [notificationCenter addObserverForName:GCControllerDidBecomeCurrentNotification object:nil queue:currentQueue usingBlock:^(NSNotification* Notification)
         {
            SignalEvent(EAppleControllerEventType::BecomeCurrent, Notification.object);
        }];
    }
    
    FEmbeddedDelegates::GetNativeToEmbeddedParamsDelegateForSubsystem(TEXT("iosinput")).AddLambda([this](const FEmbeddedCallParamsHelper& Message)
                                                                                                  {
        FString Error;
#if !PLATFORM_TVOS
        // execute any console commands
        if (Message.Command == TEXT("stopmotion"))
        {
            [MotionManager release];
            MotionManager = nil;
            
            bPauseMotion = true;
        }
        else if (Message.Command == TEXT("startmotion"))
        {
            bPauseMotion = false;
        }
        else
#endif
        {
            Error = TEXT("Unknown iosinput command ") + Message.Command;
        }
        
        Message.OnCompleteDelegate({}, Error);
    });

    
#if !PLATFORM_TVOS
    HapticFeedbackSupportLevel = [[[UIDevice currentDevice] valueForKey:@"_feedbackSupportLevel"] intValue];
#else
    HapticFeedbackSupportLevel = 0;
#endif

	InputClassName_DefaultMobileTouch = TEXT("DefaultMobileTouch");
	HardwareDeviceIdentifier_DefaultMobileTouch = TEXT("MobileTouch");
}


#if !PLATFORM_TVOS
void ModifyVectorByOrientation(FVector& Vec, bool bIsRotation)
{
    switch (FIOSApplication::CachedOrientation)
    {
        case UIInterfaceOrientationPortrait:
            // this is the base orientation, so nothing to do
            break;

        case UIInterfaceOrientationPortraitUpsideDown:
            if (bIsRotation)
            {
                // negate roll and pitch
                Vec.X = -Vec.X;
                Vec.Z = -Vec.Z;
            }
            else
            {
                // negate x/y
                Vec.X = -Vec.X;
                Vec.Y = -Vec.Y;
            }
            break;

        case UIInterfaceOrientationLandscapeRight:
            if (bIsRotation)
            {
                // swap and negate (as needed) roll and pitch
                double Temp = Vec.X;
                Vec.X = -Vec.Z;
                Vec.Z = Temp;
                Vec.Y *= -1.0f;
            }
            else
            {
                // swap and negate (as needed) x and y
                double Temp = Vec.X;
                Vec.X = -Vec.Y;
                Vec.Y = Temp;
            }
            break;

        case UIInterfaceOrientationLandscapeLeft:
            if (bIsRotation)
            {
                // swap and negate (as needed) roll and pitch
                double Temp = Vec.X;
                Vec.X = -Vec.Z;
                Vec.Z = -Temp;
            }
            else
            {
                // swap and negate (as needed) x and y
                double Temp = Vec.X;
                Vec.X = Vec.Y;
                Vec.Y = -Temp;
            }
            break;
    }
}
#endif

void FIOSInputInterface::ProcessTouchesAndKeys(uint32 ControllerId, const TArray<TouchInput>& InTouchInputStack)
{
    for(int i = 0; i < InTouchInputStack.Num(); ++i)
    {
        const TouchInput& Touch = InTouchInputStack[i];
        
		///The FInputDeviceScope::HardwareDeviceIdentifier has to be one of values in UInputPlatformSettings::HardwareDevices.
		// This is a temp solution with a hardcoded string which can be mapped to FHardwareDeviceIdentifier::DefaultMobileTouch.
		// TODO: Future improvement is needed to acquire them by values in the iOS input device info.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FInputDeviceScope InputScope(nullptr, InputClassName_DefaultMobileTouch, 0, HardwareDeviceIdentifier_DefaultMobileTouch);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

        // send input to handler
        if (Touch.Type == TouchBegan)
        {
            MessageHandler->OnTouchStarted( NULL, Touch.Position, Touch.Force, Touch.Handle, ControllerId);
        }
        else if (Touch.Type == TouchEnded)
        {
            MessageHandler->OnTouchEnded(Touch.Position, Touch.Handle, ControllerId);
        }
        else if (Touch.Type == TouchMoved)
        {
            MessageHandler->OnTouchMoved(Touch.Position, Touch.Force, Touch.Handle, ControllerId);
        }
        else if (Touch.Type == ForceChanged)
        {
            MessageHandler->OnTouchForceChanged(Touch.Position, Touch.Force, Touch.Handle, ControllerId);
        }
        else if (Touch.Type == FirstMove)
        {
            MessageHandler->OnTouchFirstMove(Touch.Position, Touch.Force, Touch.Handle, ControllerId);
        }
    }
}

void FIOSInputInterface::ProcessGestures(uint32 ControllerId, const TArray<GestureInput>& InGestureInputStack)
{
	// Remap the controller ID to FInputDeviceId, matching ProcessTouchesAndKeys.
	FPlatformUserId UserId = FGenericPlatformMisc::GetPlatformUserForUserIndex(ControllerId);
	FInputDeviceId DeviceId = INPUTDEVICEID_NONE;
	IPlatformInputDeviceMapper::Get().RemapControllerIdToPlatformUserAndDevice(ControllerId, UserId, DeviceId);

	// Inject deferred releases for discrete gestures (Tap, Flick) from the previous
	// frame so EvaluateKeyMapState had one full tick to read the non-zero press value.
	for (const GestureInput& Release : PendingDiscreteGestureReleases)
	{
		MessageHandler->OnNativeGestureEnd(Release.Type, Release.Delta, 0.0f, false, UserId, DeviceId);
	}
	PendingDiscreteGestureReleases.Reset();

	for (const GestureInput& Gesture : InGestureInputStack)
	{
		// Convert gesture values to match software gesture output.
		FVector2D ConvertedDelta = Gesture.Delta;

		switch (Gesture.Type)
		{
		case EGestureEvent::Magnify:
			if (Gesture.bIsStart)
			{
				CumulativePinchScale = static_cast<float>(Gesture.Delta.X);
			}
			else
			{
				CumulativePinchScale *= static_cast<float>(Gesture.Delta.X);
			}
			ConvertedDelta = FVector2D(CumulativePinchScale, 0.0);
			if (Gesture.bIsEnd)
			{
				CumulativePinchScale = 1.0f;
			}
			break;

		default:
			break;
		}

		if (Gesture.bIsStart)
		{
			if (Gesture.bIsEnd)
			{
				NSLog(@"Warning: Gesture event (type %d) has both bIsStart and bIsEnd set; treating as Begin.", (int)Gesture.Type);
			}
			MessageHandler->OnNativeGestureBegin(Gesture.Type, ConvertedDelta, 0.0f, false, UserId, DeviceId);
		}
		else if (Gesture.bIsEnd)
		{
			MessageHandler->OnNativeGestureEnd(Gesture.Type, ConvertedDelta, 0.0f, false, UserId, DeviceId);
		}
		else
		{
			// Treat gesture events with false values of bIsStart and bIsEnd as Update.
			MessageHandler->OnNativeGestureUpdate(Gesture.Type, ConvertedDelta, 0.0f, false, UserId, DeviceId);
		}

		// For discrete (one-shot) gestures, defer the release to next frame.
		// The press sets RawValue to the gesture value; the deferred release
		// clears it after EvaluateKeyMapState has had one tick to read it.
		const bool bIsDiscreteGesture = (Gesture.Type == EGestureEvent::Tap || Gesture.Type == EGestureEvent::Flick);
		if (bIsDiscreteGesture && Gesture.bIsStart)
		{
			GestureInput DeferredRelease;
			DeferredRelease.Type = Gesture.Type;
			DeferredRelease.Delta = FVector2D::ZeroVector;
			DeferredRelease.bIsStart = false;
			DeferredRelease.bIsEnd = true;
			PendingDiscreteGestureReleases.Add(DeferredRelease);
		}
	}
}

void FIOSInputInterface::SendControllerEvents()
{
	KeyboardController->SendControllerEvents();
	MouseController->SendControllerEvents();

    TArray<TouchInput> LocalTouchInputStack;
    {
        FScopeLock Lock(&CriticalSection);
        Exchange(LocalTouchInputStack, TouchInputStack);
    }
    
#if !PLATFORM_TVOS
    // on ios, touches always go go player 0
    ProcessTouchesAndKeys(0, LocalTouchInputStack);
#endif

	// Not sure if I should put TVOS as an exception for gesture handling.
	// It won't be harmful to keep TVOS in the list - If without gesture support, anyways the input stack will be empty.
	TArray<GestureInput> LocalGestureInputStack;
    {
        FScopeLock Lock(&CriticalSection);
		Exchange(LocalGestureInputStack, GestureInputStack);
    }
	// On iOS, gestures always go to player 0, matching touch input above.
	ProcessGestures(0, LocalGestureInputStack);
    
#if !PLATFORM_TVOS // @todo tvos: This needs to come from the Microcontroller rotation
    if (!bPauseMotion)
    {
        // Update motion controls.
        FVector Attitude;
        FVector RotationRate;
        FVector Gravity;
        FVector Acceleration;

        GetMovementData(Attitude, RotationRate, Gravity, Acceleration);

		const int32 UnifyMotionSpace = CVarUnifyMotionSpace.GetValueOnGameThread();
		if (UnifyMotionSpace == 0)
		{
			// Fix-up yaw to match directions
			Attitude.Y = -Attitude.Y;
			RotationRate.Y = -RotationRate.Y;

			// munge the vectors based on the orientation
			ModifyVectorByOrientation(Attitude, true);
			ModifyVectorByOrientation(RotationRate, true);
			ModifyVectorByOrientation(Gravity, false);
			ModifyVectorByOrientation(Acceleration, false);
		}
		else
		{
			// Match Unreal coordinate system
			auto ReorientPortrait = [](FVector InValue)
				{
					return FVector(-InValue.Z, InValue.X, InValue.Y);
				};
			auto ReorientPortraitUpsideDown = [](FVector InValue)
				{
					return FVector(-InValue.Z, -InValue.X, -InValue.Y);
				};
			auto ReorientLandscapeRight = [](FVector InValue)
				{
					return FVector(-InValue.Z, InValue.Y, -InValue.X);
				};
			auto ReorientLandscapeLeft = [](FVector InValue)
				{
					return FVector(-InValue.Z, -InValue.Y, InValue.X);
				};

			switch (FIOSApplication::CachedOrientation)
			{
			case UIInterfaceOrientationPortrait:
				Attitude = ReorientPortrait(Attitude);
				RotationRate = ReorientPortrait(RotationRate);
				Gravity = ReorientPortrait(Gravity);
				Acceleration = ReorientPortrait(Acceleration);
				break;

			case UIInterfaceOrientationPortraitUpsideDown:
				Attitude = ReorientPortraitUpsideDown(Attitude);
				RotationRate = ReorientPortraitUpsideDown(RotationRate);
				Gravity = ReorientPortraitUpsideDown(Gravity);
				Acceleration = ReorientPortraitUpsideDown(Acceleration);
				break;

			case UIInterfaceOrientationLandscapeRight:
				Attitude = ReorientLandscapeRight(Attitude);
				RotationRate = ReorientLandscapeRight(RotationRate);
				Gravity = ReorientLandscapeRight(Gravity);
				Acceleration = ReorientLandscapeRight(Acceleration);
				break;

			case UIInterfaceOrientationLandscapeLeft:
				Attitude = ReorientLandscapeLeft(Attitude);
				RotationRate = ReorientLandscapeLeft(RotationRate);
				Gravity = ReorientLandscapeLeft(Gravity);
				Acceleration = ReorientLandscapeLeft(Acceleration);
				break;
			}

			if (UnifyMotionSpace == 2)
			{
				// Right-handed variation
				Attitude = FVector(Attitude.Y, Attitude.X, Attitude.Z);
				RotationRate = FVector(RotationRate.Y, RotationRate.X, RotationRate.Z);
				Gravity = FVector(Gravity.Y, Gravity.X, Gravity.Z);
				Acceleration = FVector(Acceleration.Y, Acceleration.X, Acceleration.Z);
			}
		}

        MessageHandler->OnMotionDetected(Attitude, RotationRate, Gravity, Acceleration, 0);
    }
#endif
    
    // Generic Controller update
    FAppleControllerInterface::SendControllerEvents();
}

void FIOSInputInterface::SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
{
	FAppleControllerInterface::SetMessageHandler(InMessageHandler);
	KeyboardController->SetMessageHandler(InMessageHandler);
	MouseController->SetMessageHandler(InMessageHandler);
}

void FIOSInputInterface::QueueTouchInput(const TArray<TouchInput>& InTouchEvents)
{
    FScopeLock Lock(&CriticalSection);
    
    TouchInputStack.Append(InTouchEvents);
}

void FIOSInputInterface::QueueGestureInput(const TArray<GestureInput>& InGestureInputs)
{
	if (!FGenericPlatformApplicationMisc::AreNativeGesturesEnabled())
	{
		return;
	}

	FScopeLock Lock(&CriticalSection);
	GestureInputStack.Append(InGestureInputs);
}

void FIOSInputInterface::EnableMotionData(bool bEnable)
{
    bPauseMotion = !bEnable;
    
#if !PLATFORM_TVOS
    if (bPauseMotion && MotionManager != nil)
    {
        [ReferenceAttitude release];
        ReferenceAttitude = nil;
        
        [MotionManager release];
        MotionManager = nil;
    }
    // When enabled MotionManager will be initialized on first use
#endif
}

bool FIOSInputInterface::IsMotionDataEnabled() const
{
    return !bPauseMotion;
}

TWeakPtr<FAppleKeyboardController> FIOSInputInterface::GetKeyboardController()
{
	return KeyboardController;
}

TWeakPtr<FAppleMouseController> FIOSInputInterface::GetMouseController()
{
	return MouseController;
}

void FIOSInputInterface::SetKeyboardControllerInhibited(bool bInhibited)
{
	return KeyboardController->SetInhibited(bInhibited);
}

bool FIOSInputInterface::IsKeyboardControllerInhibited() const
{
	return KeyboardController->IsInhibited();
}

bool FIOSInputInterface::IsAnyPhysicalKeyboardConnected() const
{
	return KeyboardController->IsAnyKeyboardConnected();
}

void FIOSInputInterface::GetMovementData(FVector& Attitude, FVector& RotationRate, FVector& Gravity, FVector& Acceleration)
{
#if !PLATFORM_TVOS
    // initialize on first use
    if (MotionManager == nil)
    {
        // Look to see if we can create the motion manager
        MotionManager = [[CMMotionManager alloc] init];
        
        // Check to see if the device supports full motion (gyro + accelerometer)
        if (MotionManager.deviceMotionAvailable)
        {
            MotionManager.deviceMotionUpdateInterval = 0.02;
            
            // Start the Device updating motion
            [MotionManager startDeviceMotionUpdates];
        }
        else
        {
            [MotionManager startAccelerometerUpdates];
            CenterPitch = CenterPitch = 0;
            bIsCalibrationRequested = false;
        }
    }
    
    // do we have full motion data?
    if (MotionManager.deviceMotionActive)
    {
        // Grab the values
        CMAttitude* CurrentAttitude = MotionManager.deviceMotion.attitude;
        CMRotationRate CurrentRotationRate = MotionManager.deviceMotion.rotationRate;
        CMAcceleration CurrentGravity = MotionManager.deviceMotion.gravity;
        CMAcceleration CurrentUserAcceleration = MotionManager.deviceMotion.userAcceleration;
        
        // apply a reference attitude if we have been calibrated away from default
        if (ReferenceAttitude)
        {
            [CurrentAttitude multiplyByInverseOfAttitude : ReferenceAttitude];
        }
        
        // convert to Unreal coordinate system
        Attitude = FVector(float(CurrentAttitude.pitch), float(CurrentAttitude.yaw), float(CurrentAttitude.roll));
        RotationRate = FVector(float(CurrentRotationRate.x), float(CurrentRotationRate.y), float(CurrentRotationRate.z));
        Gravity = FVector(float(CurrentGravity.x), float(CurrentGravity.y), float(CurrentGravity.z));
        Acceleration = FVector(float(CurrentUserAcceleration.x), float(CurrentUserAcceleration.y), float(CurrentUserAcceleration.z));
    }
    else
    {
        // get the plain accleration
        CMAcceleration RawAcceleration = [MotionManager accelerometerData].acceleration;
        FVector NewAcceleration(RawAcceleration.x, RawAcceleration.y, RawAcceleration.z);
        
        // storage for keeping the accelerometer values over time (for filtering)
        static bool bFirstAccel = true;
        
        // how much of the previous frame's acceleration to keep
        const float VectorFilter = bFirstAccel ? 0.0f : 0.85f;
        bFirstAccel = false;
        
        // apply new accelerometer values to last frames
        FilteredAccelerometer = FilteredAccelerometer * VectorFilter + (1.0f - VectorFilter) * NewAcceleration;
        
        // create an normalized acceleration vector
        FVector FinalAcceleration = -FilteredAccelerometer.GetSafeNormal();
        
        // calculate Roll/Pitch
        float CurrentPitch = (float)FMath::Atan2(FinalAcceleration.Y, FinalAcceleration.Z);
        float CurrentRoll = -(float)FMath::Atan2(FinalAcceleration.X, FinalAcceleration.Z);
        
        // if we want to calibrate, use the current values as center
        if (bIsCalibrationRequested)
        {
            CenterPitch = CurrentPitch;
            CenterRoll = CurrentRoll;
            bIsCalibrationRequested = false;
        }
        
        CurrentPitch -= CenterPitch;
        CurrentRoll -= CenterRoll;
        
        Attitude = FVector(CurrentPitch, 0, CurrentRoll);
        RotationRate = FVector(LastPitch - CurrentPitch, 0, LastRoll - CurrentRoll);
        Gravity = FVector(0, 0, 0);
        
        // use the raw acceleration for acceleration
        Acceleration = NewAcceleration;
        
        // remember for next time (for rotation rate)
        LastPitch = CurrentPitch;
        LastRoll = CurrentRoll;
    }
#endif
}

void FIOSInputInterface::CalibrateMotion(uint32 PlayerIndex)
{
#if !PLATFORM_TVOS
    // If we are using the motion manager, grab a reference frame.  Note, once you set the Attitude Reference frame
    // all additional reference information will come from it
    if (MotionManager && MotionManager.deviceMotionActive)
    {
        ReferenceAttitude = [MotionManager.deviceMotion.attitude retain];
    }
    else
    {
        bIsCalibrationRequested = true;
    }
#endif
    
    if (PlayerIndex >= 0 && PlayerIndex < UE_ARRAY_COUNT(Controllers))
    {
        Controllers[PlayerIndex].bNeedsReferenceAttitude = true;
    }
}

bool FIOSInputInterface::Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
    // Keep track whether the command was handled or not.
    bool bHandledCommand = false;
    
    if (FParse::Command(&Cmd, TEXT("CALIBRATEMOTION")))
    {
        uint32 PlayerIndex = FCString::Atoi(Cmd);
        CalibrateMotion(PlayerIndex);
        bHandledCommand = true;
    }
    
    return bHandledCommand;
}

void FIOSInputInterface::SetForceFeedbackChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value)
{
    // If a controller with CoreHaptics support is bound to this id, route the feedback to its handles.
    // The phone-vibration fallback below stays in place for SiriRemote/older MFi pads with no GCDeviceHaptics.
    if (ControllerHasHaptics(ControllerId))
    {
        FAppleControllerInterface::SetForceFeedbackChannelValue(ControllerId, ChannelType, Value);
        return;
    }

    if(IsGamepadAttached() && bControllersBlockDeviceFeedback)
    {
        Value = 0.0f;
    }
    
    if(HapticFeedbackSupportLevel >= 2)
    {
        // if we are at rest, then kick when we are over the Kick cutoff
        if (LastHapticValue == 0.0f && Value > 0.0f)
        {
            const float HeavyKickVal = CVarHapticsKickHeavy.GetValueOnGameThread();
            const float MediumKickVal = CVarHapticsKickMedium.GetValueOnGameThread();
            const float LightKickVal = CVarHapticsKickLight.GetValueOnGameThread();
            // once we get past the
            if (Value > LightKickVal)
            {
                if (Value > HeavyKickVal)
                {
                    FPlatformMisc::PrepareMobileHaptics(EMobileHapticsType::ImpactHeavy);
                }
                else if (Value > MediumKickVal)
                {
                    FPlatformMisc::PrepareMobileHaptics(EMobileHapticsType::ImpactMedium);
                }
                else
                {
                    FPlatformMisc::PrepareMobileHaptics(EMobileHapticsType::ImpactLight);
                }

                FPlatformMisc::TriggerMobileHaptics();
                
                // remember it to not kick again
                LastHapticValue = Value;
            }
        }
        else
        {
            const float RestVal = CVarHapticsRest.GetValueOnGameThread();

            if (Value >= RestVal)
            {
                // always remember the last value if we are over the Rest amount
                LastHapticValue = Value;
            }
            else
            {
                // release the haptics
                FPlatformMisc::ReleaseMobileHaptics();
                
                // rest
                LastHapticValue = 0.0f;
            }
        }
    }
    else
    {
        if(Value >= 0.3f)
        {
            AudioServicesPlaySystemSound(kSystemSoundID_Vibrate);
        }
    }
}

void FIOSInputInterface::SetForceFeedbackChannelValues(int32 ControllerId, const FForceFeedbackValues &Values)
{
    // Prefer the controller's GCDeviceHaptics when available so we don't lose per-channel resolution
    // by collapsing to a single max value for phone vibration.
    if (ControllerHasHaptics(ControllerId))
    {
        FAppleControllerInterface::SetForceFeedbackChannelValues(ControllerId, Values);
        return;
    }

    // Use largest vibration state as value
    float MaxLeft = Values.LeftLarge > Values.LeftSmall ? Values.LeftLarge : Values.LeftSmall;
    float MaxRight = Values.RightLarge > Values.RightSmall ? Values.RightLarge : Values.RightSmall;
    float Value = MaxLeft > MaxRight ? MaxLeft : MaxRight;
    
    // the other function will just play, regardless of channel
    SetForceFeedbackChannelValue(ControllerId, FForceFeedbackChannelType::LEFT_LARGE, Value);
}

NSData* FIOSInputInterface::GetGamepadGlyphRawData(const FGamepadKeyNames::Type& ButtonKey, uint32 ControllerIndex)
{
    GCController* Cont = Controllers[ControllerIndex].Controller;
    GCExtendedGamepad* ExtendedGamepad = Cont.extendedGamepad;
    if (ExtendedGamepad == nil)
    {
        NSLog(@"Siri Remote is not compatible with glyphs.");
        return nullptr;
    }
    
    GCControllerButtonInput *ButtonToReturnGlyphOf = GetGCControllerButton(ButtonKey, ControllerIndex);
    
    UIImage* Image = nullptr;
    NSString *ButtonStringName = ButtonToReturnGlyphOf.sfSymbolsName;
    Image = [UIImage systemImageNamed:ButtonStringName];
    
    return UIImagePNGRepresentation(Image);
}
