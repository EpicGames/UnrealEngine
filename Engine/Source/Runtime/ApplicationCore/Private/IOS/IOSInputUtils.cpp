// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOS/IOSInputUtils.h"

#include "IOS/IOSInputDefinitions.h"

namespace IOS
{
	EUnrealKeyCode TranslateUSBHIDToUnrealKeycode(UIKeyboardHIDUsage HIDKeycode)
	{
		switch (HIDKeycode) 
		{
			case UIKeyboardHIDUsageKeyboardReturnOrEnter:
				return EUnrealKeyCode::Enter;
			case UIKeyboardHIDUsageKeyboardEscape:
				return EUnrealKeyCode::Escape;
			case UIKeyboardHIDUsageKeyboardDeleteOrBackspace:
				return EUnrealKeyCode::Backspace;
			case UIKeyboardHIDUsageKeyboardDeleteForward:
				return EUnrealKeyCode::Delete;
			case UIKeyboardHIDUsageKeyboardTab:
				return EUnrealKeyCode::Tab;
			case UIKeyboardHIDUsageKeyboardCapsLock:
				return EUnrealKeyCode::CapsLock;
			case UIKeyboardHIDUsageKeyboardSpacebar:
				return EUnrealKeyCode::SpaceBar;
			case UIKeyboardHIDUsageKeyboardF1:
				return EUnrealKeyCode::F1;
			case UIKeyboardHIDUsageKeyboardF2:
				return EUnrealKeyCode::F2;
			case UIKeyboardHIDUsageKeyboardF3:
				return EUnrealKeyCode::F3;
			case UIKeyboardHIDUsageKeyboardF4:
				return EUnrealKeyCode::F4;
			case UIKeyboardHIDUsageKeyboardF5:
				return EUnrealKeyCode::F5;
			case UIKeyboardHIDUsageKeyboardF6:
				return EUnrealKeyCode::F6;
			case UIKeyboardHIDUsageKeyboardF7:
				return EUnrealKeyCode::F7;
			case UIKeyboardHIDUsageKeyboardF8:
				return EUnrealKeyCode::F8;
			case UIKeyboardHIDUsageKeyboardF9:
				return EUnrealKeyCode::F9;
			case UIKeyboardHIDUsageKeyboardF10:
				return EUnrealKeyCode::F10;
			case UIKeyboardHIDUsageKeyboardF11:
				return EUnrealKeyCode::F11;
			case UIKeyboardHIDUsageKeyboardF12:
				return EUnrealKeyCode::F12;
			case UIKeyboardHIDUsageKeyboardRightArrow:
				return EUnrealKeyCode::Right;
			case UIKeyboardHIDUsageKeyboardLeftArrow:
				return EUnrealKeyCode::Left;
			case UIKeyboardHIDUsageKeyboardDownArrow:
				return EUnrealKeyCode::Down;
			case UIKeyboardHIDUsageKeyboardUpArrow:
				return EUnrealKeyCode::Up;
			case UIKeyboardHIDUsageKeypadEnter:
				return EUnrealKeyCode::Enter;
			case UIKeyboardHIDUsageKeyboardF13:
				return EUnrealKeyCode::F13;
			case UIKeyboardHIDUsageKeyboardF14:
				return EUnrealKeyCode::F14;
			case UIKeyboardHIDUsageKeyboardF15:
				return EUnrealKeyCode::F15;
			case UIKeyboardHIDUsageKeyboardF16:
				return EUnrealKeyCode::F16;
			case UIKeyboardHIDUsageKeyboardF17:
				return EUnrealKeyCode::F17;
			case UIKeyboardHIDUsageKeyboardF18:
				return EUnrealKeyCode::F18;
			case UIKeyboardHIDUsageKeyboardF19:
				return EUnrealKeyCode::F19;
			case UIKeyboardHIDUsageKeyboardF20:
				return EUnrealKeyCode::F20;
			case UIKeyboardHIDUsageKeyboardLeftControl:
				return EUnrealKeyCode::LeftControl;
			case UIKeyboardHIDUsageKeyboardLeftShift:
				return EUnrealKeyCode::LeftShift;
			case UIKeyboardHIDUsageKeyboardLeftAlt:
				return EUnrealKeyCode::LeftAlt;
			case UIKeyboardHIDUsageKeyboardLeftGUI:
				return EUnrealKeyCode::LeftCommand;
			case UIKeyboardHIDUsageKeyboardRightControl:
				return EUnrealKeyCode::RightControl;
			case UIKeyboardHIDUsageKeyboardRightShift:
				return EUnrealKeyCode::RightShift;
			case UIKeyboardHIDUsageKeyboardRightAlt:
				return EUnrealKeyCode::RightAlt;
			case UIKeyboardHIDUsageKeyboardRightGUI:
				return EUnrealKeyCode::RightCommand;
			case UIKeyboardHIDUsageKeyboardPageUp:
				return EUnrealKeyCode::PageUp;
			case UIKeyboardHIDUsageKeyboardPageDown:
				return EUnrealKeyCode::PageDown;
			case UIKeyboardHIDUsageKeyboardHome:
				return EUnrealKeyCode::Home;
			case UIKeyboardHIDUsageKeyboardEnd:
				return EUnrealKeyCode::End;
			case UIKeyboardHIDUsageKeyboardPause:
				return EUnrealKeyCode::Pause;
			case UIKeyboardHIDUsageKeyboardInsert:
				return EUnrealKeyCode::Insert;
			case UIKeyboardHIDUsageKeyboardLockingNumLock:
				return EUnrealKeyCode::NumLock;
			case UIKeyboardHIDUsageKeypad0:
				return EUnrealKeyCode::NumPad0;
			case UIKeyboardHIDUsageKeypad1:
				return EUnrealKeyCode::NumPad1;
			case UIKeyboardHIDUsageKeypad2:
				return EUnrealKeyCode::NumPad2;
			case UIKeyboardHIDUsageKeypad3:
				return EUnrealKeyCode::NumPad3;
			case UIKeyboardHIDUsageKeypad4:
				return EUnrealKeyCode::NumPad4;
			case UIKeyboardHIDUsageKeypad5:
				return EUnrealKeyCode::NumPad5;
			case UIKeyboardHIDUsageKeypad6:
				return EUnrealKeyCode::NumPad6;
			case UIKeyboardHIDUsageKeypad7:
				return EUnrealKeyCode::NumPad7;
			case UIKeyboardHIDUsageKeypad8:
				return EUnrealKeyCode::NumPad8;
			case UIKeyboardHIDUsageKeypad9:
				return EUnrealKeyCode::NumPad9;
			case UIKeyboardHIDUsageKeypadPlus:
				return EUnrealKeyCode::Add;
			case UIKeyboardHIDUsageKeypadHyphen:
				return EUnrealKeyCode::Subtract;
			case UIKeyboardHIDUsageKeypadSlash:
				return EUnrealKeyCode::Divide;
			case UIKeyboardHIDUsageKeypadAsterisk:
				return EUnrealKeyCode::Multiply;
			case UIKeyboardHIDUsageKeypadPeriod:
				return EUnrealKeyCode::Decimal;
			default:
				return EUnrealKeyCode::Unknown;
			}
	}
}
