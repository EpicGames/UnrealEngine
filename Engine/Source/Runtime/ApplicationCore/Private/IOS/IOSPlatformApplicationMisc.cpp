// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOS/IOSPlatformApplicationMisc.h"

#include "HAL/PlatformProcess.h"
#include "Modules/ModuleManager.h"

#include "IOS/IOSApplication.h"
#include "IOS/IOSAppDelegate.h"
#include "IOS/IOSView.h"
#include "IOS/IOSInputInterface.h"
#include "IOS/IOSErrorOutputDevice.h"
#include "IOS/IOSFeedbackContext.h"

FIOSApplication* FIOSPlatformApplicationMisc::CachedApplication = nullptr;
FGetGamePadGlyphDelegate FIOSPlatformApplicationMisc::GetGamePadGlyphDelegate;

static TAutoConsoleVariable<int32> CVarPhysicalScreenDensity(
	TEXT("ios.PhysicalScreenDensity"),
	-1,
	TEXT("Physical screen density for this iOS device. Normally this is set by device profile.\n")
	TEXT("-1 = not set by device profile (default)\n")
	TEXT(" 0 = Specified as unknown (eg tvOS)")
	TEXT(">0 = Specified as exact value for device"));

EAppReturnType::Type MessageBoxExtImpl( EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption )
{
#if PLATFORM_TVOS || PLATFORM_VISIONOS
	return FGenericPlatformMisc::MessageBoxExt(MsgType, Text, Caption);
#else
	NSString* CocoaText = (NSString*)FPlatformString::TCHARToCFString(Text);
	NSString* CocoaCaption = (NSString*)FPlatformString::TCHARToCFString(Caption);

	NSMutableArray* StringArray = [NSMutableArray arrayWithCapacity:7];

	[StringArray addObject:CocoaCaption];
	[StringArray addObject:CocoaText];

	// Figured that the order of all of these should be the same as their enum name.
	switch (MsgType)
	{
		case EAppMsgType::YesNo:
			[StringArray addObject:@"Yes"];
			[StringArray addObject:@"No"];
			break;
		case EAppMsgType::OkCancel:
			[StringArray addObject:@"Ok"];
			[StringArray addObject:@"Cancel"];
			break;
		case EAppMsgType::YesNoCancel:
			[StringArray addObject:@"Yes"];
			[StringArray addObject:@"No"];
			[StringArray addObject:@"Cancel"];
			break;
		case EAppMsgType::CancelRetryContinue:
			[StringArray addObject:@"Cancel"];
			[StringArray addObject:@"Retry"];
			[StringArray addObject:@"Continue"];
			break;
		case EAppMsgType::YesNoYesAllNoAll:
			[StringArray addObject:@"Yes"];
			[StringArray addObject:@"No"];
			[StringArray addObject:@"Yes To All"];
			[StringArray addObject:@"No To All"];
			break;
		case EAppMsgType::YesNoYesAllNoAllCancel:
			[StringArray addObject:@"Yes"];
			[StringArray addObject:@"No"];
			[StringArray addObject:@"Yes To All"];
			[StringArray addObject:@"No To All"];
			[StringArray addObject:@"Cancel"];
			break;
		case EAppMsgType::YesNoYesAll:
			[StringArray addObject : @"Yes"];
			[StringArray addObject : @"No"];
			[StringArray addObject : @"Yes To All"];
			break;
		default:
			[StringArray addObject:@"Ok"];
			break;
	}

	IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];
	
	// reset our response to unset
	AppDelegate.AlertResponse = -1;

	[AppDelegate performSelectorOnMainThread:@selector(ShowAlert:) withObject:StringArray waitUntilDone:NO];

	while (AppDelegate.AlertResponse == -1)
	{
		FPlatformProcess::Sleep(0.1f);
	}

	EAppReturnType::Type Result = (EAppReturnType::Type)AppDelegate.AlertResponse;

	// Need to remap the return type to the correct one, since AlertResponse actually returns a button index.
	switch (MsgType)
	{
	case EAppMsgType::YesNo:
		Result = (Result == EAppReturnType::No ? EAppReturnType::Yes : EAppReturnType::No);
		break;
	case EAppMsgType::OkCancel:
		// return 1 for Ok, 0 for Cancel
		Result = (Result == EAppReturnType::No ? EAppReturnType::Ok : EAppReturnType::Cancel);
		break;
	case EAppMsgType::YesNoCancel:
		// return 0 for Yes, 1 for No, 2 for Cancel
		if(Result == EAppReturnType::No)
		{
			Result = EAppReturnType::Yes;
		}
		else if(Result == EAppReturnType::Yes)
		{
			Result = EAppReturnType::No;
		}
		else
		{
			Result = EAppReturnType::Cancel;
		}
		break;
	case EAppMsgType::CancelRetryContinue:
		// return 0 for Cancel, 1 for Retry, 2 for Continue
		if(Result == EAppReturnType::No)
		{
			Result = EAppReturnType::Cancel;
		}
		else if(Result == EAppReturnType::Yes)
		{
			Result = EAppReturnType::Retry;
		}
		else
		{
			Result = EAppReturnType::Continue;
		}
		break;
	case EAppMsgType::YesNoYesAllNoAll:
		// return 0 for No, 1 for Yes, 2 for YesToAll, 3 for NoToAll
		break;
	case EAppMsgType::YesNoYesAllNoAllCancel:
		// return 0 for No, 1 for Yes, 2 for YesToAll, 3 for NoToAll, 4 for Cancel
		break;
	case EAppMsgType::YesNoYesAll:
		// return 0 for No, 1 for Yes, 2 for YesToAll
		break;
	default:
		Result = EAppReturnType::Ok;
		break;
	}

	CFRelease((CFStringRef)CocoaCaption);
	CFRelease((CFStringRef)CocoaText);

	return Result;
#endif
}

void FIOSPlatformApplicationMisc::LoadPreInitModules()
{
	FModuleManager::Get().LoadModule(TEXT("IOSAudio"));
	FModuleManager::Get().LoadModule(TEXT("AudioMixerAudioUnit"));
}

class FFeedbackContext* FIOSPlatformApplicationMisc::GetFeedbackContext()
{
	static FIOSFeedbackContext Singleton;
	return &Singleton;
}

class FOutputDeviceError* FIOSPlatformApplicationMisc::GetErrorOutputDevice()
{
	static FIOSErrorOutputDevice Singleton;
	return &Singleton;
}

GenericApplication* FIOSPlatformApplicationMisc::CreateApplication()
{
	CachedApplication = FIOSApplication::CreateIOSApplication();
	return CachedApplication;
}

bool FIOSPlatformApplicationMisc::IsScreensaverEnabled()
{
	IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];
	return [AppDelegate IsIdleTimerEnabled];
}

bool FIOSPlatformApplicationMisc::ControlScreensaver(EScreenSaverAction Action)
{
	IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];
	[AppDelegate EnableIdleTimer : (Action == FGenericPlatformApplicationMisc::Enable)];
	return true;
}

void FIOSPlatformApplicationMisc::SetGamepadsAllowed(bool bAllowed)
{
	if (FIOSInputInterface* InputInterface = (FIOSInputInterface*)CachedApplication->GetInputInterface())
	{
		InputInterface->SetGamepadsAllowed(bAllowed);
	}
}

void FIOSPlatformApplicationMisc::SetGamepadsBlockDeviceFeedback(bool bBlock)
{
	if (FIOSInputInterface* InputInterface = (FIOSInputInterface*)CachedApplication->GetInputInterface())
	{
		InputInterface->SetGamepadsBlockDeviceFeedback(bBlock);
	}
}

void FIOSPlatformApplicationMisc::ResetGamepadAssignments()
{
	UE_LOGF(LogIOS, Warning, "Restting gamepad assignments is not allowed in IOS")
}

void FIOSPlatformApplicationMisc::ResetGamepadAssignmentToController(int32 ControllerId)
{
	
}

bool FIOSPlatformApplicationMisc::IsControllerAssignedToGamepad(int32 ControllerId)
{
	FIOSInputInterface* InputInterface = (FIOSInputInterface*)CachedApplication->GetInputInterface();
	return InputInterface->IsControllerAssignedToGamepad(ControllerId);
}

class UTexture2D* FIOSPlatformApplicationMisc::GetGamepadButtonGlyph(const FGamepadKeyNames::Type& ButtonKey, uint32 ControllerIndex)
{
	if (GetGamePadGlyphDelegate.IsBound())
	{
        return GetGamePadGlyphDelegate.Execute(ButtonKey, ControllerIndex);
	}
	return nullptr;
}

void FIOSPlatformApplicationMisc::EnableMotionData(bool bEnable)
{
	FIOSInputInterface* InputInterface = (FIOSInputInterface*)CachedApplication->GetInputInterface();
	return InputInterface->EnableMotionData(bEnable);
}

bool FIOSPlatformApplicationMisc::IsMotionDataEnabled()
{
	const FIOSInputInterface* InputInterface = (const FIOSInputInterface*)CachedApplication->GetInputInterface();
	return InputInterface->IsMotionDataEnabled();
}

void FIOSPlatformApplicationMisc::ClipboardCopy(const TCHAR* Str)
{
#if !PLATFORM_TVOS
	CFStringRef CocoaString = FPlatformString::TCHARToCFString(Str);
	UIPasteboard* Pasteboard = [UIPasteboard generalPasteboard];
	[Pasteboard setString:(NSString*)CocoaString];
#endif
}

void FIOSPlatformApplicationMisc::ClipboardPaste(class FString& Result)
{
#if !PLATFORM_TVOS
	UIPasteboard* Pasteboard = [UIPasteboard generalPasteboard];
	NSString* CocoaString = [Pasteboard string];
	if(CocoaString)
	{
		TArray<TCHAR> Ch;
		Ch.AddUninitialized([CocoaString length] + 1);
		FPlatformString::CFStringToTCHAR((CFStringRef)CocoaString, Ch.GetData());
		Result = Ch.GetData();
	}
	else
	{
		Result = TEXT("");
	}
#endif
}

EScreenPhysicalAccuracy FIOSPlatformApplicationMisc::ComputePhysicalScreenDensity(int32& ScreenDensity)
{
	// look up the current scale factor
	double ContentScaleFactor = 0;
	if (const UIView* View = [IOSAppDelegate GetDelegate].IOSView)
	{
		ContentScaleFactor = View.contentScaleFactor;
	}

	// If we have a cvar set by a device profile, return it.
	ScreenDensity = CVarPhysicalScreenDensity.GetValueOnAnyThread();
	if (ScreenDensity == 0)
	{
		return EScreenPhysicalAccuracy::Unknown;
	}
	else if (ScreenDensity > 0)
	{
		if (ContentScaleFactor != 0)
		{
			// The view may not use the full resolution of the screen so we scale the screen density accordingly
			ScreenDensity = FMath::TruncToInt(ScreenDensity / ContentScaleFactor);
		}
		return EScreenPhysicalAccuracy::Truth;
	}

	// If it hasn't been set, assume that the density is a multiple of the 
	// native Content Scaling Factor.  Won't be exact, but should be close enough.
#if PLATFORM_VISIONOS
	const double NativeScale = 1.0;
#else
	const double NativeScale =[[UIScreen mainScreen] scale];
#endif
	ScreenDensity = FMath::TruncToInt(163 * NativeScale);

	if (ContentScaleFactor != 0)
	{
		ScreenDensity = FMath::TruncToInt(ScreenDensity * (ContentScaleFactor / NativeScale));
	}

	return EScreenPhysicalAccuracy::Approximation;
}

EScreenPhysicalAccuracy FIOSPlatformApplicationMisc::ComputePhysicalScreenDimensions(FVector2f& OutScreenDimensions, float& OutScreenSize)
{
#if PLATFORM_VISIONOS
	OutScreenDimensions = FVector2f::ZeroVector;
	OutScreenSize = 0.f;
	return EScreenPhysicalAccuracy::Unknown;
#else
	EScreenPhysicalAccuracy Accuracy = EScreenPhysicalAccuracy::Unknown;
	int32 ScreenDensity = CVarPhysicalScreenDensity.GetValueOnAnyThread();
	if (ScreenDensity > 0)
	{
		Accuracy = EScreenPhysicalAccuracy::Truth;
	}
	else
	{
		UE_LOGF(LogIOS, Warning, "ComputePhysicalScreenDimensions : Approximating ScreenDensity due to CVarPhysicalScreenDensity not set or unknown (%d).", ScreenDensity);

		const double NativeScale = [[UIScreen mainScreen] scale];
		ScreenDensity = FMath::TruncToInt(163 * NativeScale);
		if (ScreenDensity <= 0)
		{
			UE_LOGF(LogIOS, Warning, "ComputePhysicalScreenDimensions : Incorrect ScreenDensity resulting in Unknown ScreenPhysicalAccuracy. ScreenDensity: %d", ScreenDensity);
			return EScreenPhysicalAccuracy::Unknown;
		}

		Accuracy = EScreenPhysicalAccuracy::Approximation;
	}

	CGRect NativeBounds = [[UIScreen mainScreen] nativeBounds];
	const FVector2f NativeResolution = FVector2f((float)NativeBounds.size.width, (float)NativeBounds.size.height);
	OutScreenDimensions = NativeResolution / (float)ScreenDensity;
	OutScreenSize = OutScreenDimensions.Size();

	UE_LOGF(LogIOS, Display, "ComputePhysicalScreenDimensions - NativeResolution: %ls, ScreenDensity: %d, OutScreenDimensions (inches): %ls, OutScreenSize (inches): %f",
		*NativeResolution.ToString(),
		ScreenDensity,
		*OutScreenDimensions.ToString(),
		OutScreenSize);

	return Accuracy;
#endif
}

bool FIOSPlatformApplicationMisc::RequiresVirtualKeyboard()
{
	{
		FIOSApplication* IOSApplication = [IOSAppDelegate GetDelegate].IOSApplication;
		FIOSInputInterface* InputInterface = IOSApplication ? IOSApplication->GetIOSInputInterface() : nullptr;
		if (ensure(InputInterface))
		{
			return !InputInterface->IsAnyPhysicalKeyboardConnected();
		}
	}
	
	// Falback to showing the virtual keyboard
	return true;
}
