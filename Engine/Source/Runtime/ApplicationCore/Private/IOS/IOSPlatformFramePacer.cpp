// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOS/IOSPlatformFramePacer.h"
#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "IOS/IOSSystemIncludes.h"
#endif
#include "Containers/Array.h"
#include "HAL/ThreadingBase.h"
#include "HAL/IConsoleManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"

// Collection of events listening for this trigger.
static TArray<FEvent*> ListeningEvents;
static FCriticalSection HandlersMutex;
static NSMutableSet<FIOSFramePacerHandler>* Handlers = [NSMutableSet new];

namespace IOSDisplayConstants
{
	const uint32 MaxRefreshRate = 60;
}

/*******************************************************************
 * FIOSFramePacer implementation
 *******************************************************************/

@interface FIOSFramePacer : NSObject
{
    @public
	FEvent *FramePacerEvent;
	int EnablePresentPacing;
}

-(void)run:(id)param;
-(void)signal:(id)param;

@end

@implementation FIOSFramePacer

#ifdef __GNUC__
#pragma GCC diagnostic push
#pragma GCC diagnostic ignored "-Wdeprecated-declarations"
#endif
-(void)run:(id)param
{
	NSRunLoop *runloop = [NSRunLoop currentRunLoop];
	CADisplayLink *displayLink = [CADisplayLink displayLinkWithTarget:self selector:@selector(signal:)];
	if ([displayLink respondsToSelector : @selector(preferredFramesPerSecond)] == YES)
	{
		displayLink.preferredFramesPerSecond = EnablePresentPacing ? 0 : (FIOSPlatformRHIFramePacer::GetMaxRefreshRate() / FIOSPlatformRHIFramePacer::FrameInterval);
	}

	[displayLink addToRunLoop:runloop forMode:NSDefaultRunLoopMode];
	[runloop run];
}
#ifdef __GNUC__
#pragma GCC diagnostic pop
#endif


-(void)signal:(id)param
{
	// during shutdown, this can cause crashes (only non-backgrounding apps do this)
	if (IsEngineExitRequested())
	{
		for( auto& NextEvent : ListeningEvents )
		{
			NextEvent->Trigger();
		}
		
		// Allow ListeningEvents to keep firing indefinitely during Engine Exit so that the
		// RHIThread never hangs waiting due to frame limiting in FMetalDevice::EndDrawingViewport

		return;
	};

	static IConsoleVariable* PresentPacingCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("ios.PresentPacing"));
	EnablePresentPacing = 0;
	if (PresentPacingCVar != nullptr)
	{
		EnablePresentPacing = PresentPacingCVar->GetInt();
	}
	
	{
		FScopeLock Lock(&HandlersMutex);
		CADisplayLink* displayLink = (CADisplayLink*)param;
		double OutputSeconds = displayLink.duration + displayLink.timestamp;
		OutputSeconds = displayLink.targetTimestamp;
		double OutputDuration = displayLink.duration;
		for (FIOSFramePacerHandler Handler in Handlers)
		{
			Handler(0, OutputSeconds, OutputDuration);
		}

		// If requested, get the latest frame sync interval and clamp to the project setting
		static TConsoleVariableData<int32>* VSyncCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.vsync"));
		static TConsoleVariableData<int32>* VSyncIntervalCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("rhi.syncinterval"));

		if (VSyncCVar && VSyncCVar->GetValueOnRenderThread() > 0)
		{
			uint32 NewFrameInterval = VSyncIntervalCVar ? VSyncIntervalCVar->GetValueOnRenderThread() : FIOSPlatformRHIFramePacer::FrameInterval;

			// If changed, update the display link
			if (NewFrameInterval != FIOSPlatformRHIFramePacer::FrameInterval)
			{
				
				FIOSPlatformRHIFramePacer::FrameInterval = NewFrameInterval;
				uint32 MaxRefreshRate = FIOSPlatformRHIFramePacer::GetMaxRefreshRate();

				CADisplayLink* displayLinkParam = (CADisplayLink*)param;
				
				if (displayLinkParam.preferredFramesPerSecond > 0)
				{
					displayLinkParam.preferredFramesPerSecond = MaxRefreshRate / FIOSPlatformRHIFramePacer::FrameInterval;
				}

				// Update pacing for present
				FIOSPlatformRHIFramePacer::Pace = MaxRefreshRate / FIOSPlatformRHIFramePacer::FrameInterval;
			}
		}
	}	
    for( auto& NextEvent : ListeningEvents )
    {
        NextEvent->Trigger();
    }
}

@end



/*******************************************************************
 * FIOSPlatformRHIFramePacer implementation
 *******************************************************************/

uint32 FIOSPlatformRHIFramePacer::FrameInterval = 1;
uint32 FIOSPlatformRHIFramePacer::MinFrameInterval = 1;
FIOSFramePacer* FIOSPlatformRHIFramePacer::FramePacer = nil;
uint32 FIOSPlatformRHIFramePacer::Pace = 0;


bool FIOSPlatformRHIFramePacer::IsEnabled()
{
#if PLATFORM_VISIONOS
	return false; // XR does its own frame pacing.
#else
    static bool bIsRHIFramePacerEnabled = false;
	static bool bInitialized = false;

	if (!bInitialized)
	{
		FString FrameRateLockAsEnum;
		GConfig->GetString(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("FrameRateLock"), FrameRateLockAsEnum, GEngineIni);

		uint32 MaxRefreshRate = GetMaxRefreshRate();
		uint32 FrameRateLock = MaxRefreshRate;
		FParse::Value(*FrameRateLockAsEnum, TEXT("PUFRL_"), FrameRateLock);

        const bool bOverridesFrameRate = FParse::Value( FCommandLine::Get(), TEXT( "FrameRateLock=" ), FrameRateLockAsEnum );
        if (bOverridesFrameRate)
        {
            FParse::Value(*FrameRateLockAsEnum, TEXT("PUFRL_"), FrameRateLock);
        }
        
        if (FrameRateLock == 0)
        {
            FrameRateLock = MaxRefreshRate;
        }

        if (!bIsRHIFramePacerEnabled)
		{
			check((MaxRefreshRate % FrameRateLock) == 0);
			FrameInterval = MaxRefreshRate / FrameRateLock;
			MinFrameInterval = FrameInterval;

			bIsRHIFramePacerEnabled = (FrameInterval > 0);
			
			// remember the Pace if we are enabled
			Pace = bIsRHIFramePacerEnabled ? FrameRateLock : 0;
		}
		bInitialized = true;
	}
	
	return bIsRHIFramePacerEnabled;
#endif
}

uint32 FIOSPlatformRHIFramePacer::GetMaxRefreshRate()
{
#if PLATFORM_VISIONOS
	return IOSDisplayConstants::MaxRefreshRate;
#else
	static bool bEnableDynamicMaxFPS = false;
	static bool bInitialized = false;
	
	if (!bInitialized)
	{
		GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bEnableDynamicMaxFPS"), bEnableDynamicMaxFPS, GEngineIni);
		bInitialized = true;
	}
	
	return bEnableDynamicMaxFPS ? [UIScreen mainScreen].maximumFramesPerSecond : IOSDisplayConstants::MaxRefreshRate;
#endif
}

bool FIOSPlatformRHIFramePacer::SupportsFramePace(int32 QueryFramePace)
{
	// Support frame rates that are an integer multiple of max refresh rate, or 0 for no pacing
	return QueryFramePace == 0 || (GetMaxRefreshRate() % QueryFramePace) == 0;
}

int32 FIOSPlatformRHIFramePacer::SetFramePace(int32 InFramePace)
{
	int32 NewPace = 0;

	static IConsoleVariable* SyncIntervalCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("rhi.SyncInterval"));
	if (ensure(SyncIntervalCVar != nullptr))
	{
		int32 MaxRefreshRate = GetMaxRefreshRate();
		int32 NewSyncInterval = InFramePace > 0 ? MaxRefreshRate / InFramePace : 0;
		SyncIntervalCVar->Set(NewSyncInterval, ECVF_SetByCode);

		if (NewSyncInterval > 0)
		{
			NewPace = MaxRefreshRate / NewSyncInterval;
		}
	}
	return NewPace;
}

int32 FIOSPlatformRHIFramePacer::GetFramePace()
{
	int32 CurrentPace = 0;
	static IConsoleVariable* SyncIntervalCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("rhi.SyncInterval"));
	if (ensure(SyncIntervalCVar != nullptr))
	{
		int SyncInterval = SyncIntervalCVar->GetInt();
		if (SyncInterval > 0)
		{
			CurrentPace = GetMaxRefreshRate() / SyncInterval;
		}
	}
	return CurrentPace;
}

void FIOSPlatformRHIFramePacer::InitWithEvent(FEvent* TriggeredEvent)
{
    // Create display link thread
    FramePacer = [[FIOSFramePacer alloc] init];
    [NSThread detachNewThreadSelector:@selector(run:) toTarget:FramePacer withObject:nil];
        
    // Only one supported for now, we may want more eventually.
    ListeningEvents.Add( TriggeredEvent );
}

void FIOSPlatformRHIFramePacer::AddHandler(FIOSFramePacerHandler Handler)
{
	check (FramePacer);
	FScopeLock Lock(&HandlersMutex);
	FIOSFramePacerHandler Copy = Block_copy(Handler);
	[Handlers addObject:Copy];
	Block_release(Copy);
}

void FIOSPlatformRHIFramePacer::RemoveHandler(FIOSFramePacerHandler Handler)
{
	check (FramePacer);
	FScopeLock Lock(&HandlersMutex);
	[Handlers removeObject:Handler];
}

void FIOSPlatformRHIFramePacer::Suspend()
{
    // send a signal to the events if we are enabled
    if (IsEnabled())
    {
        [FramePacer signal:0];
    }
}

void FIOSPlatformRHIFramePacer::Resume()
{
    
}

void FIOSPlatformRHIFramePacer::Destroy()
{
    if( FramePacer != nil )
    {
        [FramePacer release];
        FramePacer = nil;
    }
}
