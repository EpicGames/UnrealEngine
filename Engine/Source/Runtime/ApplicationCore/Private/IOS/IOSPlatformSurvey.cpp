// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	IOSPlatformSurvey.cpp: HardwareSurvey implementation
=============================================================================*/

#include "IOS/IOSPlatformSurvey.h"
#include "Apple/AppleStringUtils.h"
#include "IOS/IOSAppDelegate.h"
#include "IOSWindow.h"

bool FIOSPlatformSurvey::GetSurveyResults(FHardwareSurveyResults& OutResults, bool bWait)
{
	FCString::Strcpy(OutResults.Platform, FPlatformMisc::GetDefaultDeviceProfileName());
	FCString::Strcpy(OutResults.OSVersion, *FString([[UIDevice currentDevice] systemVersion]));
	OutResults.OSBits = 64;
	FCString::Strcpy(OutResults.OSLanguage, *FString([[NSLocale preferredLanguages] objectAtIndex:0]));
	FCString::Strcpy(OutResults.RenderingAPI, TEXT("Metal"));
	OutResults.CPUCount = FPlatformMisc::NumberOfCores();

#if !PLATFORM_VISIONOS
	// display 0 is max size
	CGRect MainFrame = [[UIScreen mainScreen] bounds];
	double Scale = [[UIScreen mainScreen] scale];
	OutResults.Displays[0].CurrentModeWidth = FMath::TruncToInt(MainFrame.size.width * Scale);
	OutResults.Displays[0].CurrentModeHeight = FMath::TruncToInt(MainFrame.size.height * Scale);

	// display 1 is current size
	FPlatformRect ScreenSize = FIOSWindow::GetScreenRect();
	OutResults.Displays[1].CurrentModeWidth = ScreenSize.Right - ScreenSize.Left;
	OutResults.Displays[1].CurrentModeHeight = ScreenSize.Bottom - ScreenSize.Top;
#endif
	return true;
}
