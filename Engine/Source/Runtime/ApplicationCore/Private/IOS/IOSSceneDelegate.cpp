// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOS/IOSSceneDelegate.h"

#if UE_IOS_SCENE_LIFECYCLE

#include "IOS/IOSAppDelegate.h"

extern bool GShowSplashScreen;

@implementation IOSSceneDelegate

@synthesize window = _window;

- (void)scene:(UIScene *)scene willConnectToSession:(UISceneSession *)session options:(UISceneConnectionOptions *)connectionOptions
{
	if (![scene isKindOfClass:[UIWindowScene class]])
	{
		return;
	}

	UIWindowScene* WindowScene = (UIWindowScene*)scene;
	IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];

	self.window = [[UIWindow alloc] initWithWindowScene:WindowScene];
	[self.window makeKeyAndVisible];

	// Mirror the window onto the app delegate so existing engine code paths that read
	// [IOSAppDelegate GetDelegate].Window (notably FAppEntry::PreInit, IOSView, FIOSApplication)
	// keep working without modification.
	AppDelegate.Window = self.window;

#if !PLATFORM_TVOS
	AppDelegate.InterfaceOrientation = [WindowScene interfaceOrientation];
	AppDelegate.bDeviceInPortraitMode = UIInterfaceOrientationIsPortrait(AppDelegate.InterfaceOrientation);
#endif

#if !BUILD_EMBEDDED_APP
	FAppEntry::PreInit(AppDelegate, [UIApplication sharedApplication]);

	UIStoryboard *storyboard = [UIStoryboard storyboardWithName:@"LaunchScreen" bundle:nil];
	if (storyboard != nil)
	{
		AppDelegate.viewController = [storyboard instantiateViewControllerWithIdentifier:@"LaunchScreen"];
		AppDelegate.viewController.view.tag = 200;
		[self.window addSubview:AppDelegate.viewController.view];
		GShowSplashScreen = true;
	}

	AppDelegate.timer = [NSTimer scheduledTimerWithTimeInterval:0.05f target:AppDelegate selector:@selector(timerForSplashScreen) userInfo:nil repeats:YES];

	[AppDelegate StartGameThread];

	AppDelegate.CommandLineParseTimer = [NSTimer scheduledTimerWithTimeInterval:0.01f target:AppDelegate selector:@selector(NoUrlCommandLine) userInfo:nil repeats:NO];
#endif

	// Forward any URL contexts that arrived as part of the scene connection (cold-launch deep link).
	if (connectionOptions.URLContexts.count > 0)
	{
		[self scene:scene openURLContexts:connectionOptions.URLContexts];
	}

#if WITH_IOS_UNIVERSAL_LINKS
	for (NSUserActivity* Activity in connectionOptions.userActivities)
	{
		[AppDelegate handleContinueUserActivity:Activity];
	}
#endif
}

- (void)sceneDidDisconnect:(UIScene *)scene
{
	// Single-window app: scenes only disconnect when the system reclaims memory or the user
	// explicitly closes the window. Treat it like backgrounding for engine state.
	[[IOSAppDelegate GetDelegate] handleDidEnterBackground];
}

- (void)sceneDidBecomeActive:(UIScene *)scene
{
	[[IOSAppDelegate GetDelegate] handleDidBecomeActive];
}

- (void)sceneWillResignActive:(UIScene *)scene
{
	[[IOSAppDelegate GetDelegate] handleWillResignActive];
}

- (void)sceneWillEnterForeground:(UIScene *)scene
{
	[[IOSAppDelegate GetDelegate] handleWillEnterForeground];
}

- (void)sceneDidEnterBackground:(UIScene *)scene
{
	[[IOSAppDelegate GetDelegate] handleDidEnterBackground];
}

- (void)scene:(UIScene *)scene openURLContexts:(NSSet<UIOpenURLContext *> *)URLContexts
{
	IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];
	for (UIOpenURLContext* Context in URLContexts)
	{
		[AppDelegate handleOpenURL:Context.URL
				sourceApplication:Context.options.sourceApplication
						annotation:Context.options.annotation];
	}
}

#if WITH_IOS_UNIVERSAL_LINKS
- (void)scene:(UIScene *)scene continueUserActivity:(NSUserActivity *)userActivity
{
	[[IOSAppDelegate GetDelegate] handleContinueUserActivity:userActivity];
}
#endif

@end

#endif // UE_IOS_SCENE_LIFECYCLE
