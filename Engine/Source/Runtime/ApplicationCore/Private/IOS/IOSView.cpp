// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOS/IOSView.h"
#include "IOS/IOSAppDelegate.h"
#include "IOS/IOSApplication.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CommandLine.h"
#include "IOS/IOSPlatformProcess.h"

#import "IOS/IOSAsyncTask.h"
#import <QuartzCore/QuartzCore.h>
#import <UIKit/UIGeometry.h>
#import <UIKit/UIKeyConstants.h>

#include "IOS/IOSCommandLineHelper.h"
#include "IOS/IOSInputDefinitions.h"
#include "IOS/IOSInputUtils.h"
#include "HAL/PlatformApplicationMisc.h"

#if WITH_ACCESSIBILITY
#include "IOS/Accessibility/IOSAccessibilityCache.h"
#include "IOS/Accessibility/IOSAccessibilityElement.h"
#endif

static int32 GIOSMetalCacheLayer = 1;
static FAutoConsoleVariableRef CVarIOSMetalCacheLayer(
	TEXT("ios.Metal.CacheLayer"),
	GIOSMetalCacheLayer,
	TEXT("Whether to cache Metal layer instance. (default = 1)\n"),
	ECVF_ReadOnly
);

namespace MTL
{
    class Device;
}

MTL::Device* GMetalDevice = nullptr;

@interface IndexedPosition : UITextPosition {
	NSUInteger _index;
	id <UITextInputDelegate> _inputDelegate;
}
@property (nonatomic) NSUInteger index;
+ (IndexedPosition *)positionWithIndex:(NSUInteger)index;
@end

@interface IndexedRange : UITextRange {
	NSRange _range;
}
@property (nonatomic) NSRange range;
+ (IndexedRange *)rangeWithNSRange:(NSRange)range;

@end


@implementation IndexedPosition
@synthesize index = _index;

+ (IndexedPosition *)positionWithIndex:(NSUInteger)index {
	IndexedPosition *pos = [[IndexedPosition alloc] init];
	pos.index = index;
	return pos;
}

@end

@implementation IndexedRange
@synthesize range = _range;

+ (IndexedRange *)rangeWithNSRange:(NSRange)nsrange {
	if (nsrange.location == NSNotFound)
		return nil;
	IndexedRange *range = [[IndexedRange alloc] init];
	range.range = nsrange;
	return range;
}

- (UITextPosition *)start {
	return [IndexedPosition positionWithIndex:self.range.location];
}

- (UITextPosition *)end {
	return [IndexedPosition positionWithIndex:(self.range.location + self.range.length)];
}

-(BOOL)isEmpty {
	return (self.range.length == 0);
}
@end



@implementation FIOSView

@synthesize keyboardType = KeyboardType;
@synthesize autocorrectionType = AutocorrectionType;
@synthesize autocapitalizationType = AutocapitalizationType;
@synthesize secureTextEntry = bSecureTextEntry;
@synthesize SwapCount, markedTextStyle;


#if BUILD_EMBEDDED_APP

+(void)StartupEmbeddedUnreal
{
	// special initialization code for embedded view
	
	//// LaunchIOS replacement ///
	FIOSCommandLineHelper::InitCommandArgs(TEXT(""));
	
	//#if !PLATFORM_TVOS
	//		// reset badge count on launch
	//		Application.applicationIconBadgeNumber = 0;
	//#endif
	
	IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];

	[AppDelegate NoUrlCommandLine];
	[AppDelegate StartGameThread];
}

#endif

/**
 * @return The Layer Class for the window
 */
+ (Class)layerClass
{
#if BUILD_EMBEDDED_APP || UE_USE_SWIFT_UI_MAIN
	SCOPED_BOOT_TIMING("MetalLayer class");
	GMetalDevice = (__bridge MTL::Device*)MTLCreateSystemDefaultDevice();
	return [CAMetalLayer class];
#endif
	
	// make sure the project setting has enabled Metal support (per-project user settings in the editor)
	bool bSupportsMetal = false;
	bool bSupportsMetalMobileSM5 = false;
	bool bSupportsMetalMobileSM6 = false;
	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsMetal"), bSupportsMetal, GEngineIni);
	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsMetalMobileSM5"), bSupportsMetalMobileSM5, GEngineIni);
	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsMetalMobileSM6"), bSupportsMetalMobileSM6, GEngineIni);

	bool bTriedToInit = false;

	// the check for the function pointer itself is to determine if the Metal framework exists, before calling it
	if ((bSupportsMetal || bSupportsMetalMobileSM5 || bSupportsMetalMobileSM6) && MTLCreateSystemDefaultDevice != NULL)
	{
		// if the device is unable to run with Metal (pre-A7), this will return nullptr
		GMetalDevice = (__bridge MTL::Device*)MTLCreateSystemDefaultDevice();

		// just tracking for printout below
		bTriedToInit = true;
	}

#if !UE_BUILD_SHIPPING
	if (GMetalDevice == nullptr)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Not using Metal because: [Project Settings Disabled Metal? %s :: Commandline Forced ES2? %s :: Older OS? %s :: Pre-A7 Device? %s]"),
			bSupportsMetal ? TEXT("No") : TEXT("Yes"),
			TEXT("No"),
			(MTLCreateSystemDefaultDevice == NULL) ? TEXT("Yes") : TEXT("No"),
			bTriedToInit ? TEXT("Yes") : TEXT("Unknown (didn't test)"));
	}
#endif

	if (GMetalDevice != nullptr)
	{
		return [CAMetalLayer class];
	}
	else
	{
		return nil;
	}
}

- (id)initInternal:(CGRect)Frame
{
	CachedMarkedText = nil;

	check(GMetalDevice);
	// if the device is valid, we know Metal is usable (see +layerClass)
	MetalDevice = GMetalDevice;
	if (MetalDevice != nullptr)
	{
		// grab the MetalLayer and typecast it to match what's in layerClass
		CAMetalLayer* MetalLayer = (CAMetalLayer*)self.layer;
		MetalLayer.presentsWithTransaction = NO;
		MetalLayer.drawsAsynchronously = YES;
		
		// set a background color to make sure the layer appears
		CGFloat components[] = { 0.0, 0.0, 0.0, 1 };
		MetalLayer.backgroundColor = CGColorCreate(CGColorSpaceCreateDeviceRGB(), components);
		
		// set the device on the rendering layer and provide a pixel format
		MetalLayer.device = (__bridge id<MTLDevice>)MetalDevice;
		MetalLayer.pixelFormat = MTLPixelFormatBGRA8Unorm;
		MetalLayer.framebufferOnly = NO;
		
		if (GIOSMetalCacheLayer > 0)
		{
			CachedMetalLayer = MetalLayer;
		}

		NSLog(@"::: Created a UIView that will support Metal :::");
	}
	
	
#if !PLATFORM_TVOS
	SupportedInterfaceOrientations = UIInterfaceOrientationMaskAll;
	self.multipleTouchEnabled = YES;
#endif

	SwapCount = 0;
	FMemory::Memzero(AllTouches, sizeof(AllTouches));
	[self setAutoresizingMask: UIViewAutoresizingFlexibleWidth | UIViewAutoresizingFlexibleHeight];
	bIsInitialized = false;
	
	[self InitKeyboard];
	
#if BUILD_EMBEDDED_APP || UE_USE_SWIFT_UI_MAIN 
	//// FAppEntry::PreInit replacement ///
	IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];
	
	AppDelegate.RootView = self;
	while (AppDelegate.RootView.superview != nil)
	{
		AppDelegate.RootView = AppDelegate.RootView.superview;
	}
	AppDelegate.IOSView = self;
	
	// initialize the backbuffer of the view (so the RHI can use it)
	[self CreateFramebuffer];
	
#endif

#if !PLATFORM_TVOS
	bool bEnableGestureRecognizer = false;
	GConfig->GetBool(TEXT("/Script/Engine.InputSettings"), TEXT("bEnableGestureRecognizer"), bEnableGestureRecognizer, GInputIni);
	if (bEnableGestureRecognizer && FGenericPlatformApplicationMisc::AreNativeGesturesEnabled())
	{
		[self SetupGestureRecognizers];
	}
#endif

	return self;
}

- (id)initWithCoder:(NSCoder*)Decoder
{
	if ((self = [super initWithCoder:Decoder]))
	{
		self = [self initInternal:self.frame];
	}
	return self;
}

- (id)initWithFrame:(CGRect)Frame
{
	if ((self = [super initWithFrame:Frame]))
	{
		self = [self initInternal:self.frame];
	}
	return self;
}

-(void)dealloc
{
	[CachedMarkedText release];
	[markedTextStyle release];
	[super dealloc];
}

- (void)CalculateContentScaleFactor:(int32)ScreenWidth ScreenHeight:(int32)ScreenHeight
{
	const IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];
	const float mNativeScale = AppDelegate.NativeScale;
	const float RequestedContentScaleFactor = AppDelegate.MobileContentScaleFactor;
	const int32 RequestedResX = AppDelegate.RequestedResX;
	const int32 RequestedResY = AppDelegate.RequestedResY;
		
//	UE_LOGF(LogIOS, Log, "RequestedContentScaleFactor %f to nativeScale which is = (s:%f, ns:%f, csf:%f", RequestedContentScaleFactor, AppDelegate.ScreenScale, mNativeScale, self.contentScaleFactor);
	
	int32 Width = ScreenWidth;
	int32 Height = ScreenHeight;

	CGFloat currentScaleFactor = self.contentScaleFactor;
	CGFloat newScaleFactor = mNativeScale;
	
	// 0 means to use native size
	if (RequestedContentScaleFactor != 0.0f || RequestedResX > 0 || RequestedResY > 0)
	{
		float AspectRatio = (float)ScreenHeight / (float)ScreenWidth;
		if (RequestedResX > 0)
		{
			// set long side for orientation to requested X
			if (ScreenHeight > ScreenWidth)
			{
				Height = RequestedResX;
				Width = FMath::TruncToInt(static_cast<float>(Height) * AspectRatio + 0.5f);
			}
			else
			{
				Width = RequestedResX;
				Height = FMath::TruncToInt(static_cast<float>(Width) * AspectRatio + 0.5f);
			}
		}
		else if (RequestedResY > 0)
		{
			// set short side for orientation to requested Y
			if (ScreenHeight > ScreenWidth)
			{
				Width = RequestedResY;
				Height = FMath::TruncToInt(static_cast<float>(Width) * AspectRatio + 0.5f);
			}
			else
			{
				Height = RequestedResY;
				Width = FMath::TruncToInt(static_cast<float>(Height) * AspectRatio + 0.5f);
			}
		}
		else
		{
			newScaleFactor = RequestedContentScaleFactor;
		}
	}

	// As this can be called every tick(), and setting it requires being on the UI thread, only
	// update the value if it's actually changed.
	if (currentScaleFactor != newScaleFactor)
	{
		FIOSView* __block blockSelf = self;
		dispatch_async(dispatch_get_main_queue(), ^{
			blockSelf.contentScaleFactor = newScaleFactor;
			blockSelf = nil;
		});
	}

	_ViewSize.width = Width;
	_ViewSize.height = Height;
}

- (bool)CreateFramebuffer
{
	if (!bIsInitialized)
	{
		[self CalculateContentScaleFactor:FMath::TruncToInt(self.frame.size.width) ScreenHeight:FMath::TruncToInt(self.frame.size.height)];
		bIsInitialized = true;
	}
	return true;
}

/**
 * If view is resized, update the frame buffer so it is the same size as the display area.
 */
- (void)layoutSubviews
{
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
	// IOSApplication::OrientationChange calls into Slate which issues rendering commands
	if(![IOSAppDelegate GetDelegate].bIsSuspended)
	{
		if (!GIOSDelayRotationUntilPresent)
		{
			UIInterfaceOrientation CurrentOrientation = [self.window.windowScene interfaceOrientation];
			FIOSApplication::OrientationChanged(CurrentOrientation);
		}
	}
#endif
}

/**
 * This version always calls the OrientationChanged events. It's used when we change the ContentScaleFactor externally.
 */
- (void)forceLayoutSubviews
{
#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
	if(![IOSAppDelegate GetDelegate].bIsSuspended)
	{
		if (GIOSDelayRotationUntilPresent)
		{
			UIInterfaceOrientation CurrentOrientation = [self.window.windowScene interfaceOrientation];
			FIOSApplication::OrientationChanged(CurrentOrientation);
		}
	}
#endif
	[self layoutSubviews];
}


-(void)UpdateRenderWidth:(uint32)Width andHeight:(uint32)Height
{
	if (MetalDevice != nullptr)
	{
		// grab the MetalLayer and typecast it to match what's in layerClass, then set the new size
		CAMetalLayer* MetalLayer = (CAMetalLayer*)self.layer;
		MetalLayer.drawableSize = CGSizeMake(Width, Height);
	}
}

- (id<CAMetalDrawable>)MakeDrawable
{
	__block CAMetalLayer* MetalLayer = nil;
	if (GIOSMetalCacheLayer > 0)
	{
		MetalLayer = CachedMetalLayer;
		check(MetalLayer);
	}
	else
	{
		dispatch_sync(dispatch_get_main_queue(), ^ {
			MetalLayer = (CAMetalLayer*)[self layer];
			});
	}
	// this call cannot be made on the MainThread
	// thus requiring the code above to MainThreadCall here
	return [MetalLayer nextDrawable];
}

- (void)DestroyFramebuffer
{
	if (bIsInitialized)
	{
		// we are ready to be re-initialized
		bIsInitialized = false;
	}
}

- (void)SwapBuffers
{
	SwapCount++;
}

#if WITH_ACCESSIBILITY

-(void)SetAccessibilityWindow:(AccessibleWidgetId)WindowId
{
	[FIOSAccessibilityCache AccessibilityElementCache].RootWindowId = WindowId;
 	if (WindowId != IAccessibleWidget::InvalidAccessibleWidgetId)
	{
		FIOSAccessibilityLeaf* Window = [[FIOSAccessibilityCache AccessibilityElementCache] GetAccessibilityElement: WindowId];
		// We go ahead and assume the window will have children and add the FIOSAccessibilityContainer
		// for the Window to enforce accessibility hierarchy
self.accessibilityElements = @[Window.accessibilityContainer];
	}
	else
	{
		self.accessibilityElements = Nil;
	}
}

-(BOOL)isAccessibilityElement
{
	return NO;
}

#endif

/**
 * Returns the unique ID for the given touch
 */
-(int32) GetTouchIndex:(UITouch*)Touch
{
	// look for existing touch
	for (int Index = 0; Index < UE_ARRAY_COUNT(AllTouches); Index++)
	{
		if (AllTouches[Index] == Touch)
		{
			return Index;
		}
	}
	
	// if we get here, it's a new touch, find a slot
	for (int Index = 0; Index < UE_ARRAY_COUNT(AllTouches); Index++)
	{
		if (AllTouches[Index] == nil)
		{
			AllTouches[Index] = Touch;
			return Index;
		}
	}
	
	// if we get here, that means we are trying to use more than 5 touches at once, which is an error
	return -1;
}


-(void)HandleTouchAtLoc:(CGPoint)Loc PrevLoc:(CGPoint)PrevLoc TouchIndex:(int)TouchIndex Force:(float)Force Type:(TouchType)Type TouchesArray:(TArray<TouchInput>&)TouchesArray
{
	// init some things on begin
	if (Type == TouchBegan)
	{
		PreviousForces[TouchIndex] = -1.0;
		HasMoved[TouchIndex] = 0;
		
		NumActiveTouches++;
	}
	
	float PreviousForce = PreviousForces[TouchIndex];
	
	// make a new touch event struct
	TouchInput TouchMessage;
	TouchMessage.Handle = TouchIndex;
	TouchMessage.Type = Type;
	TouchMessage.Position = FVector2D(FMath::Min<double>(_ViewSize.width - 1, Loc.x), FMath::Min<double>(_ViewSize.height - 1, Loc.y));
	TouchMessage.LastPosition = FVector2D(FMath::Min<double>(_ViewSize.width - 1, PrevLoc.x), FMath::Min<double>(_ViewSize.height - 1, PrevLoc.y));
	TouchMessage.Force = Type != TouchEnded ? Force : 0.0f;
	
	// skip moves that didn't actually move - this will help input handling to skip over the first
	// move since it is likely a big pop from the TouchBegan location (iOS filters out small movements
	// on first press)
	if (Type != TouchMoved || (PrevLoc.x != Loc.x || PrevLoc.y != Loc.y))
	{
		// track first move event, for helping with "pop" on the filtered small movements
		if (HasMoved[TouchIndex] == 0 && Type == TouchMoved)
		{
			TouchInput FirstMoveMessage = TouchMessage;
			FirstMoveMessage.Type = FirstMove;
			HasMoved[TouchIndex] = 1;
			
			TouchesArray.Add(FirstMoveMessage);
		}
		
		TouchesArray.Add(TouchMessage);
	}
	
	// if the force changed, send an event!
	if (PreviousForce != Force)
	{
		TouchInput ForceMessage = TouchMessage;
		ForceMessage.Type = ForceChanged;
		PreviousForces[TouchIndex] = Force;
		
		TouchesArray.Add(ForceMessage);
	}
	
	// clear out the touch when it ends
	if (Type == TouchEnded)
	{
		AllTouches[TouchIndex] = nil;
		NumActiveTouches--;
	}

#if !UE_BUILD_SHIPPING
#if WITH_IOS_SIMULATOR
	// use 2 on the simulator so that Option-Click will bring up console (option-click is for doing pinch gestures, which we don't care about, atm)
	if( NumActiveTouches >= 2 )
#else
		// If there are 3 active touches, bring up the console
		if( NumActiveTouches >= 4 )
#endif
		{
			bool bShowConsole = true;
			GConfig->GetBool(TEXT("/Script/Engine.InputSettings"), TEXT("bShowConsoleOnFourFingerTap"), bShowConsole, GInputIni);
			
			if (bShowConsole)
			{
				// disable the integrated keyboard when launching the console
				/*			if (bIsUsingIntegratedKeyboard)
				 {
				 // press the console key twice to get the big one up
				 // @todo keyboard: Find a direct way to bering this up (it can get into a bad state where two presses won't do it correctly)
				 // and also the ` key could be changed via .ini
				 FIOSInputInterface::QueueKeyInput('`', '`');
				 FIOSInputInterface::QueueKeyInput('`', '`');
				 
				 [self ActivateKeyboard:true];
				 }
				 else*/
				{
					// Route the command to the main iOS thread (all UI must go to the main thread)
					[[IOSAppDelegate GetDelegate] performSelectorOnMainThread:@selector(ShowConsole) withObject:nil waitUntilDone:NO];
				}
			}
		}
#endif
}

/**
 * Pass touch events to the input queue for slate to pull off of, and trigger the debug console.
 *
 * @param View The view the event happened in
 * @param Touches Set of touch events from the OS
 */
-(void) HandleTouches:(NSSet*)Touches ofType:(TouchType)Type
{
	TArray<TouchInput> TouchesArray;
	for (UITouch* Touch in Touches)
	{
		// ignore mouse-produced touches, these will be handled by FIOSInputInterface
        if ( Touch.type == UITouchTypeIndirectPointer ) // Requires UIApplicationSupportsIndirectInputEvents:true in plist
        {
            continue;
        }
		// get info from the touch
		CGPoint Loc = [Touch locationInView:self];
		CGPoint PrevLoc = [Touch previousLocationInView:self];
		
		// View may have been modified via Cvars ("r.mobile.DesiredResX/Y" or CommandLine "mcfs, mobileresx/y"
		CGPoint ViewSizeModifier = CGPointMake(_ViewSize.width/self.frame.size.width, _ViewSize.height/self.frame.size.height);
		Loc.x *= ViewSizeModifier.x;
		Loc.y *= ViewSizeModifier.y;
		PrevLoc.x *= ViewSizeModifier.x;
		PrevLoc.y *= ViewSizeModifier.y;
			
		// convert Touch pointer to a unique 0 based index
		int32 TouchIndex = [self GetTouchIndex:Touch];
		if (TouchIndex < 0)
		{
			continue;
		}

		double Force = Touch.force;
		
		// map larger values to 1..10, so 10 is a max across platforms
		if (Force > 1.0)
		{
			Force = 10.0 * Force / Touch.maximumPossibleForce;
		}
		
		// Handle devices without force touch
		if ((Type == TouchBegan || Type == TouchMoved) && Force == 0.0)
		{
			Force = 1.0;
		}

		[self  HandleTouchAtLoc:Loc PrevLoc:PrevLoc TouchIndex:TouchIndex Force:(float)Force Type:Type TouchesArray:TouchesArray];
	}

	FIOSInputInterface::QueueTouchInput(TouchesArray);
}

/**
 * Handle the various touch types from the OS
 *
 * @param touches Array of touch events
 * @param event Event information
 */
- (void) touchesBegan:(NSSet*)touches withEvent:(UIEvent*)event 
{
	[self HandleTouches:touches ofType:TouchBegan];
	}

- (void) touchesMoved:(NSSet*)touches withEvent:(UIEvent*)event
{
	[self HandleTouches:touches ofType:TouchMoved];
}

- (void) touchesEnded:(NSSet*)touches withEvent:(UIEvent*)event
{
	[self HandleTouches:touches ofType:TouchEnded];
}

- (void) touchesCancelled:(NSSet*)touches withEvent:(UIEvent*)event
{
	[self HandleTouches:touches ofType:TouchEnded];
}


#pragma mark Keyboard


-(BOOL)canBecomeFirstResponder
{
	return YES;
}

- (BOOL)hasText
{
	return YES;
}

- (void)insertText:(NSString *)theText
{
	if (nil != CachedMarkedText) {
		[CachedMarkedText release];
		CachedMarkedText = nil;
	}
	
	TSharedPtr<FAppleKeyboardController> KeyboardController = ([self GetKeyboardController]).Pin();
	if (!KeyboardController)
	{
		return;
	}

	// insert text one key at a time, as chars, not keydowns
	for (int32 CharIndex = 0; CharIndex < [theText length]; CharIndex++)
	{
		int32 Char = [theText characterAtIndex:CharIndex];

		FAppleKeyboardController::FDeferredEvent KeyEvent;
		
		KeyEvent.CharCode = Char;

		if (Char == '\n')
		{
			// send the enter keypress
			KeyEvent.TranslatedKeyCode = static_cast<uint32>(IOS::EUnrealKeyCode::Enter);
			
			// hide the keyboard
			if (!bIsUsingIntegratedKeyboard)
			{
				[self resignFirstResponder];
			}
		}
		else
		{
			KeyEvent.TranslatedKeyCode = Char;
		}

		KeyboardController->QueueOneOffExternalKeyEvent(KeyEvent);
	}
}

- (void)deleteBackward
{
	if (nil != CachedMarkedText) {
		[CachedMarkedText release];
		CachedMarkedText = nil;
	}


	TSharedPtr<FAppleKeyboardController> KeyboardController = ([self GetKeyboardController]).Pin();
	if (!KeyboardController)
	{
		return;
	}
	
	FAppleKeyboardController::FDeferredEvent KeyEvent;
	KeyEvent.CharCode = '\b';
	KeyEvent.TranslatedKeyCode = static_cast<uint32>(IOS::EUnrealKeyCode::Backspace);
	
	KeyboardController->QueueOneOffExternalKeyEvent(KeyEvent);
}

-(void)ActivateKeyboard:(bool)bInSendEscapeOnClose
{
	FKeyboardConfig DefaultConfig;
	[self ActivateKeyboard:bInSendEscapeOnClose keyboardConfig:DefaultConfig];
}

-(void)ActivateKeyboard:(bool)bInSendEscapeOnClose keyboardConfig:(FKeyboardConfig)KeyboardConfig
{
	FPlatformAtomics::InterlockedIncrement(&KeyboardShowCount);
	
	dispatch_async(dispatch_get_main_queue(),^ {
		volatile int32 ShowCount = KeyboardShowCount;
		if (ShowCount == 1)
		{
			bool bKeyboardSettingsChanged = self.keyboardType != KeyboardConfig.KeyboardType ||
						self.autocorrectionType != KeyboardConfig.AutocorrectionType ||
						self.autocapitalizationType != KeyboardConfig.AutocapitalizationType ||
						self.secureTextEntry != KeyboardConfig.bSecureTextEntry;
			
			self.keyboardType = KeyboardConfig.KeyboardType;
			self.autocorrectionType = KeyboardConfig.AutocorrectionType;
			self.autocapitalizationType = KeyboardConfig.AutocapitalizationType;
			self.secureTextEntry = KeyboardConfig.bSecureTextEntry;
		
			// Remember the setting
			bSendEscapeOnClose = bInSendEscapeOnClose;
			
			// Dismiss the existing keyboard, if one exists, so the style can be overridden.
			if (bKeyboardSettingsChanged || !bIsUsingIntegratedKeyboard)
			{
				[self endEditing:YES];
			}
			
			[self becomeFirstResponder];
		}
		
		FPlatformAtomics::InterlockedDecrement(&KeyboardShowCount);
	});
}

-(void)DeactivateKeyboard
{
	dispatch_async(dispatch_get_main_queue(),^ {
		volatile int32 ShowCount = KeyboardShowCount;
		if (ShowCount == 0)
		{
			// Wait briefly, in case a keyboard activation is triggered.
			FPlatformProcess::Sleep(0.1F);
			
			ShowCount = KeyboardShowCount;
			if (ShowCount == 0)
			{
				// Dismiss the existing keyboard, if one exists.
				[self endEditing:YES];

			}
		}
	});
}

-(bool)ShouldUseIntegratedKeyboard
{
	bool bUseIntegratedKeyboard = false;
	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bUseIntegratedKeyboard"), bUseIntegratedKeyboard, GEngineIni);
	
	// get notifications when the keyboard is in view
	return FParse::Param(FCommandLine::Get(), TEXT("NewKeyboard")) || bUseIntegratedKeyboard;
}

-(TWeakPtr<FAppleKeyboardController>)GetKeyboardController
{
	FIOSApplication* IOSApplication = [IOSAppDelegate GetDelegate].IOSApplication;
	FIOSInputInterface* InputInterface = IOSApplication ? IOSApplication->GetIOSInputInterface() : nullptr;
	
	return InputInterface ? InputInterface->GetKeyboardController() : nullptr;
}

-(BOOL)becomeFirstResponder
{
	TSharedPtr<FAppleKeyboardController> KeyboardController = ([self GetKeyboardController]).Pin();
	bool bIsAnyPhysicalKeyboardConnected = KeyboardController && KeyboardController->IsAnyKeyboardConnected();

	volatile int32 ShowCount = KeyboardShowCount;
	if (ShowCount >= 1 && !bIsAnyPhysicalKeyboardConnected)
	{
		return [super becomeFirstResponder];
	}
	else
	{
		return NO;
	}
}

-(BOOL)resignFirstResponder
{
	if (bSendEscapeOnClose)
	{
		TSharedPtr<FAppleKeyboardController> KeyboardController = ([self GetKeyboardController]).Pin();
		if (KeyboardController)
		{
			FAppleKeyboardController::FDeferredEvent KeyEvent;
			KeyEvent.CharCode = 0;
			KeyEvent.TranslatedKeyCode = static_cast<uint32>(IOS::EUnrealKeyCode::Escape);

			// tell the console to close itself
			KeyboardController->QueueOneOffExternalKeyEvent(KeyEvent);
		}
	}
	
	return [super resignFirstResponder];
}


// @todo keyboard: This is a helper define to show functions that _may_ need to be implemented as we go forward with keyboard support
// for now, the very basics work, but most likely at some point for optimal functionality, we'll want to know the actual string
// in the box, but that needs to come from Slate, and we currently don't have a way to get it
#define REPORT_EVENT UE_LOGF(LogIOS, Display, "Got a keyboard call, line %d", __LINE__);


- (NSString *)textInRange:(UITextRange *)range
{
	if (nil != CachedMarkedText) {
		return CachedMarkedText;
	}
	return nil;
	//IndexedRange *r = (IndexedRange *)range;
	//return ([textStore substringWithRange:r.range]);
}

- (void)replaceRange:(UITextRange *)range withText:(NSString *)text
{
	REPORT_EVENT;
	return;
//	IndexedRange *r = (IndexedRange *)range;
//	[textStore replaceCharactersInRange:r.range withString:text];
}

- (UITextRange *)selectedTextRange
{
	// @todo keyboard: This is called
	return [IndexedRange rangeWithNSRange:NSMakeRange(0,0)];//self.textView.selectedTextRange];
}


- (void)setSelectedTextRange:(UITextRange *)range
{
	REPORT_EVENT;
	//IndexedRange *indexedRange = (IndexedRange *)range;
	//self.textView.selectedTextRange = indexedRange.range;
}


- (UITextRange *)markedTextRange
{
	if (nil != CachedMarkedText) {
		return[[[UITextRange alloc] init] autorelease];
	}
	return nil; // Nil if no marked text.
}


- (void)setMarkedText:(NSString *)markedText selectedRange:(NSRange)selectedRange
{
	if (markedText == CachedMarkedText) {
		return;
	}
	if (nil != CachedMarkedText) {
		[CachedMarkedText release];
	}
	CachedMarkedText = markedText;
	[CachedMarkedText retain];
	//NSLog(@"setting marked text to %@", markedText);
}


- (void)unmarkText
{
	if (CachedMarkedText != nil)
	{
		[self insertText:CachedMarkedText];
		[CachedMarkedText release];
		CachedMarkedText = nil;
	}
}


- (UITextPosition *)beginningOfDocument
{
	// @todo keyboard: This is called
	return [IndexedPosition positionWithIndex:0];
}


- (UITextPosition *)endOfDocument
{
	REPORT_EVENT;
	return [IndexedPosition positionWithIndex:0];
}


- (UITextRange *)textRangeFromPosition:(UITextPosition *)fromPosition toPosition:(UITextPosition *)toPosition
{
	// @todo keyboard: This is called
	// Generate IndexedPosition instances that wrap the to and from ranges.
	IndexedPosition *fromIndexedPosition = (IndexedPosition *)fromPosition;
	IndexedPosition *toIndexedPosition = (IndexedPosition *)toPosition;
	NSRange range = NSMakeRange(MIN(fromIndexedPosition.index, toIndexedPosition.index), ABS(toIndexedPosition.index - fromIndexedPosition.index));
 
	return [IndexedRange rangeWithNSRange:range];
}


- (UITextPosition *)positionFromPosition:(UITextPosition *)position offset:(NSInteger)offset
{
	// @todo keyboard: This is called
	return nil;
}


- (UITextPosition *)positionFromPosition:(UITextPosition *)position inDirection:(UITextLayoutDirection)direction offset:(NSInteger)offset
{
	REPORT_EVENT;
	return nil;
}


- (NSComparisonResult)comparePosition:(UITextPosition *)position toPosition:(UITextPosition *)other
{
	// @todo keyboard: This is called
	return NSOrderedSame;
}


- (NSInteger)offsetFromPosition:(UITextPosition *)from toPosition:(UITextPosition *)toPosition
{
	REPORT_EVENT;
	IndexedPosition *fromIndexedPosition = (IndexedPosition *)from;
	IndexedPosition *toIndexedPosition = (IndexedPosition *)toPosition;
	return (toIndexedPosition.index - fromIndexedPosition.index);
}



- (UITextPosition *)positionWithinRange:(UITextRange *)range farthestInDirection:(UITextLayoutDirection)direction
{
	REPORT_EVENT;
	return nil;
}


- (UITextRange *)characterRangeByExtendingPosition:(UITextPosition *)position inDirection:(UITextLayoutDirection)direction
{
	REPORT_EVENT;
	return nil;
}

- (NSWritingDirection)baseWritingDirectionForPosition:(UITextPosition *)position inDirection:(UITextStorageDirection)direction
{
	REPORT_EVENT;
	// assume left to right for now
	return NSWritingDirectionLeftToRight;
}


- (void)setBaseWritingDirection:(NSWritingDirection)writingDirection forRange:(UITextRange *)range
{
	// @todo keyboard: This is called
}



- (CGRect)firstRectForRange:(UITextRange *)range
{
	REPORT_EVENT;
	return CGRectMake(0,0,0,0);
}


- (CGRect)caretRectForPosition:(UITextPosition *)position
{
	// @todo keyboard: This is called
	return CGRectMake(0,0,0,0);
}


- (UITextPosition *)closestPositionToPoint:(CGPoint)point
{
	REPORT_EVENT;
	return nil;
}

- (UITextPosition *)closestPositionToPoint:(CGPoint)point withinRange:(UITextRange *)range
{
	REPORT_EVENT;
	return nil;
}

- (UITextRange *)characterRangeAtPoint:(CGPoint)point
{
	REPORT_EVENT;
	return nil;
}

- (NSArray *)selectionRectsForRange:(UITextRange *)range
{
	REPORT_EVENT;
	return nil;
}


- (NSDictionary *)textStylingAtPosition:(UITextPosition *)position inDirection:(UITextStorageDirection)direction
{
	// @todo keyboard: This is called
	return @{ };
}


- (void) setInputDelegate: (id <UITextInputDelegate>) delegate
{
	// @todo keyboard: This is called
}

- (id <UITextInputTokenizer>) tokenizer
{
	// @todo keyboard: This is called
	return nil;
}

- (id <UITextInputDelegate>) inputDelegate
{
	// @todo keyboard: This is called
	return nil;
}





#if !PLATFORM_TVOS
- (void)keyboardWasShown:(NSNotification*)aNotification
{
	// send a callback to let the game know where to sldie the textbox up above
	NSDictionary* info = [aNotification userInfo];
	CGRect Frame = [[info objectForKey:UIKeyboardFrameEndUserInfoKey] CGRectValue];
	
	FPlatformRect ScreenRect;
	ScreenRect.Top = FMath::TruncToInt(Frame.origin.y);
	ScreenRect.Bottom = FMath::TruncToInt(Frame.origin.y + Frame.size.height);
	ScreenRect.Left = FMath::TruncToInt(Frame.origin.x);
	ScreenRect.Right = FMath::TruncToInt(Frame.origin.x + Frame.size.width);
	
	[FAppleAsyncTask CreateTaskWithBlock:^bool(void){
		[IOSAppDelegate GetDelegate].IOSApplication->OnVirtualKeyboardShown().Broadcast(ScreenRect);
		return true;
	 }];
}

// Called when the UIKeyboardWillHideNotification is sent
- (void)keyboardWillBeHidden:(NSNotification*)aNotification
{
	[FAppleAsyncTask CreateTaskWithBlock:^bool(void){
		[IOSAppDelegate GetDelegate].IOSApplication->OnVirtualKeyboardHidden().Broadcast();
		return true;
	 }];
}
#endif

- (void)InitKeyboard
{
#if !PLATFORM_TVOS
	KeyboardShowCount = 0;

	bool bEnableVirtualKeyboardVisibilityEvent = false;
	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bEnableVirtualKeyboardVisibilityEvent"), bEnableVirtualKeyboardVisibilityEvent, GEngineIni);

	bIsUsingIntegratedKeyboard = [self ShouldUseIntegratedKeyboard];

	if (bEnableVirtualKeyboardVisibilityEvent || bIsUsingIntegratedKeyboard)
	{
		[[NSNotificationCenter defaultCenter] addObserver:self
												 selector:@selector(keyboardWasShown:)
													 name:UIKeyboardDidShowNotification object:nil];
		
		[[NSNotificationCenter defaultCenter] addObserver:self
												 selector:@selector(keyboardWillBeHidden:)
													 name:UIKeyboardWillHideNotification object:nil];
	}
#endif
}

#if !PLATFORM_TVOS
- (void)SetupGestureRecognizers
{
    // Pan — continuous single-finger drag
    UIPanGestureRecognizer* Pan = [[UIPanGestureRecognizer alloc] initWithTarget:self action:@selector(HandlePanGesture:)];
    Pan.minimumNumberOfTouches = 1;
    Pan.maximumNumberOfTouches = 1;
    Pan.cancelsTouchesInView = NO;
    Pan.delegate = self;
    [self addGestureRecognizer:Pan];
    [Pan release];

    // Pinch — two-finger scale
    UIPinchGestureRecognizer* Pinch = [[UIPinchGestureRecognizer alloc] initWithTarget:self action:@selector(HandlePinchGesture:)];
    Pinch.cancelsTouchesInView = NO;
    Pinch.delegate = self;
    [self addGestureRecognizer:Pinch];
    [Pinch release];

    // Rotate — two-finger rotation
    UIRotationGestureRecognizer* Rotation = [[UIRotationGestureRecognizer alloc] initWithTarget:self action:@selector(HandleRotateGesture:)];
    Rotation.cancelsTouchesInView = NO;
    Rotation.delegate = self;
    [self addGestureRecognizer:Rotation];
    [Rotation release];

    // Tap — single tap
    UITapGestureRecognizer* Tap = [[UITapGestureRecognizer alloc] initWithTarget:self action:@selector(HandleTapGesture:)];
    Tap.numberOfTapsRequired = 1;
    Tap.cancelsTouchesInView = NO;
    Tap.delegate = self;
    [self addGestureRecognizer:Tap];
    [Tap release];

    // Long Press — continuous (Began → Changed → Ended)
    UILongPressGestureRecognizer* LongPress = [[UILongPressGestureRecognizer alloc] initWithTarget:self action:@selector(HandleLongPressGesture:)];
    LongPress.cancelsTouchesInView = NO;
    LongPress.delegate = self;
    [self addGestureRecognizer:LongPress];
    [LongPress release];

}

// Allow gesture recognizers to work simultaneously with raw touches
- (BOOL)gestureRecognizer:(UIGestureRecognizer*)gestureRecognizer
    shouldRecognizeSimultaneouslyWithGestureRecognizer:(UIGestureRecognizer*)otherGestureRecognizer
{
    return YES;
}

- (void)HandlePanGesture:(UIPanGestureRecognizer*)Recognizer
{
    CGPoint Translation = [Recognizer translationInView:self];
    // Apply same view size scaling as touch events
    CGPoint ViewSizeModifier = CGPointMake(_ViewSize.width / self.frame.size.width,
                                           _ViewSize.height / self.frame.size.height);

    TArray<GestureInput> Gestures;
    GestureInput G;
    G.Type = EGestureEvent::Pan;
    G.Delta = FVector2D(Translation.x * ViewSizeModifier.x, Translation.y * ViewSizeModifier.y);
    G.bIsStart = (Recognizer.state == UIGestureRecognizerStateBegan);
    G.bIsEnd   = (Recognizer.state == UIGestureRecognizerStateEnded ||
                  Recognizer.state == UIGestureRecognizerStateCancelled);
    Gestures.Add(G);

    // Detect flick: if pan ended with sufficient velocity, emit a one-shot Flick gesture.
    // Angle convention matches FGestureRecognizer: right=0, up=90, left=180, down=270.
    if (G.bIsEnd)
    {
        CGPoint Velocity = [Recognizer velocityInView:self];
        CGFloat SpeedSq = Velocity.x * Velocity.x + Velocity.y * Velocity.y;
        const CGFloat FlickThresholdSq = 500.0 * 500.0; // 500 pts/s
        if (SpeedSq > FlickThresholdSq)
        {
            float Angle = (float)FRotator::ClampAxis(FMath::RadiansToDegrees(FMath::Atan2(-Velocity.y, Velocity.x)));
            float VelocityMagnitude = (float)sqrt(SpeedSq);

            GestureInput Flick;
            Flick.Type = EGestureEvent::Flick;
            Flick.Delta = FVector2D(Angle, VelocityMagnitude);
            Flick.bIsStart = true;
            Flick.bIsEnd = false;
            Gestures.Add(Flick);
        }
    }

    FIOSInputInterface::QueueGestureInput(Gestures);

    // Reset translation so delta is per-callback, not cumulative
    [Recognizer setTranslation:CGPointZero inView:self];
}

- (void)HandlePinchGesture:(UIPinchGestureRecognizer*)Recognizer
{
    TArray<GestureInput> Gestures;
    GestureInput G;
    G.Type = EGestureEvent::Magnify;
    G.Delta = FVector2D((float)Recognizer.scale, 0.0f);
    G.bIsStart = (Recognizer.state == UIGestureRecognizerStateBegan);
    G.bIsEnd   = (Recognizer.state == UIGestureRecognizerStateEnded ||
                  Recognizer.state == UIGestureRecognizerStateCancelled);
    Gestures.Add(G);
    FIOSInputInterface::QueueGestureInput(Gestures);

    // Reset scale for delta-based reporting
    Recognizer.scale = 1.0;
}

- (void)HandleRotateGesture:(UIRotationGestureRecognizer*)Recognizer
{
    TArray<GestureInput> Gestures;
    GestureInput G;
    G.Type = EGestureEvent::Rotate;
    G.Delta = FVector2D(FMath::RadiansToDegrees((float)Recognizer.rotation), 0.0f);
    G.bIsStart = (Recognizer.state == UIGestureRecognizerStateBegan);
    G.bIsEnd   = (Recognizer.state == UIGestureRecognizerStateEnded ||
                  Recognizer.state == UIGestureRecognizerStateCancelled);
    Gestures.Add(G);
    FIOSInputInterface::QueueGestureInput(Gestures);
}

- (void)HandleTapGesture:(UITapGestureRecognizer*)Recognizer
{
	// UITapGestureRecognizer is discrete — it only fires once at StateRecognized.
	// We queue only the press here; ProcessGestures() defers the release to the
	// next frame so EvaluateKeyMapState has one tick to read the non-zero value.
	if (Recognizer.state != UIGestureRecognizerStateRecognized)
	{
		return;
	}

	CGPoint Location = [Recognizer locationInView:self];
	CGPoint ViewSizeModifier = CGPointMake(_ViewSize.width / self.frame.size.width,
										   _ViewSize.height / self.frame.size.height);
	FVector2D ScreenLocation(Location.x * ViewSizeModifier.x, Location.y * ViewSizeModifier.y);

	GestureInput Tap;
	Tap.Type = EGestureEvent::Tap;
	Tap.Delta = ScreenLocation;
	Tap.bIsStart = true;
	Tap.bIsEnd = false;

	TArray<GestureInput> Gestures;
	Gestures.Add(Tap);
	FIOSInputInterface::QueueGestureInput(Gestures);
}

- (void)HandleLongPressGesture:(UILongPressGestureRecognizer*)Recognizer
{
	CGPoint Location = [Recognizer locationInView:self];
	CGPoint ViewSizeModifier = CGPointMake(_ViewSize.width / self.frame.size.width,
										   _ViewSize.height / self.frame.size.height);
	FVector2D ScreenLocation(Location.x * ViewSizeModifier.x, Location.y * ViewSizeModifier.y);

	bool bIsEnd = (Recognizer.state == UIGestureRecognizerStateEnded ||
				  Recognizer.state == UIGestureRecognizerStateCancelled);

	GestureInput G;
	G.Type = EGestureEvent::LongPress;
	G.Delta = bIsEnd ? FVector2D::ZeroVector : ScreenLocation;
	G.bIsStart = (Recognizer.state == UIGestureRecognizerStateBegan);
	G.bIsEnd = bIsEnd;

	TArray<GestureInput> Gestures;
	Gestures.Add(G);
	FIOSInputInterface::QueueGestureInput(Gestures);
}
#endif // !PLATFORM_TVOS

@end


#pragma mark IOSViewController

@implementation IOSViewController

/**
 * The ViewController was created, so now we need to create our view to be controlled
 */
- (void) loadView
{
#if PLATFORM_VISIONOS
	CGRect Frame = CGRectMake(0, 0, 1000, 1000);
#else
	// get the landcape size of the screen
	CGRect Frame = [[UIScreen mainScreen] bounds];
	if (![IOSAppDelegate GetDelegate].bDeviceInPortraitMode)
	{
		Swap(Frame.size.width, Frame.size.height);
	}
#endif
 
	[IOSAppDelegate GetDelegate].IOSController = self;

	self.view = [[UIView alloc] initWithFrame:Frame];

	// settings copied from InterfaceBuilder
	self.edgesForExtendedLayout = UIRectEdgeNone;

	self.view.clearsContextBeforeDrawing = NO;
#if !PLATFORM_TVOS
	self.view.multipleTouchEnabled = NO;

	[self setNeedsUpdateOfScreenEdgesDeferringSystemGestures];
#endif
}

/**
 * View was unloaded from us
 */ 
- (void) viewDidUnload
{
	UE_LOGF(LogIOS, Log, "IOSViewController unloaded the view. This is unexpected, tell Josh Adams");
	[super viewDidUnload];
}

#if !PLATFORM_TVOS

/**
 * Tell the OS about the default supported orientations
 */
 - (UIInterfaceOrientationMask)supportedInterfaceOrientations_Internal
{
	const IOSAppDelegate *AppDelegate = [IOSAppDelegate GetDelegate];
	const FIOSView *View = [AppDelegate IOSView];
	if (View != nil)
	{
		// if a Blueprint has changed the default rotation constraints, honour that change
		if (View->SupportedInterfaceOrientations != UIInterfaceOrientationMaskAll)
		{
			return View->SupportedInterfaceOrientations;
		}
	}

	// View either not yet created or Blueprint is not overriding the default, so use what the Window has set
	UIApplication *app = [UIApplication sharedApplication];
	return [AppDelegate application:app supportedInterfaceOrientationsForWindow:[AppDelegate window]];
}

 - (UIInterfaceOrientationMask)supportedInterfaceOrientations
 {
	IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];
	const FIOSView* View = [AppDelegate IOSView];
	if (GIOSDelayRotationUntilPresent && AppDelegate.bPlatformInit && View != nil && (int)View->LastFrameInterfaceOrientationMask != 0)
	{
#if IOS_ROTATION_DEBUG_LOGGING
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[rotation] supportedInterfaceOrientations: allowing %d\n"), (int)View->LastFrameInterfaceOrientationMask);
#endif

		// Only allow rotation to the orientation of the last frame we have rendered, until a new one is available in the new orientation.
		return View->LastFrameInterfaceOrientationMask;
	}
	else
	{
#if IOS_ROTATION_DEBUG_LOGGING
		if (GIOSDelayRotationUntilPresent)
		{
			if (!AppDelegate.bPlatformInit)
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[rotation] supportedInterfaceOrientations: bPlatformInit=false\n"));
			}
			else
			if (View == nil)
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[rotation] supportedInterfaceOrientations: view is nil\n"));
			}
			else
			{
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[rotation] supportedInterfaceOrientations: last %d\n"), (int)View->LastFrameInterfaceOrientationMask);
			}
		}
#endif

		return [self supportedInterfaceOrientations_Internal];
	}
 }


#endif

/**
 * Tell the OS that our view controller can auto-rotate between supported orientations
 */
- (BOOL)shouldAutorotate
{
	return YES;
}

/**
 * Tell the OS to hide the status bar (iOS 7 method for hiding)
 */
- (BOOL)prefersStatusBarHidden
{
	return YES;
}

- (BOOL)prefersPointerLocked
{
    UE_LOGF(LogIOS, Log, "IOSViewController prefersPointerLocked");
    return YES;
}

/**
 * Tell the OS to hide the home bar
 */
- (BOOL)prefersHomeIndicatorAutoHidden
{
	return YES;
}

/*
* Set the preferred landscape orientation 
*/
- (UIInterfaceOrientation)preferredInterfaceOrientationForPresentation
{
	FString PreferredLandscapeOrientation;
	bool bSupportsLandscapeLeft = false;
	bool bSupportsLandscapeRight = false;

	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsLandscapeLeftOrientation"), bSupportsLandscapeLeft, GEngineIni);
	GConfig->GetBool(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("bSupportsLandscapeRightOrientation"), bSupportsLandscapeRight, GEngineIni);
	GConfig->GetString(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("PreferredLandscapeOrientation"), PreferredLandscapeOrientation, GEngineIni);

	if(bSupportsLandscapeLeft && bSupportsLandscapeRight)
	{
		if (PreferredLandscapeOrientation.Equals("LandscapeRight"))
		{
			return UIInterfaceOrientationLandscapeRight;
		}
		return UIInterfaceOrientationLandscapeLeft;
	}

	return UIInterfaceOrientationPortrait;
}

- (UIRectEdge)preferredScreenEdgesDeferringSystemGestures
{
	// Default to all edges
	// From IOSRuntimeSettings.h: (EIOSScreenEdge::Top | EIOSScreenEdge::Left | EIOSScreenEdge::Bottom | EIOSScreenEdge::Right)
	int32 EdgeMask = 0xF;
	if (!GConfig->GetInt(TEXT("/Script/IOSRuntimeSettings.IOSRuntimeSettings"), TEXT("ScreenEdgesDeferredGestures"), EdgeMask, GEngineIni))
	{
		UE_LOGF(LogIOS, Log, "MISSING SCREENEDGE VALUE");
	}

	UIRectEdge Result = UIRectEdgeNone;
	if (EdgeMask & (1 << 0)) Result |= UIRectEdgeTop;
	if (EdgeMask & (1 << 1)) Result |= UIRectEdgeLeft;
	if (EdgeMask & (1 << 2)) Result |= UIRectEdgeBottom;
	if (EdgeMask & (1 << 3)) Result |= UIRectEdgeRight;
	return Result;
}

- (void)viewWillTransitionToSize : (CGSize)size withTransitionCoordinator : (id<UIViewControllerTransitionCoordinator>)coordinator
{
	const IOSAppDelegate *AppDelegate = [IOSAppDelegate GetDelegate];
	const FIOSView *View = [AppDelegate IOSView];
	
#if IOS_ROTATION_DEBUG_LOGGING
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[rotation] viewWillTransitionToSize\n"));
#endif

	[super viewWillTransitionToSize : size withTransitionCoordinator : coordinator];

#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
	if (GIOSDelayRotationUntilPresent)
	{
		[coordinator animateAlongsideTransition:nil completion:^(id<UIViewControllerTransitionCoordinatorContext>  _Nonnull context)
		{
			// rotation has completed, we need to update the actual safe zone
			FIOSApplication::UpdateSafeZoneAfterRotation();
		}];
	}
#endif
}

#if !PLATFORM_TVOS && !PLATFORM_VISIONOS

- (void)notifyPresentAfterRotateOrientationMask : (UIInterfaceOrientationMask)NewOrientationMask withSizeX : (unsigned int)SizeX withSizeY : (unsigned int)SizeY
{
	const FIOSView* View = [[IOSAppDelegate GetDelegate] IOSView];

#if IOS_ROTATION_DEBUG_LOGGING
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("[rotation] notifyPresentAfterRotate OrientationMask:%d SizeX:%d SizeY:%d\n"), NewOrientationMask, SizeX, SizeY);
#endif

	View->LastFrameInterfaceOrientationMask = NewOrientationMask;

	// Attempt to rotate to desired orientation
	[IOSAppDelegate UpdateSupportedInterfaceOrientations];
 } 
 #endif

- (BOOL)canBecomeFirstResponder
{
	return true;
}

-(TWeakPtr<FAppleKeyboardController>)GetKeyboardController
{
	FIOSApplication* IOSApplication = [IOSAppDelegate GetDelegate].IOSApplication;
	FIOSInputInterface* InputInterface = IOSApplication ? IOSApplication->GetIOSInputInterface() : nullptr;
	return InputInterface ? InputInterface->GetKeyboardController() : nullptr;
}

- (bool)IsAnyPhysicalKeyboardConnected
{
	TSharedPtr<FAppleKeyboardController> KeyboardController = ([self GetKeyboardController]).Pin();
	return KeyboardController ? KeyboardController->IsAnyKeyboardConnected() : false;
}

- (bool)ProcessKeyPresses:(NSSet<UIPress *> *)keypresses eventType:(FAppleKeyboardController::EKeyEvent)eventType
{
	TSharedPtr<FAppleKeyboardController> KeyboardController = ([self GetKeyboardController]).Pin();
	if (KeyboardController && KeyboardController->IsAnyKeyboardConnected())
	{
		for (UIPress* Press in keypresses.allObjects)
		{
			if (UIKey* PressedKey = Press ? Press.key : nil)
			{

				// This can contain escape secuences, or descriptive strings like Left Arrow. We only care about single characters, 
				// we handle specific escape secuences we support manually
				unichar Char = [PressedKey.characters length] == 1 ? [PressedKey.characters characterAtIndex:0] : 0;

				// Anything in the unicode private area range is non printable
				bool bNonPrintable = Char >= 0xF700 && Char <= 0xF8FF;

				FAppleKeyboardController::FDeferredEvent KeyEvent;
				KeyEvent.KeyEventType = eventType;
				
				// Unreal has two key input handling paths, one for keycodes and other for characters. for characters , the keycode needs to be the character
				// for non character key events, we need to forward the translated unreal keycode, and no character... except for escape secuences for backspace and new line
				// otherwise some UMG widgets will execute these actions.
				
				using namespace IOS;

				EUnrealKeyCode TranslateKeycode = IOS::TranslateUSBHIDToUnrealKeycode(PressedKey.keyCode);
				if (TranslateKeycode == EUnrealKeyCode::Unknown)
				{
					if (bNonPrintable)
					{
						// The event if from a key that is not printable or a mapped hid code, we can't handle this event
						// so allow the responder chain to handle it
						return false;
					}
					
					KeyEvent.TranslatedKeyCode = Char;
					KeyEvent.CharCode = Char;
				}
				else
				{
					KeyEvent.TranslatedKeyCode = static_cast<uint32>(TranslateKeycode);
					
					switch (PressedKey.keyCode) {
						case UIKeyboardHIDUsageKeyboardDeleteOrBackspace:
							KeyEvent.CharCode = '\b';
							break;
						case UIKeyboardHIDUsageKeyboardReturnOrEnter:
							KeyEvent.CharCode = '\n';
							break;
						default:
							KeyEvent.CharCode = bNonPrintable ? 0 : Char;
							break;
					}
				}
				
				KeyboardController->QueueExternalKeyChangeEvent(KeyEvent);
			}
		}
		
		return true;
	}
	
	return false;
}

- (void)pressesBegan:(NSSet<UIPress *> *)presses withEvent:(nullable UIPressesEvent *)event
{
	if ([self ProcessKeyPresses:presses eventType:FAppleKeyboardController::EKeyEvent::KeyDown])
	{
		return;
	}
	
	[super pressesBegan:presses withEvent:event];
}

- (void)pressesChanged:(NSSet<UIPress *> *)presses withEvent:(nullable UIPressesEvent *)event
{
	if ([self IsAnyPhysicalKeyboardConnected])
	{
		return;
	}
	[super pressesChanged:presses withEvent:event];
}

- (void)pressesEnded:(NSSet<UIPress *> *)presses withEvent:(nullable UIPressesEvent *)event
{
	if ([self ProcessKeyPresses:presses eventType:FAppleKeyboardController::EKeyEvent::KeyUp])
	{
		return;
	}

	[super pressesEnded:presses withEvent:event];
}

- (void)pressesCancelled:(NSSet<UIPress *> *)presses withEvent:(nullable UIPressesEvent *)event
{
	if ([self ProcessKeyPresses:presses eventType:FAppleKeyboardController::EKeyEvent::KeyUp])
	{
		return;
	}

	[super pressesCancelled:presses withEvent:event];
}

@end
