// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSWindow.h"
#include "IOS/IOSAppDelegate.h"
#include "IOS/IOSView.h"
#include "IOS/IOSApplication.h"

FIOSWindow::~FIOSWindow()
{
	// NOTE: The Window is invalid here!
	//       Use NativeWindow_Destroy() instead.
}

TSharedRef<FIOSWindow> FIOSWindow::Make()
{
	return MakeShareable( new FIOSWindow() );
}

FIOSWindow::FIOSWindow()
{
}

void FIOSWindow::Initialize( class FIOSApplication* const Application, const TSharedRef< FGenericWindowDefinition >& InDefinition, const TSharedPtr< FIOSWindow >& InParent, const bool bShowImmediately )
{
	OwningApplication = Application;
	Definition = InDefinition;
	
	NSArray<UIScene*> *windowScenes = [[[UIApplication sharedApplication] connectedScenes] allObjects];
	
	// TODO: iOS only has one window scene (as of iOS15), however, iPadOS15(?) might have more than one with an external monitor (maybe?).
	//       Need to investigate and check if looping through windowScene will be required.
	UIWindowScene *windowScene = (UIWindowScene*)[windowScenes firstObject];
	for (UIWindow *currentWindow in windowScene.windows)
	{
		if (currentWindow.isKeyWindow)
		{
			Window = currentWindow;
			break;
		}
	}

#if !PLATFORM_TVOS
	if(InParent.Get() != NULL)
	{
		dispatch_async(dispatch_get_main_queue(),^ {
			if ([UIAlertController class])
			{
				UIAlertController* AlertController = [UIAlertController alertControllerWithTitle:@""
														message:@"Error: Only one UIWindow may be created on iOS."
														preferredStyle:UIAlertControllerStyleAlert];
				UIAlertAction* okAction = [UIAlertAction
											actionWithTitle:NSLocalizedString(@"OK", nil)
											style:UIAlertActionStyleDefault
											handler:^(UIAlertAction* action)
											{
												[AlertController dismissViewControllerAnimated : YES completion : nil];
											}
				];

				[AlertController addAction : okAction];
				[[IOSAppDelegate GetDelegate].IOSController presentViewController : AlertController animated : YES completion : nil];
			}
		} );
	}
#endif
}

void FIOSWindow::OnScaleFactorChanged(IConsoleVariable* CVar)
{
	IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];
	FIOSView* View = AppDelegate.IOSView;
		
	// If r.MobileContentScaleFactor is set by a console command, clear out the r.mobile.DesiredResX/Y CVars
	if ((CVar->GetFlags() & ECVF_SetByMask) == ECVF_SetByConsole)
	{
		IConsoleVariable* CVarResX = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.DesiredResX"));
		IConsoleVariable* CVarResY = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.DesiredResY"));
		
		// If CVarResX/Y needs to be reset, let that CVar callback handle the layout change
		bool OtherCVarChanged = false;
		if (CVarResX && CVarResX->GetInt() != 0)
		{
			CVarResX->Set(0, ECVF_SetByConsole);
			OtherCVarChanged = true;
		}
		if (CVarResY && CVarResY->GetInt() != 0)
		{
			CVarResY->Set(0, ECVF_SetByConsole);
			OtherCVarChanged = true;
		}
		
		if (OtherCVarChanged)
		{
			return;
		}
	}
		
	// Load the latest Cvars that might affect screen size
	[AppDelegate LoadScreenResolutionModifiers];
		
	// Force a re-layout of our views as the size has probably changed
	CGRect Frame = [View frame];
	[View CalculateContentScaleFactor:FMath::TruncToInt(Frame.size.width) ScreenHeight:FMath::TruncToInt(Frame.size.height)];
	[View forceLayoutSubviews];
}

void FIOSWindow::OnConsoleResolutionChanged(IConsoleVariable* CVar)
{
	IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];
	FIOSView* View = AppDelegate.IOSView;
		
	// Load the latest Cvars that might affect screen size
	[AppDelegate LoadScreenResolutionModifiers];
		
	// Force a re-layout of our views as the size has probably changed
	CGRect Frame = [View frame];
	[View CalculateContentScaleFactor:FMath::TruncToInt(Frame.size.width) ScreenHeight:FMath::TruncToInt(Frame.size.height)];
	[View forceLayoutSubviews];
}

FPlatformRect FIOSWindow::GetScreenRect()
{
	// get the main view's frame
	IOSAppDelegate* AppDelegate = [IOSAppDelegate GetDelegate];
	FIOSView* View = AppDelegate.IOSView;
	
	FPlatformRect ScreenRect;
	if (View != nil)
	{
		CGRect Frame = [View frame];
		CGFloat Scale = View.contentScaleFactor;
		
		ScreenRect.Top = FMath::TruncToInt(Frame.origin.y * Scale);
		ScreenRect.Left = FMath::TruncToInt(Frame.origin.x * Scale);
		ScreenRect.Bottom = FMath::TruncToInt((Frame.origin.y + Frame.size.height) * Scale);
		ScreenRect.Right = FMath::TruncToInt((Frame.origin.x + Frame.size.width) * Scale);

#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
		if (GIOSDelayRotationUntilPresent)
		{	
			int32 WindowWidth = ScreenRect.Right - ScreenRect.Left;
			int32 WindowHeight = ScreenRect.Bottom - ScreenRect.Top;

			if ((UIInterfaceOrientationIsPortrait(FIOSApplication::CachedOrientation) && WindowWidth > WindowHeight) ||
				(UIInterfaceOrientationIsLandscape(FIOSApplication::CachedOrientation) && WindowHeight > WindowWidth))
			{
				Exchange(ScreenRect.Top, ScreenRect.Left);
				Exchange(ScreenRect.Bottom, ScreenRect.Right);
			}			
		}
#endif

		// see QuantizeSceneBufferSize
		const uint32 DividableBy = 4;
		const uint32 Mask = ~(DividableBy - 1);
		ScreenRect.Top = (ScreenRect.Top + DividableBy - 1) & Mask;
		ScreenRect.Left = (ScreenRect.Left + DividableBy - 1) & Mask;
		ScreenRect.Bottom = (ScreenRect.Bottom + DividableBy - 1) & Mask;
		ScreenRect.Right = (ScreenRect.Right + DividableBy - 1) & Mask;
	}
#if PLATFORM_VISIONOS
	else if (AppDelegate.SwiftLayer != nullptr)
	{
		// using the last viewport since this will work with 1 eye or 2 eye setup, to get the full size of the screen
		CGRect LastViewport = [[AppDelegate.SwiftLayerViewports lastObject] CGRectValue];
		ScreenRect.Left = 0;
		ScreenRect.Top = 0;
		ScreenRect.Right = (int)(LastViewport.origin.x + LastViewport.size.width);
		ScreenRect.Bottom = (int)(LastViewport.origin.y + LastViewport.size.height);
	}
#endif
	return ScreenRect;
}

bool FIOSWindow::GetFullScreenInfo( int32& X, int32& Y, int32& Width, int32& Height ) const
{
	FPlatformRect ScreenRect = GetScreenRect();

	X = ScreenRect.Left;
	Y = ScreenRect.Top;
	Width = ScreenRect.Right - ScreenRect.Left;
	Height = ScreenRect.Bottom - ScreenRect.Top;

	return true;
}
