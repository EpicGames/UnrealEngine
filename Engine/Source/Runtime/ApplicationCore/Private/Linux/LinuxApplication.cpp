// Copyright Epic Games, Inc. All Rights Reserved.

#include "Linux/LinuxApplication.h"
#include "Null/NullApplication.h"
#include "HAL/PlatformTime.h"
#include "Misc/StringUtility.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"
#include "Features/IModularFeatures.h"
#include "Linux/LinuxPlatformApplicationMisc.h"
#include "Null/NullPlatformApplicationMisc.h"
#include "IInputDeviceModule.h"
#include "IHapticDevice.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "GenericPlatform/InputDeviceRegistry.h"

//
// GameController thresholds
//
#define GAMECONTROLLER_LEFT_THUMB_DEADZONE  7849
#define GAMECONTROLLER_RIGHT_THUMB_DEADZONE 8689
#define GAMECONTROLLER_TRIGGER_THRESHOLD    30

namespace
{
	// How long we wait from a FocusOut event to deactivate the application (100ms default)
	double DeactivationThreadshold = 0.1;

	// Mask away the higher bits (but preserve negative flags), as SDL uses those as flags
	// See: SDL_WINDOWPOS_UNDEFINED_MASK & SDL_WINDOWPOS_CENTERED_MASK for context
	int MaskAwayHigherBits(int Input)
	{
		if (Input < 0)
		{
			return -(-Input & 0xFFFF);
		}

		return Input & 0xFFFF;
	}

	// Returns the screen-absolute origin of an SDL window. SDL_GetWindowPosition reports
	// parent-relative coordinates for popup-class windows, so walk up the popup chain until
	// the first non-popup ancestor (whose own position is screen-absolute) and sum offsets.
	void GetWindowScreenPosition(SDL_HWindow Window, int32& OutX, int32& OutY)
	{
		OutX = 0;
		OutY = 0;
		SDL_HWindow Current = Window;
		while (Current)
		{
			int32 CurX = 0;
			int32 CurY = 0;
			SDL_GetWindowPosition(Current, &CurX, &CurY);
			OutX += MaskAwayHigherBits(CurX);
			OutY += MaskAwayHigherBits(CurY);

			const SDL_WindowFlags Flags = SDL_GetWindowFlags(Current);
			if (!(Flags & (SDL_WINDOW_POPUP_MENU | SDL_WINDOW_TOOLTIP)))
			{
				break;
			}
			Current = SDL_GetWindowParent(Current);
		}
	}
}

float ShortToNormalFloat(short AxisVal)
{
	// normalize [-32768..32767] -> [-1..1]
	const float Norm = (AxisVal <= 0 ? 32768.f : 32767.f);
	return float(AxisVal) / Norm;
}

namespace UE::Input
{
	/** A helper method for converting an Input Device id to an int32 */
	static int32 ConvertInputDeviceToInt(FInputDeviceId DeviceId)
	{
		int32 UserStateControllerId = INDEX_NONE;
		IPlatformInputDeviceMapper& Mapper = IPlatformInputDeviceMapper::Get();

		FPlatformUserId UserId = Mapper.GetUserForInputDevice(DeviceId);
		Mapper.RemapUserAndDeviceToControllerId(UserId, UserStateControllerId, DeviceId);

		return UserStateControllerId;
	}
}

FLinuxApplication* LinuxApplication = NULL;

FLinuxApplication* FLinuxApplication::CreateLinuxApplication()
{
	if (!FApp::CanEverRender())	// this assumes that we're running in "headless" mode, and we don't need any kind of multimedia
	{
		return new FLinuxApplication();
	}

	if (!FLinuxPlatformApplicationMisc::InitSDL()) //	will not initialize more than once
	{
		UE_LOGF(LogInit, Error, "FLinuxApplication::CreateLinuxApplication() : InitSDL() failed, cannot create application instance.");
		FLinuxPlatformMisc::RequestExitWithStatus(true, 1);
		// unreachable
		return nullptr;
	}

#if DO_CHECK
	uint32 InitializedSubsystems = SDL_WasInit(0);
	check(InitializedSubsystems & SDL_INIT_EVENTS);
	check(InitializedSubsystems & SDL_INIT_JOYSTICK);
	check(InitializedSubsystems & SDL_INIT_GAMEPAD);
#endif // DO_CHECK

	LinuxApplication = new FLinuxApplication();

	int NumJoysticks;
	SDL_JoystickID* Joysticks = SDL_GetJoysticks(&NumJoysticks);
	if(Joysticks)
	{
		for (int i = 0; i < NumJoysticks; ++i)
		{
			SDL_JoystickID InstanceID = Joysticks[i];
			if (SDL_IsGamepad(InstanceID))
			{
 				LinuxApplication->AddGameController(InstanceID);
			}
		}
		SDL_free(Joysticks);
	}

	return LinuxApplication;
}


FLinuxApplication::FLinuxApplication() 
	:	GenericApplication( MakeShareable( new FLinuxCursor() ) )
	,	SDLDoubleClickTime(0)
	,	SDLDoubleClickRadius(0)	
	,	bIsMouseCursorLocked(false)
	,	bIsMouseCaptureEnabled(false)
	,	LastTitleBarClickX(0.0f)
	,	LastTitleBarClickY(0.0f)
	,	LastTitleBarClickTimestampNS(0ULL)
	,	LastTitleBarClickWindow(nullptr)
	,	LastHitTestTimestampNS(0ULL)
	,	bHasLoadedInputPlugins(false)
	,	bInsideOwnWindow(false)
	,	bIsDragWindowButtonPressed(false)
	,	bActivateApp(false)
	,	FocusOutDeactivationTime(0.0)
	,	LastTimeCachedDisplays(-1.0)
{
	bUsingHighPrecisionMouseInput = false;
	bAllowedToDeferMessageProcessing = true;
	MouseCaptureWindow = NULL;

	fMouseWheelScrollAccel = 1.0f;
	if (GConfig)
	{
		GConfig->GetFloat(TEXT("X11.Tweaks"), TEXT( "MouseWheelScrollAcceleration" ), fMouseWheelScrollAccel, GEngineIni);
	}
}

FLinuxApplication::~FLinuxApplication()
{
	if ( GConfig )
	{
		GConfig->GetFloat(TEXT("X11.Tweaks"), TEXT("MouseWheelScrollAcceleration"), fMouseWheelScrollAccel, GEngineIni);
		GConfig->Flush(false, GEngineIni);
	}

	IModularFeatures::Get().OnModularFeatureRegistered().RemoveAll(this);
}

void FLinuxApplication::DestroyApplication()
{
	for(auto ControllerIt = ControllerStates.CreateConstIterator(); ControllerIt; ++ControllerIt)
	{
		if(ControllerIt.Value().Controller != nullptr)
		{
			SDL_CloseGamepad(ControllerIt.Value().Controller);
		}
	}
	ControllerStates.Empty();
}

TSharedRef< FGenericWindow > FLinuxApplication::MakeWindow()
{
	return FLinuxWindow::Make();
}

void FLinuxApplication::InitializeWindow(	const TSharedRef< FGenericWindow >& InWindow,
											const TSharedRef< FGenericWindowDefinition >& InDefinition,
											const TSharedPtr< FGenericWindow >& InParent,
											const bool bShowImmediately )
{
	const TSharedRef< FLinuxWindow > Window = StaticCastSharedRef< FLinuxWindow >( InWindow );
	const TSharedPtr< FLinuxWindow > ParentWindow = StaticCastSharedPtr< FLinuxWindow >( InParent );

	Window->Initialize( this, InDefinition, ParentWindow, bShowImmediately );
	Windows.Add(Window);

	// Add the windows into the focus stack.
	if (Window->IsFocusWhenFirstShown())
	{
		RevertFocusStack.Add(Window);
	}

	// Add the window into the notification list if it is a notification window.
	if(Window->IsNotificationWindow())
	{
		NotificationWindows.Add(Window);
	}
}

void FLinuxApplication::SetMessageHandler( const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler )
{
	GenericApplication::SetMessageHandler(InMessageHandler);
}

namespace
{
	TSharedPtr< FLinuxWindow > FindWindowBySDLWindow(const TArray< TSharedRef< FLinuxWindow > >& WindowsToSearch, SDL_HWindow const WindowHandle)
	{
		for (int32 WindowIndex=0; WindowIndex < WindowsToSearch.Num(); ++WindowIndex)
		{
			TSharedRef< FLinuxWindow > Window = WindowsToSearch[WindowIndex];
			if (Window->GetHWnd() == WindowHandle)
			{
				return Window;
			}
		}

		return TSharedPtr< FLinuxWindow >(nullptr);
	}
}

TSharedPtr< FLinuxWindow > FLinuxApplication::FindWindowBySDLWindow(SDL_Window *win)
{
	return ::FindWindowBySDLWindow(Windows, win);
}

void FLinuxApplication::PumpMessages( const float TimeDelta )
{
	FPlatformApplicationMisc::PumpMessages( true );

	// Check for netlink network change events and update status if needed.
	FPlatformMisc::CheckNetworkConnectionEvents();
}

bool FLinuxApplication::IsCursorDirectlyOverSlateWindow() const
{
	return bInsideOwnWindow;
}

bool FLinuxApplication::RequiresPerWindowTooltips() const
{
	// Both X11 and Wayland SDL3 backends require a tooltip parent. Per-owner
	// pooling ties each tooltip's lifecycle to its actual owner so hide,
	// show, and destroy affect only that owner's tooltip.
	return true;
}

TSharedPtr<FGenericWindow> FLinuxApplication::GetWindowUnderCursor()
{
	// If we are drag and dropping the current window under the cursor will always be the DnD window
	// fallback to the SlateApplication finding the top most in DnD situations
	if (CurrentUnderCursorWindow.IsValid() && CurrentUnderCursorWindow->IsDragAndDropWindow())
	{
		return nullptr;
	}

	return CurrentUnderCursorWindow;
}

void FLinuxApplication::AddPendingEvent( SDL_Event SDLEvent )
{
	if( GPumpingMessagesOutsideOfMainLoop && bAllowedToDeferMessageProcessing )
	{
		PendingEvents.Add( SDLEvent );
	}
	else
	{
		// When not deferring messages, process them immediately
		ProcessDeferredMessage( SDLEvent );
	}
}

bool FLinuxApplication::GeneratesKeyCharMessage(const SDL_KeyboardEvent & KeyDownEvent)
{
	bool bCmdKeyPressed = (KeyDownEvent.mod & (SDL_KMOD_LCTRL | SDL_KMOD_RCTRL)) != 0;
	const SDL_Keycode Sym = KeyDownEvent.key;

	// filter out command keys, non-ASCI and arrow keycodes that don't generate WM_CHAR under Windows (TODO: find a table?)
	return !bCmdKeyPressed && Sym < 128 &&
		(Sym != SDLK_DOWN && Sym != SDLK_LEFT && Sym != SDLK_RIGHT && Sym != SDLK_UP && Sym != SDLK_DELETE);
}

// Windows handles translating numpad numbers to arrow keys, but we have to do it manually
static SDL_Keycode TranslateNumLockKeySyms(const SDL_Keycode KeySym)
{
	if ((SDL_GetModState() & SDL_KMOD_NUM) == 0)
	{
		switch (KeySym)
		{
		case SDLK_KP_2:
			return SDLK_DOWN;
		case SDLK_KP_4:
			return SDLK_LEFT;
		case SDLK_KP_6:
			return SDLK_RIGHT;
		case SDLK_KP_8:
			return SDLK_UP;
		default:
			break;
		}
	}
	return KeySym;
}

static inline uint32 CharCodeFromSDLKeySym(const SDL_Keycode KeySym)
{
	// Mirrors windows returning nonzero char codes for numpad keys
	if (KeySym == SDLK_KP_2 || KeySym == SDLK_KP_4 || KeySym == SDLK_KP_6 || KeySym == SDLK_KP_8)
	{
		return (uint32) KeySym;
	}
	else if ((KeySym & SDLK_SCANCODE_MASK) != 0)
	{
		return 0;
	}
	else if (KeySym == SDLK_DELETE)  // this doesn't use the scancode mask for some reason.
	{
		return 0;
	}
	return (uint32) KeySym;
}

void FLinuxApplication::ProcessDeferredMessage( SDL_Event Event )
{
	// This function can be reentered when entering a modal tick loop.
	// We need to make a copy of the events that need to be processed or we may end up processing the same messages twice
	SDL_HWindow NativeWindow = NULL;

	// get pointer to window that received this event
	bool bWindowlessEvent = false;
	TSharedPtr< FLinuxWindow > CurrentEventWindow = FindEventWindow(&Event, bWindowlessEvent);

	if (CurrentEventWindow.IsValid())
	{
		NativeWindow = CurrentEventWindow->GetHWnd();
	}
	if (!NativeWindow && !bWindowlessEvent)
	{
		return;
	}

	switch(Event.type)
	{
	case SDL_EVENT_KEY_DOWN:
		{
			const SDL_KeyboardEvent &KeyEvent = Event.key;
			SDL_Keycode KeySym = KeyEvent.key;
			const uint32 CharCode = CharCodeFromSDLKeySym(KeySym);
			KeySym = TranslateNumLockKeySyms(KeySym);
			const bool bIsRepeated = KeyEvent.repeat != 0;

			// Text input is now handled in SDL_TEXTINPUT: see below
			MessageHandler->OnKeyDown(KeySym, CharCode, bIsRepeated);

			// Backspace input in only caught here.
			if (KeySym == SDLK_BACKSPACE)
			{
				MessageHandler->OnKeyChar('\b', bIsRepeated);
			}
			else if (KeySym == SDLK_RETURN)
			{
				MessageHandler->OnKeyChar('\r', bIsRepeated);
			}
		}
		break;
	case SDL_EVENT_KEY_UP:
		{
			const SDL_KeyboardEvent &KeyEvent = Event.key;
			const SDL_Keycode KeySym = KeyEvent.key;
			const uint32 CharCode = CharCodeFromSDLKeySym(KeySym);
			const bool IsRepeat = KeyEvent.repeat != 0;

			MessageHandler->OnKeyUp( KeySym, CharCode, IsRepeat );
		}
		break;
	case SDL_EVENT_TEXT_INPUT:
		{
			// Slate now gets all its text from here, I hope.
			const bool bIsRepeated = false;  //Event.key.repeat != 0;
			const FString TextStr(UTF8_TO_TCHAR(Event.text.text));
			for (auto TextIter = TextStr.CreateConstIterator(); TextIter; ++TextIter)
			{
				MessageHandler->OnKeyChar(*TextIter, bIsRepeated);
			}
		}
		break;
	case SDL_EVENT_MOUSE_MOTION:
		{
			SDL_MouseMotionEvent motionEvent = Event.motion;
			FLinuxCursor *LinuxCursor = (FLinuxCursor*)Cursor.Get();
			LinuxCursor->InvalidateCaches();

			if (!LinuxCursor->IsHidden())
			{
				FWindowProperties Props;
				GetWindowPropertiesInEventLoop(NativeWindow, Props);

				// When bUsingHighPrecisionMouseInput=1, changing the position cache causes the cursor (inside top/left/right etc. ViewPort)
				// to not move correct with the selection tool. The next part should be only run when not in Editor mode.
				if(!GIsEditor)
				{
					int32 BorderSizeX, BorderSizeY;
					CurrentEventWindow->GetNativeBordersSize(BorderSizeX, BorderSizeY);

					LinuxCursor->SetCachedPosition((float)(motionEvent.x + Props.Location.X + BorderSizeX), (float)(motionEvent.y + Props.Location.Y + BorderSizeY));
				}

				// Coordinate trace for diagnosing menu and tooltip cursor-offset issues. Logs
				// the SDL window-local event coords, the cached window screen origin, and the
				// SDL global cursor query side-by-side so a divergence is visible at a glance.
				if (UE_LOG_ACTIVE(LogLinuxWindow, VeryVerbose))
				{
					float GlobalX = 0.0f;
					float GlobalY = 0.0f;
					SDL_GetGlobalMouseState(&GlobalX, &GlobalY);
					int32 ParentX = 0;
					int32 ParentY = 0;
					SDL_GetWindowPosition(NativeWindow, &ParentX, &ParentY);
					FVector2D Cached = LinuxCursor->GetPosition();
					UE_LOGF(LogLinuxWindow, VeryVerbose,
						"MOUSE_MOTION wid=%u local=(%.1f,%.1f) sdl_win_pos=(%d,%d) props_loc=(%.1f,%.1f) global=(%.1f,%.1f) cached=(%.1f,%.1f)",
						motionEvent.windowID,
						motionEvent.x, motionEvent.y,
						ParentX, ParentY,
						Props.Location.X, Props.Location.Y,
						GlobalX, GlobalY,
						Cached.X, Cached.Y);
				}

				if( !CurrentEventWindow->GetDefinition().HasOSWindowBorder )
				{
					if ( CurrentEventWindow->IsRegularWindow() )
					{
						FVector2D CurrentPosition = LinuxCursor->GetPosition();
						MessageHandler->GetWindowZoneForPoint(CurrentEventWindow.ToSharedRef(), (int32)(CurrentPosition.X - Props.Location.X), (int32)(CurrentPosition.Y - Props.Location.Y));
						MessageHandler->OnCursorSet();
					}
				}
			}

			if(bUsingHighPrecisionMouseInput)
			{
				MessageHandler->OnRawMouseMove((int32)motionEvent.xrel, (int32)motionEvent.yrel);
			}
			else
			{
				MessageHandler->OnMouseMove();
			}
		}
		break;
	case SDL_EVENT_MOUSE_BUTTON_DOWN:
	case SDL_EVENT_MOUSE_BUTTON_UP:
		{
			SDL_MouseButtonEvent buttonEvent = Event.button;

			EMouseButtons::Type button;
			switch(buttonEvent.button)
			{
			case SDL_BUTTON_LEFT:
				button = EMouseButtons::Left;
				break;
			case SDL_BUTTON_MIDDLE:
				button = EMouseButtons::Middle;
				break;
			case SDL_BUTTON_RIGHT:
				button = EMouseButtons::Right;
				break;
			case SDL_BUTTON_X1:
				button = EMouseButtons::Thumb01;
				break;
			case SDL_BUTTON_X2:
				button = EMouseButtons::Thumb02;
				break;
			default:
				button = EMouseButtons::Invalid;
				break;
			}
			
			if (buttonEvent.type == SDL_EVENT_MOUSE_BUTTON_UP)
			{
				MessageHandler->OnMouseUp(button);
				
				if (buttonEvent.button == SDL_BUTTON_LEFT)
				{
					bIsDragWindowButtonPressed = false;
				}
			}
			else
			{
				UE_LOGF(LogLinuxWindowEvent, Verbose, "MOUSE_DOWN                                 : %d", CurrentEventWindow->GetID());
				// User clicked any button. Is the application active? If not activate it.
				if (!bActivateApp)
				{
					ActivateApplication();
				}

				if (buttonEvent.button == SDL_BUTTON_LEFT)
				{
					bIsDragWindowButtonPressed = true;
				}

				if (buttonEvent.clicks == 2)
				{
					MessageHandler->OnMouseDoubleClick(CurrentEventWindow, button);
				}
				else
				{
					// Check if we have to activate the window.
					if (CurrentlyActiveWindow != CurrentEventWindow)
					{
						ActivateWindow(CurrentEventWindow);
					}

					// Check if we have to set the focus.
					if(CurrentFocusWindow != CurrentEventWindow)
					{
						SDL_RaiseWindow(CurrentEventWindow->GetHWnd());
					}

					MessageHandler->OnMouseDown(CurrentEventWindow, button);

					if (NotificationWindows.Num() > 0)
					{
						RaiseNotificationWindows(CurrentEventWindow);
					}
				}
			}
		}
		break;
	case SDL_EVENT_MOUSE_WHEEL:
		{
			SDL_MouseWheelEvent *WheelEvent = &Event.wheel;
			float Amount = (float)WheelEvent->y * fMouseWheelScrollAccel;

			MessageHandler->OnMouseWheel(Amount);
		}
		break;
	case SDL_EVENT_GAMEPAD_ADDED:
		{
			SDL_GamepadDeviceEvent& GamepadEvent = Event.gdevice;
			AddGameController(GamepadEvent.which);
		}
		break;
	case SDL_EVENT_GAMEPAD_REMOVED:
		{
			SDL_GamepadDeviceEvent& GamepadEvent = Event.gdevice;
			RemoveGameController(GamepadEvent.which);
		}
		break;
	case SDL_EVENT_GAMEPAD_AXIS_MOTION:
		{
			SDL_GamepadAxisEvent caxisEvent = Event.gaxis;
			FGamepadKeyNames::Type Axis = FGamepadKeyNames::Invalid;
			float AxisValue = ShortToNormalFloat(caxisEvent.value);

			if (!ControllerStates.Contains(caxisEvent.which))
			{
				break;
			}

			SDLControllerState &ControllerState = ControllerStates[caxisEvent.which];
			FPlatformUserId UserId = IPlatformInputDeviceMapper::Get().GetUserForInputDevice(ControllerState.DeviceId);

			switch (caxisEvent.axis)
			{
			case SDL_GAMEPAD_AXIS_LEFTX:
				Axis = FGamepadKeyNames::LeftAnalogX;
				if(caxisEvent.value > GAMECONTROLLER_LEFT_THUMB_DEADZONE)
				{
					if(!ControllerState.AnalogOverThreshold[0])
					{
						MessageHandler->OnControllerButtonPressed(FGamepadKeyNames::LeftStickRight, UserId, ControllerState.DeviceId, false);
						ControllerState.AnalogOverThreshold[0] = true;
					}
				}
				else if(ControllerState.AnalogOverThreshold[0])
				{
					MessageHandler->OnControllerButtonReleased(FGamepadKeyNames::LeftStickRight, UserId, ControllerState.DeviceId, false);
					ControllerState.AnalogOverThreshold[0] = false;
				}
				if(caxisEvent.value < -GAMECONTROLLER_LEFT_THUMB_DEADZONE)
				{
					if(!ControllerState.AnalogOverThreshold[1])
					{
						MessageHandler->OnControllerButtonPressed(FGamepadKeyNames::LeftStickLeft, UserId, ControllerState.DeviceId, false);
						ControllerState.AnalogOverThreshold[1] = true;
					}
				}
				else if(ControllerState.AnalogOverThreshold[1])
				{
					MessageHandler->OnControllerButtonReleased(FGamepadKeyNames::LeftStickLeft, UserId, ControllerState.DeviceId, false);
					ControllerState.AnalogOverThreshold[1] = false;
				}
				break;
			case SDL_GAMEPAD_AXIS_LEFTY:
				Axis = FGamepadKeyNames::LeftAnalogY;
				AxisValue *= -1;
				if(caxisEvent.value > GAMECONTROLLER_LEFT_THUMB_DEADZONE)
				{
					if(!ControllerState.AnalogOverThreshold[2])
					{
						MessageHandler->OnControllerButtonPressed(FGamepadKeyNames::LeftStickDown, UserId, ControllerState.DeviceId, false);
						ControllerState.AnalogOverThreshold[2] = true;
					}
				}
				else if(ControllerState.AnalogOverThreshold[2])
				{
					MessageHandler->OnControllerButtonReleased(FGamepadKeyNames::LeftStickDown, UserId, ControllerState.DeviceId, false);
					ControllerState.AnalogOverThreshold[2] = false;
				}
				if(caxisEvent.value < -GAMECONTROLLER_LEFT_THUMB_DEADZONE)
				{
					if(!ControllerState.AnalogOverThreshold[3])
					{
						MessageHandler->OnControllerButtonPressed(FGamepadKeyNames::LeftStickUp, UserId, ControllerState.DeviceId, false);
						ControllerState.AnalogOverThreshold[3] = true;
					}
				}
				else if(ControllerState.AnalogOverThreshold[3])
				{
					MessageHandler->OnControllerButtonReleased(FGamepadKeyNames::LeftStickUp, UserId, ControllerState.DeviceId, false);
					ControllerState.AnalogOverThreshold[3] = false;
				}
				break;
			case SDL_GAMEPAD_AXIS_RIGHTX:
				Axis = FGamepadKeyNames::RightAnalogX;
				if(caxisEvent.value > GAMECONTROLLER_RIGHT_THUMB_DEADZONE)
				{
					if(!ControllerState.AnalogOverThreshold[4])
					{
						MessageHandler->OnControllerButtonPressed(FGamepadKeyNames::RightStickRight, UserId, ControllerState.DeviceId, false);
						ControllerState.AnalogOverThreshold[4] = true;
					}
				}
				else if(ControllerState.AnalogOverThreshold[4])
				{
					MessageHandler->OnControllerButtonReleased(FGamepadKeyNames::RightStickRight, UserId, ControllerState.DeviceId, false);
					ControllerState.AnalogOverThreshold[4] = false;
				}
				if(caxisEvent.value < -GAMECONTROLLER_RIGHT_THUMB_DEADZONE)
				{
					if(!ControllerState.AnalogOverThreshold[5])
					{
						MessageHandler->OnControllerButtonPressed(FGamepadKeyNames::RightStickLeft, UserId, ControllerState.DeviceId, false);
						ControllerState.AnalogOverThreshold[5] = true;
					}
				}
				else if(ControllerState.AnalogOverThreshold[5])
				{
					MessageHandler->OnControllerButtonReleased(FGamepadKeyNames::RightStickLeft, UserId, ControllerState.DeviceId, false);
					ControllerState.AnalogOverThreshold[5] = false;
				}
				break;
			case SDL_GAMEPAD_AXIS_RIGHTY:
				Axis = FGamepadKeyNames::RightAnalogY;
				AxisValue *= -1;
				if(caxisEvent.value > GAMECONTROLLER_RIGHT_THUMB_DEADZONE)
				{
					if(!ControllerState.AnalogOverThreshold[6])
					{
						MessageHandler->OnControllerButtonPressed(FGamepadKeyNames::RightStickDown, UserId, ControllerState.DeviceId, false);
						ControllerState.AnalogOverThreshold[6] = true;
					}
				}
				else if(ControllerState.AnalogOverThreshold[6])
				{
					MessageHandler->OnControllerButtonReleased(FGamepadKeyNames::RightStickDown, UserId, ControllerState.DeviceId, false);
					ControllerState.AnalogOverThreshold[6] = false;
				}
				if(caxisEvent.value < -GAMECONTROLLER_RIGHT_THUMB_DEADZONE)
				{
					if(!ControllerState.AnalogOverThreshold[7])
					{
						MessageHandler->OnControllerButtonPressed(FGamepadKeyNames::RightStickUp, UserId, ControllerState.DeviceId, false);
						ControllerState.AnalogOverThreshold[7] = true;
					}
				}
				else if(ControllerState.AnalogOverThreshold[7])
				{
					MessageHandler->OnControllerButtonReleased(FGamepadKeyNames::RightStickUp, UserId, ControllerState.DeviceId, false);
					ControllerState.AnalogOverThreshold[7] = false;
				}
				break;
			case SDL_GAMEPAD_AXIS_LEFT_TRIGGER:
				Axis = FGamepadKeyNames::LeftTriggerAnalog;
				if(caxisEvent.value > GAMECONTROLLER_TRIGGER_THRESHOLD)
				{
					if(!ControllerState.AnalogOverThreshold[8])
					{
						MessageHandler->OnControllerButtonPressed(FGamepadKeyNames::LeftTriggerThreshold, UserId, ControllerState.DeviceId, false);
						ControllerState.AnalogOverThreshold[8] = true;
					}
				}
				else if(ControllerState.AnalogOverThreshold[8])
				{
					MessageHandler->OnControllerButtonReleased(FGamepadKeyNames::LeftTriggerThreshold, UserId, ControllerState.DeviceId, false);
					ControllerState.AnalogOverThreshold[8] = false;
				}
				break;
			case SDL_GAMEPAD_AXIS_RIGHT_TRIGGER:
				Axis = FGamepadKeyNames::RightTriggerAnalog;
				if(caxisEvent.value > GAMECONTROLLER_TRIGGER_THRESHOLD)
				{
					if(!ControllerState.AnalogOverThreshold[9])
					{
						MessageHandler->OnControllerButtonPressed(FGamepadKeyNames::RightTriggerThreshold, UserId, ControllerState.DeviceId, false);
						ControllerState.AnalogOverThreshold[9] = true;
					}
				}
				else if(ControllerState.AnalogOverThreshold[9])
				{
					MessageHandler->OnControllerButtonReleased(FGamepadKeyNames::RightTriggerThreshold, UserId, ControllerState.DeviceId, false);
					ControllerState.AnalogOverThreshold[9] = false;
				}
				break;
			default:
				break;
			}

			if (Axis != FGamepadKeyNames::Invalid)
			{
				float & ExistingAxisEventValue = ControllerState.AxisEvents.FindOrAdd(Axis);
				ExistingAxisEventValue = AxisValue;
			}
		}
		break;
	case SDL_EVENT_GAMEPAD_BUTTON_DOWN:
	case SDL_EVENT_GAMEPAD_BUTTON_UP:
		{
			SDL_GamepadButtonEvent cbuttonEvent = Event.gbutton;
			FGamepadKeyNames::Type Button = FGamepadKeyNames::Invalid;

			if (!ControllerStates.Contains(cbuttonEvent.which))
			{
				break;
			}

			switch (cbuttonEvent.button)
			{
				case SDL_GAMEPAD_BUTTON_SOUTH:
					Button = FGamepadKeyNames::FaceButtonBottom;
					break;
				case SDL_GAMEPAD_BUTTON_EAST:
					Button = FGamepadKeyNames::FaceButtonRight;
					break;
				case SDL_GAMEPAD_BUTTON_WEST:
					Button = FGamepadKeyNames::FaceButtonLeft;
					break;
				case SDL_GAMEPAD_BUTTON_NORTH:
					Button = FGamepadKeyNames::FaceButtonTop;
					break;
				case SDL_GAMEPAD_BUTTON_BACK:
					Button = FGamepadKeyNames::SpecialLeft;
					break;
				case SDL_GAMEPAD_BUTTON_START:
					Button = FGamepadKeyNames::SpecialRight;
					break;
				case SDL_GAMEPAD_BUTTON_LEFT_STICK:
					Button = FGamepadKeyNames::LeftThumb;
					break;
				case SDL_GAMEPAD_BUTTON_RIGHT_STICK:
					Button = FGamepadKeyNames::RightThumb;
					break;
				case SDL_GAMEPAD_BUTTON_LEFT_SHOULDER:
					Button = FGamepadKeyNames::LeftShoulder;
					break;
				case SDL_GAMEPAD_BUTTON_RIGHT_SHOULDER:
					Button = FGamepadKeyNames::RightShoulder;
					break;
				case SDL_GAMEPAD_BUTTON_DPAD_UP:
					Button = FGamepadKeyNames::DPadUp;
					break;
				case SDL_GAMEPAD_BUTTON_DPAD_DOWN:
					Button = FGamepadKeyNames::DPadDown;
					break;
				case SDL_GAMEPAD_BUTTON_DPAD_LEFT:
					Button = FGamepadKeyNames::DPadLeft;
					break;
				case SDL_GAMEPAD_BUTTON_DPAD_RIGHT:
					Button = FGamepadKeyNames::DPadRight;
					break;
				default:
					break;
			}

			if (Button != FGamepadKeyNames::Invalid)
			{
				FPlatformUserId UserId = IPlatformInputDeviceMapper::Get().GetUserForInputDevice(ControllerStates[cbuttonEvent.which].DeviceId);

				if(cbuttonEvent.type == SDL_EVENT_GAMEPAD_BUTTON_DOWN)
				{
					MessageHandler->OnControllerButtonPressed(Button, UserId, ControllerStates[cbuttonEvent.which].DeviceId, false);
				}
				else
				{
					MessageHandler->OnControllerButtonReleased(Button, UserId, ControllerStates[cbuttonEvent.which].DeviceId, false);
				}
			}
		}
		break;

	case SDL_EVENT_WINDOW_PIXEL_SIZE_CHANGED:
		{
			int NewWidth  = Event.window.data1;
			int NewHeight = Event.window.data2;

			MessageHandler->OnSizeChanged(
				CurrentEventWindow.ToSharedRef(),
				NewWidth,
				NewHeight,
				//	bWasMinimized
				false
			);
		}
		break;

	case SDL_EVENT_WINDOW_RESIZED:
		{
			MessageHandler->OnResizingWindow( CurrentEventWindow.ToSharedRef() );
		}
		break;

	case SDL_EVENT_WINDOW_CLOSE_REQUESTED:
		{
			MessageHandler->OnWindowClose( CurrentEventWindow.ToSharedRef() );
		}
		break;

	case SDL_EVENT_WINDOW_SHOWN:
		{
			// (re)cache native properties
			//CurrentEventWindow->CacheNativeProperties();

			// A window did show up. Is the whole Application active? If not first activate it (ignore tooltips).
			if (!bActivateApp && !CurrentEventWindow->IsTooltipWindow())
			{
				ActivateApplication();
			}

			// Check if this window is different then the currently active one. If it is another one
			// activate that window and if necessary deactivate the one which was active.
			if (CurrentlyActiveWindow != CurrentEventWindow)
			{
				ActivateWindow(CurrentEventWindow);
			}

			// Set focus if the window wants to have a focus when first shown.
			if (CurrentEventWindow->IsFocusWhenFirstShown())
			{
				// If we don't surpress the window activation phase of RaiseWindow, X11/Gnome will bring up a notification window saying
				// the window is ready
				bool bActivate = SDL_GetHintBoolean(SDL_HINT_WINDOW_ACTIVATE_WHEN_RAISED, true);
				SDL_SetHint(SDL_HINT_WINDOW_ACTIVATE_WHEN_RAISED, "0");
				SDL_RaiseWindow(CurrentEventWindow->GetHWnd());
				SDL_SetHint(SDL_HINT_WINDOW_ACTIVATE_WHEN_RAISED, bActivate ? "1" : "0");
			}
		}
		break;

	case SDL_EVENT_WINDOW_MOVED:
		{
			// Mask away the higher bits (but preserve negative flags), as SDL uses those as flags
			// See: SDL_WINDOWPOS_UNDEFINED_MASK & SDL_WINDOWPOS_CENTERED_MASK for context
			int32 ClientScreenX = MaskAwayHigherBits(Event.window.data1);
			int32 ClientScreenY = MaskAwayHigherBits(Event.window.data2);

			// SDL3 reports parent-relative coordinates in WINDOW_MOVED for popup menus and
			// tooltips. Walk the parent chain to convert back to screen-absolute for Slate.
			const SDL_WindowFlags MovedFlags = SDL_GetWindowFlags(CurrentEventWindow->GetHWnd());
			if (MovedFlags & (SDL_WINDOW_POPUP_MENU | SDL_WINDOW_TOOLTIP))
			{
				SDL_HWindow Parent = SDL_GetWindowParent(CurrentEventWindow->GetHWnd());
				if (Parent)
				{
					int32 ParentScreenX = 0;
					int32 ParentScreenY = 0;
					GetWindowScreenPosition(Parent, ParentScreenX, ParentScreenY);
					ClientScreenX += ParentScreenX;
					ClientScreenY += ParentScreenY;
				}
			}

			int32 BorderSizeX, BorderSizeY;
			CurrentEventWindow->GetNativeBordersSize(BorderSizeX, BorderSizeY);
			ClientScreenX += BorderSizeX;
			ClientScreenY += BorderSizeY;

			UE_LOGF(LogLinuxWindow, VeryVerbose,
				"WINDOW_MOVED wid=%u event_xy=(%d,%d) screen_xy=(%d,%d)",
				SDL_GetWindowID(CurrentEventWindow->GetHWnd()),
				MaskAwayHigherBits(Event.window.data1),
				MaskAwayHigherBits(Event.window.data2),
				ClientScreenX, ClientScreenY);

			MessageHandler->OnMovedWindow(CurrentEventWindow.ToSharedRef(), ClientScreenX, ClientScreenY);

			if (bFirstFrameOfWindowMove)
			{
				bFirstFrameOfWindowMove = false;

				if (NotificationWindows.Num() > 0)
				{
					RaiseNotificationWindows(CurrentEventWindow);
				}
			}

			// Mouse dragging is the likely cause of this window move, so invalidate the cached cursor position
			FLinuxCursor *LinuxCursor = static_cast<FLinuxCursor*>(Cursor.Get());
			LinuxCursor->InvalidateCaches();
		}
		break;

	case SDL_EVENT_WINDOW_MAXIMIZED:
		{
			MessageHandler->OnWindowAction(CurrentEventWindow.ToSharedRef(), EWindowAction::Maximize);
			UE_LOGF(LogLinuxWindowEvent, Verbose, "Window: '%d' got maximized", CurrentEventWindow->GetID());

			if (NotificationWindows.Num() > 0)
			{
				RaiseNotificationWindows(CurrentEventWindow);
			}
		}
		break;

	case SDL_EVENT_WINDOW_RESTORED:
		{
			MessageHandler->OnWindowAction(CurrentEventWindow.ToSharedRef(), EWindowAction::Restore);
			UE_LOGF(LogLinuxWindowEvent, Verbose, "Window: '%d' got restored", CurrentEventWindow->GetID());

			if (NotificationWindows.Num() > 0)
			{
				RaiseNotificationWindows(CurrentEventWindow);
			}
		}
		break;

	case SDL_EVENT_WINDOW_MOUSE_ENTER:
		{
			if (CurrentEventWindow.IsValid())
			{
				MessageHandler->OnCursorSet();

				// Tooltip windows also receive enter/leave events as they map/raise, but they're
				// decoration -- they should never be considered the window "under" the cursor for
				// hit-testing.
				if (!CurrentEventWindow->IsTooltipWindow())
				{
					CurrentUnderCursorWindow = CurrentEventWindow;
					bInsideOwnWindow = true;
				}

				UE_LOGF(LogLinuxWindow, Verbose, "Entered one of application windows. Cursor under Window: '%d'%hs",
					CurrentEventWindow->GetID(),
					CurrentEventWindow->IsTooltipWindow() ? " (tooltip -- cache not updated)" : "");
			}
		}
		break;

	case SDL_EVENT_WINDOW_MOUSE_LEAVE:
		{
			if (CurrentEventWindow.IsValid())
			{
				if (GetCapture() != nullptr)
				{
					UpdateMouseCaptureWindow((SDL_HWindow)GetCapture());
				}

				// Mirror of the MOUSE_ENTER filter: ignore leaves on tooltip windows -- if we never
				// pointed the cache at the tooltip, we don't need to clear it when the tooltip
				// unmaps.
				if (CurrentEventWindow->IsTooltipWindow())
				{
					UE_LOGF(LogLinuxWindow, Verbose, "Left tooltip Window '%d'. Cache not touched.", CurrentEventWindow->GetID());
					break;
				}

				SDL_Window* const SDLFocusWindow = SDL_GetMouseFocus();
				if (SDLFocusWindow != nullptr)
				{
					TSharedPtr<FLinuxWindow> FocusedLinuxWindow = FindWindowBySDLWindow(SDLFocusWindow);
					if (FocusedLinuxWindow.IsValid() && !FocusedLinuxWindow->IsTooltipWindow())
					{
						CurrentUnderCursorWindow = FocusedLinuxWindow;
						bInsideOwnWindow = true;
						UE_LOGF(LogLinuxWindow, Verbose, "Left Window '%d' but pointer is still inside Window '%d'. Keeping under-cursor cache.",
							CurrentEventWindow->GetID(), FocusedLinuxWindow->GetID());
						break;
					}
				}

				CurrentUnderCursorWindow = nullptr;
				bInsideOwnWindow = false;
				UE_LOGF(LogLinuxWindow, Verbose, "Left an application window we were hovering above.");
			}
		}
		break;

	case SDL_EVENT_WINDOW_HIT_TEST:
		{
			UE_LOGF(LogLinuxWindowEvent, Verbose, "SDL_EVENT_WINDOW_HIT_TEST received");

			// SDL3 X11 XInput2 emits two XI_ButtonPress events per single
			// Button1 press (one through the logical pointer aggregate, one
			// direct from the physical pointer device), and
			// X11_HandleButtonPress fires SDL_EVENT_WINDOW_HIT_TEST on each.
			// SDL_PrivateSendMouseButton dedups the button events via
			// per-source buttonstate tracking, but HIT_TEST bypasses that
			// gate, so the editor receives two HIT_TEST events ~1ms apart
			// for every single click in a draggable region -- which the
			// title-bar double-click detector below would otherwise trip on
			// every single click. Coalesce duplicates within 50 ms; real
			// human double-clicks take >100 ms.
			const uint64 HitTestEventTimeNS = Event.window.timestamp;
			static constexpr uint64 HitTestDedupWindowNS = 50ULL * 1000000ULL;
			if (LastHitTestTimestampNS != 0 && (HitTestEventTimeNS - LastHitTestTimestampNS) < HitTestDedupWindowNS)
			{
				UE_LOGF(LogLinuxWindowEvent, Verbose, "  ignoring duplicate HIT_TEST within %llu ns of previous", static_cast<unsigned long long>(HitTestEventTimeNS - LastHitTestTimestampNS));
				break;
			}
			LastHitTestTimestampNS = HitTestEventTimeNS;

			// The user clicked into the hit test area (Titlebar for example). Is the whole Application active?
			// If not, first activate (ignore tooltips).
			if (!bActivateApp && !CurrentEventWindow->IsTooltipWindow())
			{
				ActivateApplication();
			}

			// Check if this window is different then the currently active one. If it is another one activate this 
			// window and deactivate the other one.
			if (CurrentlyActiveWindow != CurrentEventWindow)
			{
				ActivateWindow(CurrentEventWindow);
			}

			// Set the input focus.
			if (CurrentEventWindow.IsValid())
			{
				SDL_RaiseWindow(CurrentEventWindow->GetHWnd());
			}

			if (NotificationWindows.Num() > 0)
			{
				RaiseNotificationWindows(CurrentEventWindow);
			}

			bFirstFrameOfWindowMove = true;

			// Using the SDL_EVENT_WINDOW_HIT_TEST event as a replacement to a mouse click event.  Which SDL doesn't send for clicks 
			// in the draggable hit test area (title bars in UE)
			// If we see two of these close enough together, assume it's a double click on the titlebar area and maximize/restore the window

			uint64 EventTimeNS;
			float MouseX;
			float MouseY;

			// Get the event time from the event
			const SDL_WindowEvent &WindowEvent = Event.window;
			EventTimeNS = WindowEvent.timestamp;
			// The event doesn't pass mouse coordinates, it always passes 0,0 so we are going to get them from the system
			SDL_GetGlobalMouseState(&MouseX, &MouseY);

			if (LastTitleBarClickTimestampNS > 0 && LastTitleBarClickWindow.IsValid() &&
				LastTitleBarClickWindow == CurrentEventWindow &&
				((EventTimeNS - LastTitleBarClickTimestampNS) <= (GetSDLDoubleClickTime() * 1000000ULL)) &&
				(FMath::Abs(MouseX - LastTitleBarClickX) <= (float)GetSDLDoubleClickRadius()) &&
				(FMath::Abs(MouseY - LastTitleBarClickY) <= (float)GetSDLDoubleClickRadius()))
			{
				// double clicked the title bar we need to maximize or restore the window
				UE_LOGF(LogLinuxWindowEvent, Verbose, "MOUSE Double Click on Titlebar X:%f Y:%f", MouseX, MouseY);
				if (CurrentEventWindow->IsMaximized())
				{
					CurrentEventWindow->Restore();
				}
				else
				{
					CurrentEventWindow->Maximize();
				}
				// reset the last click
				LastTitleBarClickTimestampNS = 0;
				LastTitleBarClickWindow = nullptr;
			}
			else
			{
				// remember this click and see if another one happens quickly
				LastTitleBarClickTimestampNS = EventTimeNS;
				LastTitleBarClickX = MouseX;
				LastTitleBarClickY = MouseY;
				LastTitleBarClickWindow = CurrentEventWindow;
			}
		}
		break;

	case SDL_EVENT_WINDOW_FOCUS_GAINED:
		{
			UE_LOGF(LogLinuxWindowEvent, Verbose, "WM_SETFOCUS                                 : %d", CurrentEventWindow->GetID());

			CurrentFocusWindow = CurrentEventWindow;

			if (NotificationWindows.Num() > 0)
			{
				RaiseNotificationWindows(CurrentEventWindow);
			}

			// We have gained focus, we can stop trying to check if we need to deactivate
			FocusOutDeactivationTime = 0.0;
		}
		break;

	case SDL_EVENT_WINDOW_FOCUS_LOST:
		{
			UE_LOGF(LogLinuxWindowEvent, Verbose, "WM_KILLFOCUS                                : %d", CurrentEventWindow->GetID());

			// OK, the active window lost focus. This could mean the app went completely out of
			// focus. That means the app must be deactivated. To make sure that the user did
			// not click to another window we delay the deactivation.
			// TODO Figure out if the delay time may cause problems.
			if(CurrentFocusWindow == CurrentEventWindow)
			{
				// Only do if the application is active.
				if(bActivateApp)
				{
					FocusOutDeactivationTime = FPlatformTime::Seconds() + DeactivationThreadshold;
				}
			}
			CurrentFocusWindow = nullptr;
		}
		break;

	case SDL_EVENT_WINDOW_HIDDEN:	// intended fall-through
	case SDL_EVENT_WINDOW_EXPOSED:	// intended fall-through
	case SDL_EVENT_WINDOW_MINIMIZED:	// intended fall-through
		break;

	case SDL_EVENT_DROP_BEGIN:
		{
			check(DragAndDropQueue.Num() == 0);  // did we get confused?
			check(DragAndDropTextQueue.Num() == 0);  // did we get confused?
		}
		break;

	case SDL_EVENT_DROP_FILE:
		{
			FString tmp = StringUtility::UnescapeURI(UTF8_TO_TCHAR(Event.drop.data));
			DragAndDropQueue.Add(tmp);
			UE_LOGF(LogLinuxWindowEvent, Verbose, "File dropped: %ls", *tmp);
		}
		break;

	case SDL_EVENT_DROP_TEXT:
		{
			FString tmp = UTF8_TO_TCHAR(Event.drop.data);
			DragAndDropTextQueue.Add(tmp);
			UE_LOGF(LogLinuxWindowEvent, Verbose, "Text dropped: %ls", *tmp);
		}
		break;

	case SDL_EVENT_DROP_COMPLETE:
		{
			if (DragAndDropQueue.Num() > 0)
			{
				MessageHandler->OnDragEnterFiles(CurrentEventWindow.ToSharedRef(), DragAndDropQueue);
				MessageHandler->OnDragDrop(CurrentEventWindow.ToSharedRef());
				DragAndDropQueue.Empty();
			}

			if (DragAndDropTextQueue.Num() > 0)
			{
				for (const auto & Text : DragAndDropTextQueue)
				{
					MessageHandler->OnDragEnterText(CurrentEventWindow.ToSharedRef(), Text);
					MessageHandler->OnDragDrop(CurrentEventWindow.ToSharedRef());
				}
				DragAndDropTextQueue.Empty();
			}
			UE_LOGF(LogLinuxWindowEvent, Verbose, "DragAndDrop finished for Window              : %d", CurrentEventWindow->GetID());
		}
		break;

	case SDL_EVENT_FINGER_DOWN:
		{
			UE_LOGF(LogLinuxWindow, Verbose, "Finger %llu is down at (%f, %f)", Event.tfinger.fingerID, Event.tfinger.x, Event.tfinger.y);

			// touch events can have no window associated with them, in that case ignore (with a warning)
			if (LIKELY(!bWindowlessEvent))
			{
				// remove touch context even if it existed
				uint64 FingerId = static_cast<uint64>(Event.tfinger.fingerID);

				FTouchContext* TouchContext = Touches.Find(FingerId);
				if (UNLIKELY(Touches.Find(FingerId) != nullptr))
				{
					TouchIds[TouchContext->TouchIndex] = TOptional<uint64>();
					Touches.Remove(FingerId);
					UE_LOGF(LogLinuxWindow, Warning, "Received another SDL_FINGERDOWN for finger %llu which was already down.", FingerId);
				}

				FTouchContext NewTouch;
				NewTouch.TouchIndex = GetFirstFreeTouchId();
				NewTouch.Location = GetTouchEventLocation(NativeWindow, Event);
				NewTouch.DeviceId = Event.tfinger.touchID;
				Touches.Add(FingerId, NewTouch);
				TouchIds[NewTouch.TouchIndex] = TOptional<uint64>(FingerId);

				UE_LOGF(LogLinuxWindow, Verbose, "OnTouchStarted at (%f, %f), finger %lld (system touch id %u)", NewTouch.Location.X, NewTouch.Location.Y, FingerId, NewTouch.TouchIndex);
				MessageHandler->OnTouchStarted(CurrentEventWindow, NewTouch.Location, 1.0f, NewTouch.TouchIndex, 0);// NewTouch.DeviceId);
			}
			else
			{
				UE_LOGF(LogLinuxWindow, Warning, "Ignoring touch event SDL_FINGERDOWN (finger: %llu, x=%f, y=%f) that doesn't have a window associated with it",
					Event.tfinger.fingerID, Event.tfinger.x, Event.tfinger.y);
			}
		}
		break;
	case SDL_EVENT_FINGER_UP:
		{
			UE_LOGF(LogLinuxWindow, Verbose, "Finger %llu is up at (%f, %f)", Event.tfinger.fingerID, Event.tfinger.x, Event.tfinger.y);

			// touch events can have no window associated with them, in that case ignore (with a warning)
			if (LIKELY(!bWindowlessEvent))
			{
				uint64 FingerId = static_cast<uint64>(Event.tfinger.fingerID);
				FTouchContext* TouchContext = Touches.Find(FingerId);
				if (UNLIKELY(TouchContext == nullptr))
				{
					UE_LOGF(LogLinuxWindow, Warning, "Received SDL_FINGERUP for finger %llu which was already up.", FingerId);
					// do not send a duplicate up
				}
				else
				{
					TouchContext->Location = GetTouchEventLocation(NativeWindow, Event);
					// check touch device?

					UE_LOGF(LogLinuxWindow, Verbose, "OnTouchEnded at (%f, %f), finger %lld (system touch id %u)", TouchContext->Location.X, TouchContext->Location.Y, FingerId, TouchContext->TouchIndex);
					MessageHandler->OnTouchEnded(TouchContext->Location, TouchContext->TouchIndex, 0);// TouchContext->DeviceId);

					// remove the touch
					TouchIds[TouchContext->TouchIndex] = TOptional<uint64>();
					Touches.Remove(FingerId);
				}
			}
			else
			{
				UE_LOGF(LogLinuxWindow, Warning, "Ignoring touch event SDL_FINGERUP (finger: %llu, x=%f, y=%f) that doesn't have a window associated with it",
					Event.tfinger.fingerID, Event.tfinger.x, Event.tfinger.y);
			}
		}
		break;
	case SDL_EVENT_FINGER_MOTION:
		{
			// touch events can have no window associated with them, in that case ignore (with a warning)
			if (LIKELY(!bWindowlessEvent))
			{
				uint64 FingerId = static_cast<uint64>(Event.tfinger.fingerID);
				FTouchContext* TouchContext = Touches.Find(FingerId);
				if (UNLIKELY(TouchContext == nullptr))
				{
					UE_LOGF(LogLinuxWindow, Warning, "Received SDL_FINGERMOTION for finger %llu which was not down.", FingerId);
					// ignore the event
				}
				else
				{
					// do not send moved event if position has not changed
					FVector2D Location = GetTouchEventLocation(NativeWindow, Event);
					if (LIKELY((Location - TouchContext->Location).IsNearlyZero() == false))
					{
						TouchContext->Location = Location;
						UE_LOGF(LogLinuxWindow, Verbose, "OnTouchMoved at (%f, %f), finger %d (system touch id %llu)", TouchContext->Location.X, TouchContext->Location.Y, TouchContext->TouchIndex, FingerId);
						MessageHandler->OnTouchMoved(TouchContext->Location, 1.0f, TouchContext->TouchIndex, 0);// TouchContext->DeviceId);
					}
				}
			}
			else
			{
				UE_LOGF(LogLinuxWindow, Warning, "Ignoring touch event SDL_FINGERMOTION (finger: %llu, x=%f, y=%f) that doesn't have a window associated with it",
					Event.tfinger.fingerID, Event.tfinger.x, Event.tfinger.y);
			}
		}
		break;

	case SDL_EVENT_QUIT:
		{
			// SDL sends this when the Wayland display connection is killed (e.g. protocol error)
			// or when the user requests the application to quit. Request a clean engine exit.
			UE_LOGF(LogLinuxWindow, Warning, "Received SDL_EVENT_QUIT, requesting engine exit.");
			FLinuxPlatformMisc::RequestExitWithStatus(true, 0);
		}
		break;

	default:
		UE_LOGF(LogLinuxWindow, Verbose, "Received unknown SDL event type=%d", Event.type);
		break;
	}
}

void FLinuxApplication::CheckIfApplicatioNeedsDeactivation()
{
	// If FocusOutDeactivationTime is set we have had a focus out event and are waiting to see if we need to Deactivate the Application
	if (!FMath::IsNearlyZero(FocusOutDeactivationTime))
	{
		// We still havent hit our timeout limit, keep waiting
		if (FocusOutDeactivationTime > FPlatformTime::Seconds())
		{
			return;
		}
		// If we don't use bIsDragWindowButtonPressed the draged window will be destroyed because we
		// deactivate the whole appliacton. TODO Is that a bug? Do we have to do something?
		else if (!CurrentFocusWindow.IsValid() && !bIsDragWindowButtonPressed)
		{
			DeactivateApplication();

			FocusOutDeactivationTime = 0.0;
		}
		else
		{
			FocusOutDeactivationTime = 0.0;
		}
	}
}

int FLinuxApplication::GetFirstFreeTouchId()
{
	for (int i = 0; i < TouchIds.Num(); i++)
	{
		if (TouchIds[i].IsSet() == false)
		{
			return i;
		}
	}

	return TouchIds.Add(TOptional<uint64>()); 
}

FVector2D FLinuxApplication::GetTouchEventLocation(SDL_HWindow NativeWindow, SDL_Event TouchEvent)
{
	checkf(TouchEvent.type == SDL_EVENT_FINGER_DOWN || TouchEvent.type == SDL_EVENT_FINGER_UP || TouchEvent.type == SDL_EVENT_FINGER_MOTION, TEXT("Wrong touch event."));

	FWindowProperties Props;
	GetWindowPropertiesInEventLoop(NativeWindow, Props);
	// coordinates aren't necessarily normalized: e.g. if the input is grabbed and we're sliding outside of the window we can get x > 1
	return FVector2D(Props.Location.X + Props.Size.X * TouchEvent.tfinger.x, Props.Location.Y + Props.Size.Y * TouchEvent.tfinger.y);
}

EWindowZone::Type FLinuxApplication::WindowHitTest(const TSharedPtr< FLinuxWindow > &Window, int x, int y)
{
	return MessageHandler->GetWindowZoneForPoint(Window.ToSharedRef(), x, y);
}

void FLinuxApplication::ProcessDeferredEvents( const float TimeDelta )
{
	// This function can be reentered when entering a modal tick loop.
	// We need to make a copy of the events that need to be processed or we may end up processing the same messages twice
	SDL_HWindow NativeWindow = NULL;

	TArray< SDL_Event > Events( PendingEvents );
	PendingEvents.Empty();

	for( int32 Index = 0; Index < Events.Num(); ++Index )
	{
		ProcessDeferredMessage( Events[Index] );
	}
}

void FLinuxApplication::PollGameDeviceState( const float TimeDelta )
{
	IPlatformInputDeviceMapper& Mapper = IPlatformInputDeviceMapper::Get();
	for(auto ControllerIt = ControllerStates.CreateIterator(); ControllerIt; ++ControllerIt)
	{
		for(auto Event = ControllerIt.Value().AxisEvents.CreateConstIterator(); Event; ++Event)
		{
			FPlatformUserId UserId = Mapper.GetUserForInputDevice(ControllerIt.Value().DeviceId);
			MessageHandler->OnControllerAnalog(Event.Key(), UserId, ControllerIt.Value().DeviceId, Event.Value());
		}
		ControllerIt.Value().AxisEvents.Empty();

		ControllerIt.Value().UpdateHapticEffect();
	}

	if (!bHasLoadedInputPlugins)
	{
		// Create all externally-implemented input devices from loaded modules.
		TArray<IInputDeviceModule*> PluginImplementations = IModularFeatures::Get().GetModularFeatureImplementations<IInputDeviceModule>(IInputDeviceModule::GetModularFeatureName());
		for (auto InputPluginIt = PluginImplementations.CreateIterator(); InputPluginIt; ++InputPluginIt)
		{
			TSharedPtr<IInputDevice> Device = (*InputPluginIt)->CreateInputDevice(MessageHandler);
			if (Device.IsValid())
			{
				UE_LOGF(LogInit, Log, "Adding external input plugin.");
				ExternalInputDevices.Add(MoveTemp(Device));
			}
		}

		// Modules loaded after this point are handled by this callback
		IModularFeatures::Get().OnModularFeatureRegistered().AddRaw(this, &FLinuxApplication::OnInputDeviceModuleRegistered);

		bHasLoadedInputPlugins = true;
	}
	
	// Poll externally-implemented devices
	for (auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt)
	{
		(*DeviceIt)->Tick(TimeDelta);
		(*DeviceIt)->SendControllerEvents();
	}
}

TSharedPtr< FLinuxWindow > FLinuxApplication::FindEventWindow(SDL_Event* Event, bool& bOutWindowlessEvent)
{
	uint32 WindowID = 0;
	bOutWindowlessEvent = false;

	if(Event->type >= SDL_EVENT_WINDOW_FIRST && Event->type <= SDL_EVENT_WINDOW_LAST)
	{
		WindowID = Event->window.windowID;
	}
	else switch (Event->type)
	{
		case SDL_EVENT_TEXT_INPUT:
			WindowID = Event->text.windowID;
			break;
		case SDL_EVENT_TEXT_EDITING:
			WindowID = Event->edit.windowID;
			break;
		case SDL_EVENT_KEY_DOWN:
		case SDL_EVENT_KEY_UP:
			WindowID = Event->key.windowID;
			break;
		case SDL_EVENT_MOUSE_MOTION:
			WindowID = Event->motion.windowID;
			break;
		case SDL_EVENT_MOUSE_BUTTON_DOWN:
		case SDL_EVENT_MOUSE_BUTTON_UP:
			WindowID = Event->button.windowID;
			break;
		case SDL_EVENT_MOUSE_WHEEL:
			WindowID = Event->wheel.windowID;
			break;
		case SDL_EVENT_DROP_BEGIN:
		case SDL_EVENT_DROP_FILE:
		case SDL_EVENT_DROP_TEXT:
		case SDL_EVENT_DROP_COMPLETE:
			WindowID = Event->drop.windowID;
			break;
		case SDL_EVENT_FINGER_DOWN:
		case SDL_EVENT_FINGER_UP:
		case SDL_EVENT_FINGER_MOTION:
			// SDL touch events are windowless, but Slate needs to associate touch down with a particular window.
			// Assume that the current focus window is the one relevant for the touch and if there's none, assume the event windowless
			if (CurrentFocusWindow.IsValid())
			{
				return CurrentFocusWindow;
			}
			else
			{
				bOutWindowlessEvent = true;
				return TSharedPtr<FLinuxWindow>(nullptr);
			}
		default:
			bOutWindowlessEvent = true;
			return TSharedPtr< FLinuxWindow >(nullptr);
	}

	for (int32 WindowIndex=0; WindowIndex < Windows.Num(); ++WindowIndex)
	{
		TSharedRef< FLinuxWindow > Window = Windows[WindowIndex];
		
		if (SDL_GetWindowID(Window->GetHWnd()) == WindowID)
		{
			return Window;
		}
	}

	return TSharedPtr< FLinuxWindow >(nullptr);
}

void FLinuxApplication::RemoveEventWindow(SDL_HWindow HWnd)
{
	for (int32 WindowIndex=0; WindowIndex < Windows.Num(); ++WindowIndex)
	{
		TSharedRef< FLinuxWindow > Window = Windows[ WindowIndex ];
		
		if ( Window->GetHWnd() == HWnd )
		{
			Windows.RemoveAt(WindowIndex);
			return;
		}
	}
}

FModifierKeysState FLinuxApplication::GetModifierKeys() const
{
	SDL_Keymod modifiers = SDL_GetModState();

	const bool bIsLeftShiftDown	    = (modifiers & SDL_KMOD_LSHIFT) != 0;
	const bool bIsRightShiftDown	= (modifiers & SDL_KMOD_RSHIFT) != 0;
	const bool bIsLeftControlDown	= (modifiers & SDL_KMOD_LCTRL) != 0;
	const bool bIsRightControlDown	= (modifiers & SDL_KMOD_RCTRL) != 0;
	const bool bIsLeftAltDown	    = (modifiers & SDL_KMOD_LALT) != 0;
	const bool bIsRightAltDown	    = (modifiers & SDL_KMOD_RALT) != 0;
	const bool bAreCapsLocked	    = (modifiers & SDL_KMOD_CAPS) != 0;

	return FModifierKeysState( bIsLeftShiftDown, bIsRightShiftDown, bIsLeftControlDown, bIsRightControlDown, bIsLeftAltDown, bIsRightAltDown, false, false, bAreCapsLocked );
}

void FLinuxApplication::SetCapture( const TSharedPtr< FGenericWindow >& InWindow )
{
	bIsMouseCaptureEnabled = InWindow.IsValid();
	UpdateMouseCaptureWindow( bIsMouseCaptureEnabled ? ((FLinuxWindow*)InWindow.Get())->GetHWnd() : NULL );
}

void* FLinuxApplication::GetCapture( void ) const
{
	return ( bIsMouseCaptureEnabled && MouseCaptureWindow ) ? MouseCaptureWindow : NULL;
}

bool FLinuxApplication::IsGamepadAttached() const
{
	for (const TPair<SDL_JoystickID, SDLControllerState>& ControllerState : ControllerStates)
	{
		if (SDL_GamepadConnected(ControllerState.Value.Controller))
		{
			return true;
		}
	}
	
	return false;
}

void FLinuxApplication::UpdateMouseCaptureWindow(SDL_HWindow TargetWindow)
{
	const bool bEnable = bIsMouseCaptureEnabled || bIsMouseCursorLocked;
	FLinuxCursor *LinuxCursor = static_cast<FLinuxCursor*>(Cursor.Get());

	// this is a hacky heuristic which makes QA-ClickHUD work while not ruining SlateViewer...
	bool bShouldGrab = (IS_PROGRAM != 0 || WITH_ENGINE != 0 || GIsEditor) && !LinuxCursor->IsHidden();
	if (bEnable)
	{
		if (TargetWindow)
		{
			MouseCaptureWindow = TargetWindow;
		}
		if (bShouldGrab && MouseCaptureWindow)
		{
			SDL_CaptureMouse(true);
		}
	}
	else
	{
		SDL_CaptureMouse(false);
		MouseCaptureWindow = nullptr;
	}
}

void FLinuxApplication::SetHighPrecisionMouseMode( const bool Enable, const TSharedPtr< FGenericWindow >& InWindow )
{
	static const FString EnvValue = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_NORELATIVEMOUSEMODE")); 
	static const bool bNoRelativeMouseMode = (FParse::Param(FCommandLine::Get(), TEXT("norelativemousemode")) || !EnvValue.IsEmpty());

	MessageHandler->OnCursorSet();
	bUsingHighPrecisionMouseInput = Enable;
	
	if (!bNoRelativeMouseMode)
	{
		if(!InWindow.Get())
		{
			return;
		}
			
		SDL_SetWindowRelativeMouseMode(((FLinuxWindow*)InWindow.Get())->GetHWnd(), Enable);
	}
}

void FLinuxApplication::RefreshDisplayCache()
{
	const double kCacheLifetime = 5.0;

	double CurrentTime = FPlatformTime::Seconds();
	if (CurrentTime - LastTimeCachedDisplays > kCacheLifetime)
	{
		CachedDisplays.Empty();

		int NumDisplays = 0;
		SDL_DisplayID* Displays = SDL_GetDisplays(&NumDisplays);

		for (int DisplayIdx = 0; DisplayIdx < NumDisplays; ++DisplayIdx)
		{
			FCachedDisplay Entry;
			Entry.Id = Displays[DisplayIdx];
			if (SDL_GetDisplayUsableBounds(Entry.Id, &Entry.UsableBounds))
			{
				CachedDisplays.Add(Entry);
			}
		}
		SDL_free(Displays);

		LastTimeCachedDisplays = CurrentTime;
	}
}

FPlatformRect FLinuxApplication::GetWorkArea( const FPlatformRect& CurrentWindow ) const
{
	(const_cast<FLinuxApplication *>(this))->RefreshDisplayCache();

	if (CachedDisplays.Num() <= 0)
	{
		return CurrentWindow;
	}

	// Let SDL pick the display from full bounds. Usable bounds inherit the
	// global _NET_WORKAREA panel offset and miscompare for rects above it.
	SDL_Rect QueryRect;
	QueryRect.x = CurrentWindow.Left;
	QueryRect.y = CurrentWindow.Top;
	QueryRect.w = FMath::Max(1, CurrentWindow.Right  - CurrentWindow.Left);
	QueryRect.h = FMath::Max(1, CurrentWindow.Bottom - CurrentWindow.Top);

	SDL_DisplayID PickedId = SDL_GetDisplayForRect(&QueryRect);

	const SDL_Rect* PickedUsable = nullptr;
	for (const FCachedDisplay& Cached : CachedDisplays)
	{
		if (Cached.Id == PickedId)
		{
			PickedUsable = &Cached.UsableBounds;
			break;
		}
	}
	if (PickedUsable == nullptr)
	{
		PickedUsable = &CachedDisplays[0].UsableBounds;
	}

	FPlatformRect WorkArea;
	WorkArea.Left	= PickedUsable->x;
	WorkArea.Top	= PickedUsable->y;
	WorkArea.Right	= PickedUsable->x + PickedUsable->w;
	WorkArea.Bottom	= PickedUsable->y + PickedUsable->h;

	UE_LOGF(LogLinuxWindow, Verbose, "GetWorkArea: input=(L=%d,T=%d,R=%d,B=%d) -> WorkArea=(L=%d,T=%d,R=%d,B=%d)",
		CurrentWindow.Left, CurrentWindow.Top, CurrentWindow.Right, CurrentWindow.Bottom,
		WorkArea.Left, WorkArea.Top, WorkArea.Right, WorkArea.Bottom);

	return WorkArea;
}

void FDisplayMetrics::RebuildDisplayMetrics(FDisplayMetrics& OutDisplayMetrics)
{
	if (FNullPlatformApplicationMisc::IsUsingNullApplication())
	{
		FNullPlatformDisplayMetrics::RebuildDisplayMetrics(OutDisplayMetrics);
	}
	else
	{
		if (LIKELY(FApp::CanEverRender()))
		{
			if (!FLinuxPlatformApplicationMisc::InitSDL()) //	will not initialize more than once
			{
				UE_LOGF(LogInit, Warning, "FDisplayMetrics::GetDisplayMetrics: InitSDL() failed, cannot get display metrics");
			}
		}

		int NumDisplays = 0;
		SDL_DisplayID* Displays = nullptr;

		if(SDL_WasInit(SDL_INIT_VIDEO))
		{
			Displays = SDL_GetDisplays(&NumDisplays);
		}

		OutDisplayMetrics.MonitorInfo.Empty();

		if (NumDisplays <= 0)
		{
			if (IsRunningDedicatedServer())
			{
				// dedicated servers has always been exiting early
				OutDisplayMetrics.PrimaryDisplayWorkAreaRect = FPlatformRect(0, 0, 0, 0);
				OutDisplayMetrics.VirtualDisplayRect = OutDisplayMetrics.PrimaryDisplayWorkAreaRect;
				OutDisplayMetrics.PrimaryDisplayWidth = 0;
				OutDisplayMetrics.PrimaryDisplayHeight = 0;

				if(Displays)
				{
					SDL_free(Displays);
					Displays = nullptr;
				}
				return;
			}
			else
			{
				// headless clients need some plausible values because high level logic depends on viewport sizes not being 0 (see e.g. UnrealClient.cpp)
				int32 Width = 1920;
				int32 Height = 1080;

				FMonitorInfo Display;
				Display.bIsPrimary = true;
				if (FPlatformApplicationMisc::IsHighDPIAwarenessEnabled())
				{
					Display.DPI = 96;
				}
				Display.ID = TEXT("fakedisplay");
				Display.NativeWidth = Width;
				Display.NativeHeight = Height;
				Display.MaxResolution = FIntPoint(Width, Height);
				Display.DisplayRect = FPlatformRect(0, 0, Width, Height);
				Display.WorkArea = FPlatformRect(0, 0, Width, Height);

				OutDisplayMetrics.PrimaryDisplayWorkAreaRect = Display.WorkArea;
				OutDisplayMetrics.VirtualDisplayRect = OutDisplayMetrics.PrimaryDisplayWorkAreaRect;
				OutDisplayMetrics.PrimaryDisplayWidth = Display.NativeWidth;
				OutDisplayMetrics.PrimaryDisplayHeight = Display.NativeHeight;
				OutDisplayMetrics.MonitorInfo.Add(Display);
			}
		}

		for (int32 DisplayIdx = 0; DisplayIdx < NumDisplays; ++DisplayIdx)
		{
			SDL_Rect DisplayBounds, UsableBounds;
			FMonitorInfo Display;
			SDL_GetDisplayBounds(Displays[DisplayIdx], &DisplayBounds);
			SDL_GetDisplayUsableBounds(Displays[DisplayIdx], &UsableBounds);

			Display.Name = UTF8_TO_TCHAR(SDL_GetDisplayName(Displays[DisplayIdx]));
			Display.ID = FString::Printf(TEXT("display%d"), DisplayIdx);
			Display.NativeWidth = DisplayBounds.w;
			Display.NativeHeight = DisplayBounds.h;
			Display.MaxResolution = FIntPoint(DisplayBounds.w, DisplayBounds.h);
			Display.DisplayRect = FPlatformRect(DisplayBounds.x, DisplayBounds.y, DisplayBounds.x + DisplayBounds.w, DisplayBounds.y + DisplayBounds.h);
			Display.WorkArea = FPlatformRect(UsableBounds.x, UsableBounds.y, UsableBounds.x + UsableBounds.w, UsableBounds.y + UsableBounds.h);
			Display.bIsPrimary = DisplayIdx == 0;

			if (FPlatformApplicationMisc::IsHighDPIAwarenessEnabled())
			{
				// This is really only an approximation because SDL_GetDisplayDPI() was removed for being unreliable across platforms
				Display.DPI = (int32)(SDL_GetDisplayContentScale(Displays[DisplayIdx]) * 96.0f);
			}

			OutDisplayMetrics.MonitorInfo.Add(Display);

			if (Display.bIsPrimary)
			{
				OutDisplayMetrics.PrimaryDisplayWorkAreaRect = FPlatformRect(UsableBounds.x, UsableBounds.y, UsableBounds.x + UsableBounds.w, UsableBounds.y + UsableBounds.h);

				OutDisplayMetrics.PrimaryDisplayWidth = DisplayBounds.w;
				OutDisplayMetrics.PrimaryDisplayHeight = DisplayBounds.h;

				// VirtualDisplayRect is the union of all monitors' full display bounds (matching
				// the Windows and Mac platform layers). The non-primary branch below accumulates
				// from DisplayBounds for the same reason; seed with the same rect here so the
				// single-monitor case stays consistent with the multi-monitor case.
				OutDisplayMetrics.VirtualDisplayRect = FPlatformRect(DisplayBounds.x, DisplayBounds.y, DisplayBounds.x + DisplayBounds.w, DisplayBounds.y + DisplayBounds.h);
			}
			else
			{
				// accumulate the total bound rect
				OutDisplayMetrics.VirtualDisplayRect.Left = FMath::Min(DisplayBounds.x, OutDisplayMetrics.VirtualDisplayRect.Left);
				OutDisplayMetrics.VirtualDisplayRect.Right = FMath::Max(OutDisplayMetrics.VirtualDisplayRect.Right, DisplayBounds.x + DisplayBounds.w);
				OutDisplayMetrics.VirtualDisplayRect.Top = FMath::Min(DisplayBounds.y, OutDisplayMetrics.VirtualDisplayRect.Top);
				OutDisplayMetrics.VirtualDisplayRect.Bottom = FMath::Max(OutDisplayMetrics.VirtualDisplayRect.Bottom, DisplayBounds.y + DisplayBounds.h);
			}
		}

		SDL_free(Displays);
	}

	// Apply the debug safe zones
	OutDisplayMetrics.ApplyDefaultSafeZones();
}

void FLinuxApplication::RemoveNotificationWindow(SDL_HWindow HWnd)
{
	for (int32 WindowIndex=0; WindowIndex < NotificationWindows.Num(); ++WindowIndex)
	{
		TSharedRef< FLinuxWindow > Window = NotificationWindows[ WindowIndex ];

		if ( Window->GetHWnd() == HWnd )
		{
			NotificationWindows.RemoveAt(WindowIndex);
			return;
		}
	}
}

void FLinuxApplication::RaiseNotificationWindows(const TSharedPtr< FLinuxWindow >& ParentWindow)
{
	// Raise notification windows above everything except for modal windows
	for (int32 WindowIndex=0; WindowIndex < NotificationWindows.Num(); ++WindowIndex)
	{
		TSharedRef< FLinuxWindow > NotificationWindow = NotificationWindows[WindowIndex];

		if(!ParentWindow->IsModalWindow())
		{
			SDL_RaiseWindow(NotificationWindow->GetHWnd());
		}
	}
}

void FLinuxApplication::RemoveRevertFocusWindow(SDL_HWindow HWnd)
{
	for (int32 WindowIndex=0; WindowIndex < RevertFocusStack.Num(); ++WindowIndex)
	{
		TSharedRef< FLinuxWindow > Window = RevertFocusStack[ WindowIndex ];

		if (Window->GetHWnd() == HWnd)
		{
			UE_LOGF(LogLinuxWindow, Verbose, "Found Window that is going to be destroyed. Going to revert focus ...");
			RevertFocusStack.RemoveAt(WindowIndex);

			if(Window->IsUtilityWindow() || Window->IsDialogWindow())
			{
				ActivateWindow(Window->GetParent());
			}
			// Was the deleted window a Blueprint, Cascade etc. window?
			else if (Window->IsNotificationWindow())
			{
				// Do not revert focus if the root window of the destroyed window is another one.
				TSharedPtr<FLinuxWindow> RevertFocusToWindow = Window->GetParent(); 
				TSharedPtr<FLinuxWindow> RootWindow = GetRootWindow(Window);
				UE_LOGF(LogLinuxWindow, Verbose, "CurrentlyActiveWindow: %d, RootParentWindow: %d ", 	CurrentlyActiveWindow.IsValid() ? CurrentlyActiveWindow->GetID() : -1,
																											RootWindow.IsValid() ? RootWindow->GetID() : -1);

				// Only do this if the destroyed window had a root and the currently active is neither itself nor the root window.
				// If the currently active window is not the root another window got active before we could destroy it. So we give the focus to the
				// currently active one and the currently active window shouldn't be the destructed one, if yes that means that no other window got active
				// so we can process normally.
				if(CurrentlyActiveWindow.IsValid() && RootWindow.IsValid() && (CurrentlyActiveWindow != RootWindow) && (CurrentlyActiveWindow != Window) )
				{
					UE_LOGF(LogLinuxWindow, Verbose, "Root Parent is different, going to set focus to CurrentlyActiveWindow: %d", CurrentlyActiveWindow.IsValid() ? CurrentlyActiveWindow->GetID() : -1);
					RevertFocusToWindow = CurrentlyActiveWindow;
				}

				ActivateWindow(RevertFocusToWindow);
			}
			// Was the deleted window a top level window and we have still at least one other window in the stack?
			else if (Window->IsTopLevelWindow() && (RevertFocusStack.Num() > 0))
			{
				// OK, give focus to the one on top of the stack.
				TSharedPtr< FLinuxWindow > TopmostWindow = RevertFocusStack.Top();
				if (TopmostWindow.IsValid())
				{
					ActivateWindow(TopmostWindow);
				}
			}
			// Was it a popup menu?
			else if (Window->IsPopupMenuWindow() && bActivateApp)
			{
				ActivateWindow(Window->GetParent());

				UE_LOGF(LogLinuxWindowType, Verbose, "FLinuxWindow::Destroy: Going to revert focus to %d", Window->GetParent()->GetID());
			}
			break;
		}
	}
}

void FLinuxApplication::ActivateApplication()
{
	MessageHandler->OnApplicationActivationChanged( true );
	bActivateApp = true;
	UE_LOGF(LogLinuxWindowEvent, Verbose, "WM_ACTIVATEAPP, wParam = 1");
}

void FLinuxApplication::DeactivateApplication()
{
	MessageHandler->OnApplicationActivationChanged( false );
	CurrentlyActiveWindow = nullptr;
	CurrentFocusWindow = nullptr;
	bActivateApp = false;
	UE_LOGF(LogLinuxWindowEvent, Verbose, "WM_ACTIVATEAPP, wParam = 0");
}

void FLinuxApplication::ActivateWindow(const TSharedPtr< FLinuxWindow >& Window)
{
	if (Window->GetActivationPolicy() == EWindowActivationPolicy::Never)
	{
		return;
	}

	PreviousActiveWindow = CurrentlyActiveWindow;
	CurrentlyActiveWindow = Window;
	if(PreviousActiveWindow.IsValid())
	{
		MessageHandler->OnWindowActivationChanged(PreviousActiveWindow.ToSharedRef(), EWindowActivation::Deactivate);
		UE_LOGF(LogLinuxWindowEvent, Verbose, "WM_ACTIVATE,    wParam = WA_INACTIVE     : %d", PreviousActiveWindow->GetID());
	}
	MessageHandler->OnWindowActivationChanged(CurrentlyActiveWindow.ToSharedRef(), EWindowActivation::Activate);
	UE_LOGF(LogLinuxWindowEvent, Verbose, "WM_ACTIVATE,    wParam = WA_ACTIVE       : %d", CurrentlyActiveWindow->GetID());
}

void FLinuxApplication::ActivateRootWindow(const TSharedPtr< FLinuxWindow >& Window)
{
	TSharedPtr< FLinuxWindow > ParentWindow = Window;
	while(ParentWindow.IsValid() && ParentWindow->GetParent().IsValid())
	{
		ParentWindow = ParentWindow->GetParent();
	}
	ActivateWindow(ParentWindow);
}

TSharedPtr< FLinuxWindow > FLinuxApplication::GetRootWindow(const TSharedPtr< FLinuxWindow >& Window)
{
	TSharedPtr< FLinuxWindow > ParentWindow = Window;
	while(ParentWindow->GetParent().IsValid())
	{
		ParentWindow = ParentWindow->GetParent();
	}
	return ParentWindow;
}

bool FLinuxApplication::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	// Ignore any execs that doesn't start with LinuxApp
	if (!FParse::Command(&Cmd, TEXT("LinuxApp")))
	{
		return false;
	}

	if (FParse::Command(&Cmd, TEXT("Cursor")))
	{
		return HandleCursorCommand(Cmd, Ar);
	}
	else if (FParse::Command(&Cmd, TEXT("Window")))
	{
		return HandleWindowCommand(Cmd, Ar);
	}

	return false;
}

bool FLinuxApplication::HandleCursorCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("Status")))
	{
		FLinuxCursor *LinuxCursor = static_cast<FLinuxCursor*>(Cursor.Get());
		FVector2D CurrentPosition = LinuxCursor->GetPosition();

		Ar.Logf(TEXT("Cursor status:"));
		Ar.Logf(TEXT("Position: (%f, %f)"), CurrentPosition.X, CurrentPosition.Y);
		Ar.Logf(TEXT("IsHidden: %s"), LinuxCursor->IsHidden() ? TEXT("true") : TEXT("false"));
		Ar.Logf(TEXT("bIsMouseCaptureEnabled: %s"), bIsMouseCaptureEnabled ? TEXT("true") : TEXT("false"));
		Ar.Logf(TEXT("bUsingHighPrecisionMouseInput: %s"), bUsingHighPrecisionMouseInput ? TEXT("true") : TEXT("false"));
		Ar.Logf(TEXT("bIsMouseCaptureEnabled: %s"), bIsMouseCaptureEnabled ? TEXT("true") : TEXT("false"));

		return true;
	}

	return false;
}

bool FLinuxApplication::HandleWindowCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("List")))
	{
		Ar.Logf(TEXT("Window list:"));
		for (int WindowIdx = 0, NumWindows = Windows.Num(); WindowIdx < NumWindows; ++WindowIdx)
		{
			Ar.Logf(TEXT("%d: native handle: %p, debugging ID: %d"), WindowIdx, Windows[WindowIdx]->GetHWnd(), Windows[WindowIdx]->GetID());
		}

		return true;
	}

	return false;
}

void FLinuxApplication::SaveWindowPropertiesForEventLoop(void)
{
	for (int32 WindowIndex = 0; WindowIndex < Windows.Num(); ++WindowIndex)
	{
		TSharedRef< FLinuxWindow > Window = Windows[WindowIndex];
		int X = 0;
		int Y = 0;
		int Width = 0;
		int Height = 0;
		SDL_HWindow NativeWindow = Window->GetHWnd();
		GetWindowScreenPosition(NativeWindow, X, Y);
		SDL_GetWindowSize(NativeWindow, &Width, &Height);

		FWindowProperties Props;
		Props.Location = FVector2D(static_cast<float>(X), static_cast<float>(Y));
		Props.Size = FVector2D(static_cast<float>(Width), static_cast<float>(Height));
		SavedWindowPropertiesForEventLoop.FindOrAdd(NativeWindow) = Props;
	}
}

void FLinuxApplication::ClearWindowPropertiesAfterEventLoop(void)
{
	SavedWindowPropertiesForEventLoop.Empty();

	// This is a hack for 4.22.1 to avoid changing a header. We will be calling in from here at
	// LinuxPlatformApplicationMisc.cpp in FLinuxPlatformApplicationMisc::PumpMessages
	//
	// In 4.23 this will be in a function: void FLinuxApplication::CheckIfApplicatioNeedsDeactivation()

	// If FocusOutDeactivationTime is set we have had a focus out event and are waiting to see if we need to Deactivate the Application
	if (!FMath::IsNearlyZero(FocusOutDeactivationTime))
	{
		// We still havent hit our timeout limit, keep waiting
		if (FocusOutDeactivationTime > FPlatformTime::Seconds())
		{
			return;
		}
		// If we don't use bIsDragWindowButtonPressed the draged window will be destroyed because we
		// deactivate the whole appliacton. TODO Is that a bug? Do we have to do something?
		else if (!CurrentFocusWindow.IsValid() && !bIsDragWindowButtonPressed)
		{
			DeactivateApplication();

			FocusOutDeactivationTime = 0.0;
		}
		else
		{
			FocusOutDeactivationTime = 0.0;
		}
	}
}

void FLinuxApplication::GetWindowPropertiesInEventLoop(SDL_HWindow NativeWindow, FWindowProperties& Properties)
{
	FWindowProperties *SavedProps = SavedWindowPropertiesForEventLoop.Find(NativeWindow);
	if(SavedProps)
	{
		Properties = *SavedProps;
	}
	else if(NativeWindow)
	{
		int X = 0;
		int Y = 0;
		int Width = 0;
		int Height = 0;
		GetWindowScreenPosition(NativeWindow, X, Y);
		SDL_GetWindowSize(NativeWindow, &Width, &Height);
		Properties.Location = FVector2D(static_cast<float>(X), static_cast<float>(Y));
		Properties.Size = FVector2D(static_cast<float>(Width), static_cast<float>(Height));

		// If we've hit this case, then we're either not in the event
		// loop, or suddenly have a new window to keep track of.
		// Record the initial window position.
		SavedWindowPropertiesForEventLoop.FindOrAdd(NativeWindow) = Properties;
	}
	else
	{
		UE_LOGF(LogLinuxWindowEvent, Error, "Tried to get the location of a non-existent window\n");
		Properties.Location = FVector2D(0, 0);
		Properties.Size = FVector2D(0, 0);
	}
}

bool FLinuxApplication::IsMouseAttached() const
{
	int rc;
	char Mouse[64] = "/sys/class/input/mouse0";
	int MouseIdx = strlen(Mouse) - 1;
	FCStringAnsi::StrncatTruncateDest(Mouse, UE_ARRAY_COUNT(Mouse), "/device/name");

	for (int i=0; i<9; i++)
	{
		Mouse[MouseIdx] = (char)('0' + i);
		if (access(Mouse, F_OK) == 0)
		{
			return true;
		}
	}

	return false;
}

void FLinuxApplication::SetForceFeedbackChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value)
{
	if (FApp::UseVRFocus() && !FApp::HasVRFocus())
	{
		return; // do not proceed if the app uses VR focus but doesn't have it
	}

	for (auto ControllerIt = ControllerStates.CreateIterator(); ControllerIt; ++ControllerIt)
	{
		auto& ControllerState = ControllerIt.Value();
		int32 ControllerIndex = UE::Input::ConvertInputDeviceToInt(ControllerState.DeviceId);
		if (ControllerIndex == ControllerId)
		{
			switch (ChannelType)
			{
			case FForceFeedbackChannelType::LEFT_LARGE:
				ControllerState.ForceFeedbackValues.LeftLarge = Value;
				break;
			case FForceFeedbackChannelType::LEFT_SMALL:
				ControllerState.ForceFeedbackValues.LeftSmall = Value;
				break;
			case FForceFeedbackChannelType::RIGHT_LARGE:
				ControllerState.ForceFeedbackValues.RightLarge = Value;
				break;
			case FForceFeedbackChannelType::RIGHT_SMALL:
				ControllerState.ForceFeedbackValues.RightSmall = Value;
				break;
			}
			break;
		}
	}

	// send vibration to externally-implemented devices
	for (auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt)
	{
		(*DeviceIt)->SetChannelValue(ControllerId, ChannelType, Value);
	}
}
void FLinuxApplication::SetForceFeedbackChannelValues(int32 ControllerId, const FForceFeedbackValues &Values)
{
	if (FApp::UseVRFocus() && !FApp::HasVRFocus())
	{
		return; // do not proceed if the app uses VR focus but doesn't have it
	}

	for (auto ControllerIt = ControllerStates.CreateIterator(); ControllerIt; ++ControllerIt)
	{
		auto& ControllerState = ControllerIt.Value();
		int32 ControllerIndex = UE::Input::ConvertInputDeviceToInt(ControllerState.DeviceId);
		if (ControllerIndex == ControllerId)
		{
			ControllerState.ForceFeedbackValues = Values;
			break;
		}
	}

	// send vibration to externally-implemented devices
	for (auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt)
	{
		if ((*DeviceIt)->IsGamepadAttached())
		{
			(*DeviceIt)->SetChannelValues(ControllerId, Values);
		}
	}
}

void FLinuxApplication::SetHapticFeedbackValues(int32 ControllerId, int32 Hand, const FHapticFeedbackValues& Values)
{
	if (FApp::UseVRFocus() && !FApp::HasVRFocus())
	{
		return; // do not proceed if the app uses VR focus but doesn't have it
	}
	for (auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt)
	{
		IHapticDevice* HapticDevice = (*DeviceIt)->GetHapticDevice();
		if (HapticDevice)
		{
			HapticDevice->SetHapticFeedbackValues(ControllerId, Hand, Values);
		}
	}
}

void FLinuxApplication::SDLControllerState::UpdateHapticEffect()
{
	if (Controller == nullptr)
	{
		return;
	}

	const float LargeValue = FMath::Max(ForceFeedbackValues.LeftLarge, ForceFeedbackValues.RightLarge);
	const float SmallValue = FMath::Max(ForceFeedbackValues.LeftSmall, ForceFeedbackValues.RightSmall);

	const uint16 LowFrequencyRumble  = (uint16)(FMath::Clamp(LargeValue, 0.0f, 1.0f) * 65535.0f);
	const uint16 HighFrequencyRumble = (uint16)(FMath::Clamp(SmallValue, 0.0f, 1.0f) * 65535.0f);

	if (!SDL_RumbleGamepad(Controller, LowFrequencyRumble, HighFrequencyRumble, 1000))
	{
		UE_LOGF(LogLinux, Warning, "SDL_RumbleGamepad failed: %s", SDL_GetError());
	}
}

void FLinuxApplication::AddGameController(SDL_JoystickID Id)
{
	if(ControllerStates.Contains(Id))
	{
		return;
	}

	auto Controller = SDL_OpenGamepad(Id);
	if (Controller == nullptr)
	{
		UE_LOGF(LogLinux, Warning, "Could not open gamecontroller %i: %ls", Id, UTF8_TO_TCHAR(SDL_GetError()) );
		return;
	}

	IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();

	uint32 UsedBits = 0;
	for (auto ControllerIt = ControllerStates.CreateIterator(); ControllerIt; ++ControllerIt)
	{
		FPlatformUserId UserId = DeviceMapper.GetUserForInputDevice(ControllerIt.Value().DeviceId);
		UsedBits |= (1 << UserId.GetInternalId());
	}

	int32 FirstUnusedIndex = FMath::CountTrailingZeros(~UsedBits);

	UE_LOGF(LogLinux, Verbose, "Adding controller %i '%ls'", FirstUnusedIndex, UTF8_TO_TCHAR(SDL_GetGamepadName(Controller)));
	auto& ControllerState = ControllerStates.Add(Id);
	ControllerState.Controller = Controller;

	const SDL_PropertiesID GamepadProps = SDL_GetGamepadProperties(Controller);
	const bool bHasRumble = SDL_GetBooleanProperty(GamepadProps, SDL_PROP_GAMEPAD_CAP_RUMBLE_BOOLEAN, false);
	const bool bHasTriggerRumble = SDL_GetBooleanProperty(GamepadProps, SDL_PROP_GAMEPAD_CAP_TRIGGER_RUMBLE_BOOLEAN, false);
	UE_LOGF(LogLinux, Log, "Gamepad %i '%s' caps: rumble=%d trigger_rumble=%d",
		FirstUnusedIndex, SDL_GetGamepadName(Controller),
		bHasRumble ? 1 : 0, bHasTriggerRumble ? 1 : 0);

	FPlatformUserId UserId = FPlatformUserId::CreateFromInternalId(FirstUnusedIndex);
	DeviceMapper.RemapControllerIdToPlatformUserAndDevice(FirstUnusedIndex, UserId, ControllerState.DeviceId);

	{
		static const FName LinuxGamepadInputDeviceName(TEXT("FLinuxApplication"));
		FInputDeviceDescriptor Desc;
		Desc.HardwareDeviceHandle     = ControllerState.DeviceId;
		Desc.InputDeviceName          = LinuxGamepadInputDeviceName;
		Desc.HardwareDeviceIdentifier = FName(UTF8_TO_TCHAR(SDL_GetGamepadName(Controller)));
		FInputDeviceRegistry::RegisterDevice(Desc);
	}
	DeviceMapper.Internal_MapInputDeviceToUser(ControllerState.DeviceId, UserId, EInputDeviceConnectionState::Connected);
}

void FLinuxApplication::RemoveGameController(SDL_JoystickID Id)
{
	if (!ControllerStates.Contains(Id))
	{
		return;
	}

	SDLControllerState& ControllerState = ControllerStates[Id];
	UE_LOGF(LogLinux, Verbose, "Removing controller %i '%ls'", ControllerState.DeviceId.GetId(), UTF8_TO_TCHAR(SDL_GetGamepadName(ControllerState.Controller)));

	SDL_CloseGamepad(ControllerState.Controller);

	// Use the device/user we recorded at AddGameController time.
	IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
	FInputDeviceId DeviceId = ControllerState.DeviceId;
	FPlatformUserId PlatformUserId = DeviceMapper.GetUserForInputDevice(DeviceId);

	ControllerStates.Remove(Id);

	DeviceMapper.Internal_MapInputDeviceToUser(DeviceId, PlatformUserId, EInputDeviceConnectionState::Disconnected);
}

void FLinuxApplication::OnInputDeviceModuleRegistered(const FName& Type, IModularFeature* ModularFeature)
{
	if (Type == IInputDeviceModule::GetModularFeatureName())
	{
		IInputDeviceModule* Module = static_cast<IInputDeviceModule*>(ModularFeature);

		TSharedPtr<IInputDevice> Device = Module->CreateInputDevice(MessageHandler);
		if (Device.IsValid())
		{
			UE_LOGF(LogInit, Log, "Adding external input plugin.");
			ExternalInputDevices.Add(MoveTemp(Device));
		}
	}
}

// These functions are duplicating how SDL3 works since it doesn't provide accessors to these values
const uint32 FLinuxApplication::GetSDLDoubleClickTime()
{
	if (SDLDoubleClickTime == 0)
	{
		const char *hint = SDL_GetHint(SDL_HINT_MOUSE_DOUBLE_CLICK_TIME);
		if (hint && *hint) 
		{
			int i = SDL_atoi(hint);

			SDLDoubleClickTime = SDL_max(0, i);
		}

		if (SDLDoubleClickTime == 0)
		{
			SDLDoubleClickTime = 500;   // Linux defaults to 500 ms
		}
	}
	return SDLDoubleClickTime;
}

const uint32 FLinuxApplication::GetSDLDoubleClickRadius()
{
	if (SDLDoubleClickRadius == 0)
	{
		const char *hint = SDL_GetHint(SDL_HINT_MOUSE_DOUBLE_CLICK_RADIUS);
		if (hint && *hint) 
		{
			int i = SDL_atoi(hint);

			SDLDoubleClickRadius = SDL_max(0, i);
		}

		if (SDLDoubleClickRadius == 0)
		{
			SDLDoubleClickRadius = 32;   // Linux defaults to 32 pixels
		}
	}

	return SDLDoubleClickRadius;
}
