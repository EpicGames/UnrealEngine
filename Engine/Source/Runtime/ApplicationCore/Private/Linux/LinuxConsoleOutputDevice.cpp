// Copyright Epic Games, Inc. All Rights Reserved.

#include "Linux/LinuxConsoleOutputDevice.h"
#include "Containers/StringConv.h"
#include "Logging/LogMacros.h"
#include "CoreGlobals.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/CoreDelegates.h"
#include "Misc/App.h"
#include "Linux/LinuxPlatformApplicationMisc.h"

#define CONSOLE_RED		"\x1b[31m"
#define CONSOLE_GREEN	"\x1b[32m"
#define CONSOLE_YELLOW	"\x1b[33m"
#define CONSOLE_BLUE	"\x1b[34m"
#define CONSOLE_NONE	"\x1b[0m"

FLinuxConsoleOutputDevice::FLinuxConsoleOutputDevice()
	: bOverrideColorSet(false),
	  bOutputtingToTerminal(isatty(STDOUT_FILENO)),
	  bIsWindowShown(true),
	  bIsStdoutSet(false)
{
	FString CommandLine = FCommandLine::Get();

	// If -nostdout is specified and not -stdout, default to not spewing log messages.
	// This is useful on apps like UnrealLightmass so we don't overwhelm console output
	//  with duplicate entries and all the UE_LOG messages.
	if (FParse::Param(*CommandLine, TEXT("nostdout")) &&
		!FParse::Param(*CommandLine, TEXT("stdout")))
	{
		bIsWindowShown = false;
	}

	// If -stdout is getting set we are doing a printf in LaunchEngineLoop.cpp, so lets avoid double printing here
	bIsStdoutSet = FParse::Param(*CommandLine, TEXT("stdout"));

	if (FParse::Param(*CommandLine, TEXT("SuppressConsoleOutputVerboseLogging")))
	{
		AllowedLogVerbosity = ELogVerbosity::Log;
	}
}

FLinuxConsoleOutputDevice::~FLinuxConsoleOutputDevice()
{
}

void FLinuxConsoleOutputDevice::Show(bool bShowWindow)
{
	bIsWindowShown = bShowWindow;
}

bool FLinuxConsoleOutputDevice::IsShown()
{
	return bIsWindowShown;
}

void FLinuxConsoleOutputDevice::Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category)
{
	if (!bIsWindowShown || bIsStdoutSet)
		return;

	if ((Verbosity & ELogVerbosity::VerbosityMask) > AllowedLogVerbosity)
		return;

	static bool bEntry=false;
	if (!GIsCriticalError || bEntry)
	{
		if (Verbosity == ELogVerbosity::SetColor)
		{
			printf("%s", TCHAR_TO_UTF8(Data));
		}
		else
		{
			const ANSICHAR *ConsoleColorStr = "";

			if (bOutputtingToTerminal && !bOverrideColorSet)
			{
				if (Verbosity == ELogVerbosity::Error)
				{
					ConsoleColorStr = CONSOLE_RED;
				}
				else if (Verbosity == ELogVerbosity::Warning)
				{
					ConsoleColorStr = CONSOLE_YELLOW;
				}
			}

			printf("%s%s%s\n",
				ConsoleColorStr,
				TCHAR_TO_UTF8(*FOutputDeviceHelper::FormatLogLine(Verbosity, Category, Data, GPrintLogTimes)),
				ConsoleColorStr[0] ? CONSOLE_NONE : "");
		}
	}
	else
	{
		bEntry=true;
#if !PLATFORM_EXCEPTIONS_DISABLED
		try
		{
#endif // !PLATFORM_EXCEPTIONS_DISABLED
			// Ignore errors to prevent infinite-recursive exception reporting.
			Serialize(Data, Verbosity, Category);
#if !PLATFORM_EXCEPTIONS_DISABLED
		}
		catch (...)
		{
		}
#endif // !PLATFORM_EXCEPTIONS_DISABLED
		bEntry = false;
	}
}

bool FLinuxConsoleOutputDevice::CanBeUsedOnAnyThread() const
{
	return true;
}

bool FLinuxConsoleOutputDevice::CanBeUsedOnPanicThread() const
{
	return true;
}
