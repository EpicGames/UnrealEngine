// Copyright Epic Games, Inc. All Rights Reserved.

#include "Linux/LinuxCursor.h"

#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Linux/LinuxApplication.h"
#include "Linux/LinuxPlatformApplicationMisc.h"

FLinuxCursor::FLinuxCursor()
	: 	bHidden(false)
	,	CachedGlobalXPosition(0.0f)
	,	CachedGlobalYPosition(0.0f)
	,	bPositionCacheIsValid(false)
{
	if (!FApp::CanEverRender())
	{
		// assume that non-rendering application will be fine with token cursor
		UE_LOGF(LogInit, Log, "Not creating cursor resources due to headless application.");
		return;
	}

	if (!FLinuxPlatformApplicationMisc::InitSDL()) //	will not initialize more than once
	{
		UE_LOGF(LogInit, Fatal, "FLinuxCursor::FLinuxCursor() : InitSDL() failed, cannot construct cursor.");
		// unreachable
		return;
	}

#if DO_CHECK
	uint32 InitializedSubsystems = SDL_WasInit(SDL_INIT_VIDEO);
	check(InitializedSubsystems & SDL_INIT_VIDEO);
#endif // DO_CHECK

	// Load up cursors that we'll be using
	for( int32 CurCursorIndex = 0; CurCursorIndex < EMouseCursor::TotalCursorCount; ++CurCursorIndex )
	{
		CursorHandles[ CurCursorIndex ] = NULL;
		CursorOverrideHandles[ CurCursorIndex ] = NULL;

		SDL_HCursor CursorHandle = NULL;
		switch( CurCursorIndex )
		{
		case EMouseCursor::None:
		case EMouseCursor::Custom:
			// The mouse cursor will not be visible when None is used
			break;

		case EMouseCursor::Default:
			CursorHandle = SDL_CreateSystemCursor( SDL_SYSTEM_CURSOR_DEFAULT );
			break;

		case EMouseCursor::TextEditBeam:
			CursorHandle = SDL_CreateSystemCursor( SDL_SYSTEM_CURSOR_TEXT );
			break;

		case EMouseCursor::ResizeLeftRight:
			CursorHandle = SDL_CreateSystemCursor( SDL_SYSTEM_CURSOR_EW_RESIZE );
			break;

		case EMouseCursor::ResizeUpDown:
			CursorHandle = SDL_CreateSystemCursor( SDL_SYSTEM_CURSOR_NS_RESIZE );
			break;

		case EMouseCursor::ResizeSouthEast:
			CursorHandle = SDL_CreateSystemCursor( SDL_SYSTEM_CURSOR_NWSE_RESIZE );
			break;

		case EMouseCursor::ResizeSouthWest:
			CursorHandle = SDL_CreateSystemCursor( SDL_SYSTEM_CURSOR_NESW_RESIZE );
			break;

		case EMouseCursor::CardinalCross:
			CursorHandle = SDL_CreateSystemCursor( SDL_SYSTEM_CURSOR_MOVE );
			break;

		case EMouseCursor::Crosshairs:
			CursorHandle = SDL_CreateSystemCursor( SDL_SYSTEM_CURSOR_CROSSHAIR );
			break;

		case EMouseCursor::Hand:
			CursorHandle = SDL_CreateSystemCursor( SDL_SYSTEM_CURSOR_POINTER );
			break;

		case EMouseCursor::GrabHand:
			{
				FString path(FPlatformProcess::BaseDir() / FPaths::EngineContentDir() / FString("Editor/Slate/Cursor/grabhand.bmp") );
				SDL_Surface *surf = SDL_LoadBMP(TCHAR_TO_ANSI(*path));
				CursorHandle = SDL_CreateColorCursor(surf, 15, 15);
				SDL_DestroySurface(surf);
			}
			break;

		case EMouseCursor::GrabHandClosed:
			{
				FString path(FPlatformProcess::BaseDir() / FPaths::EngineContentDir() / FString("Editor/Slate/Cursor/grabhand_closed.bmp") );
				SDL_Surface *surf = SDL_LoadBMP(TCHAR_TO_ANSI(*path));
				CursorHandle = SDL_CreateColorCursor(surf, 8, 8);
				SDL_DestroySurface(surf);
			}
			break;

		case EMouseCursor::SlashedCircle:
			CursorHandle = SDL_CreateSystemCursor( SDL_SYSTEM_CURSOR_NOT_ALLOWED );
			break;

		case EMouseCursor::EyeDropper:
			{
				Uint8 mask[32]={0x00, 0x07, 0x00, 0x0f, 0x00, 0x5f, 0x00, 0xfe,
								0x01, 0xfc, 0x00, 0xf8, 0x01, 0xfc, 0x02, 0xf8,
								0x07, 0xd0, 0x0f, 0x80, 0x1f, 0x00, 0x3e, 0x00,
								0x7c, 0x00, 0x78, 0x00, 0xf0, 0x00, 0x40, 0x00};

				Uint8 data[32]={0x00, 0x07, 0x00, 0x0b, 0x00, 0x53, 0x00, 0xa6,
								0x01, 0x0c, 0x00, 0xf8, 0x01, 0x7c, 0x02, 0x38,
								0x04, 0x50, 0x08, 0x80, 0x11, 0x00, 0x22, 0x00,
								0x44, 0x00, 0x48, 0x00, 0xb0, 0x00, 0x40, 0x00};

				CursorHandle = SDL_CreateCursor(data, mask, 16, 16, 0, 15);
			}
			break;

			// NOTE: For custom app cursors, use:
			//		CursorHandle = ::LoadCursor( InstanceHandle, (LPCWSTR)MY_RESOURCE_ID );

		default:
			// Unrecognized cursor type!
			check( 0 );
			break;
		}

		CursorHandles[ CurCursorIndex ] = CursorHandle;
	}

	// Set the default cursor
	SetType( EMouseCursor::Default );
}

FLinuxCursor::~FLinuxCursor()
{
	// Release cursors
	// NOTE: Shared cursors will automatically be destroyed when the application is destroyed.
	//       For dynamically created cursors, use DestroyCursor
	for( int32 CurCursorIndex = 0; CurCursorIndex < EMouseCursor::TotalCursorCount; ++CurCursorIndex )
	{
		switch( CurCursorIndex )
		{
			case EMouseCursor::None:
			case EMouseCursor::Default:
			case EMouseCursor::TextEditBeam:
			case EMouseCursor::ResizeLeftRight:
			case EMouseCursor::ResizeUpDown:
			case EMouseCursor::ResizeSouthEast:
			case EMouseCursor::ResizeSouthWest:
			case EMouseCursor::CardinalCross:
			case EMouseCursor::Crosshairs:
			case EMouseCursor::Hand:
			case EMouseCursor::SlashedCircle:
			case EMouseCursor::Custom:
				// Standard shared cursors don't need to be destroyed
				break;
			case EMouseCursor::GrabHand:
			case EMouseCursor::GrabHandClosed:
			case EMouseCursor::EyeDropper:
				SDL_DestroyCursor(CursorHandles[CurCursorIndex]);
				break;

			default:
				// Unrecognized cursor type!
				check( 0 );
				break;
		}
	}
}

void* FLinuxCursor::CreateCursorFromRGBABuffer(const FColor* Pixels, int32 Width, int32 Height, FVector2D InHotSpot)
{
	uint32 Rmask = 0x000000ff;
	uint32 Gmask = 0x0000ff00;
	uint32 Bmask = 0x00ff0000;
	uint32 Amask = 0xff000000;

	SDL_Surface* Surface = SDL_CreateSurfaceFrom(Width, Height, SDL_GetPixelFormatForMasks(32, Rmask, Gmask, Bmask, Amask),(void*) Pixels, Width*4);
	if (Surface)
	{
		SDL_HCursor CursorHandle = SDL_CreateColorCursor(Surface, FMath::RoundToInt(InHotSpot.X * Width), FMath::RoundToInt(InHotSpot.Y * Height));

		SDL_DestroySurface(Surface);

		return CursorHandle;
	}

	return nullptr;
}

void FLinuxCursor::SetCustomShape( SDL_HCursor CursorHandle )
{
	CursorHandles[EMouseCursor::Custom] = CursorHandle;
}

FVector2D FLinuxCursor::GetPosition() const
{
	if (FApp::CanEverRender() && !bPositionCacheIsValid)
	{
		SDL_GetGlobalMouseState(&CachedGlobalXPosition, &CachedGlobalYPosition);
		bPositionCacheIsValid = true;
	}

	float CursorX = CachedGlobalXPosition;
	float CursorY = CachedGlobalYPosition;

	return FVector2D( CursorX, CursorY );
}

void FLinuxCursor::InvalidateCaches()
{
	bPositionCacheIsValid = false;
}

void FLinuxCursor::SetCachedPosition( const float X, const float Y )
{
	CachedGlobalXPosition = X;
	CachedGlobalYPosition = Y;
	bPositionCacheIsValid = true;
}

void FLinuxCursor::SetPosition( const int32 X, const int32 Y )
{
	// according to reports on IRC SDL_WarpMouseGlobal() doesn't work on some WMs so use InWindow unless we don't have a window.
	SDL_HWindow WndFocus = SDL_GetMouseFocus();

	if (WndFocus)
	{
		int WndX, WndY;

		// Get top-left.
		if(LinuxApplication)
		{
			FLinuxApplication::FWindowProperties Props;
			LinuxApplication->GetWindowPropertiesInEventLoop(WndFocus, Props);
			WndX = static_cast<int>(Props.Location.X);
			WndY = static_cast<int>(Props.Location.Y);
		}
		else
		{
			SDL_GetWindowPosition(WndFocus, &WndX, &WndY);
		}

		SDL_WarpMouseInWindow(WndFocus, (float)(X - WndX), (float)(Y - WndY));
	}
	else
	{
		SDL_WarpMouseGlobal((float)X, (float)Y);
	}

	SetCachedPosition((float)X, (float)Y);
}

void FLinuxCursor::SetType(const EMouseCursor::Type InNewCursor)
{
	checkf(InNewCursor < EMouseCursor::TotalCursorCount, TEXT("Invalid cursor(%d) supplied"), InNewCursor);
	CurrentType = InNewCursor;

	SDL_HCursor CurrentCursor = CursorOverrideHandles[InNewCursor] ? CursorOverrideHandles[InNewCursor] : CursorHandles[InNewCursor];

	if (CurrentCursor == nullptr)
	{
		if (InNewCursor != EMouseCursor::Custom)
		{
			bHidden = true;
		}
		SDL_HideCursor();
		SDL_SetCursor(CursorHandles[0]);
	}
	else
	{
		bHidden = false;
		SDL_ShowCursor();
		SDL_SetCursor(CurrentCursor);
	}
}

void FLinuxCursor::GetSize( int32& Width, int32& Height ) const
{
	Width = 16;
	Height = 16;
}

void FLinuxCursor::Show( bool bShow )
{
	if( bShow )
	{
		// Show mouse cursor.
		bHidden = false;
		SDL_ShowCursor();
	}
	else
	{
		// Disable the cursor.
		bHidden = true;
		SDL_HideCursor();
	}
}

static SDL_Rect GetConfinementRect(const RECT* const Bounds, const TSharedPtr<FLinuxWindow>& CurrentFocusWindow)
{
	SDL_Rect ConfineRect;

	int Left = 0;
	int Top = 0;
	int Right = 0;
	int Bottom = 0;
	SDL_GetWindowBordersSize(CurrentFocusWindow->GetHWnd(), &Top, &Left, &Bottom, &Right);

	ConfineRect.x = Bounds->left + Left;
	ConfineRect.y = Bounds->top - Top;

	ConfineRect.w = Bounds->right - Bounds->left - 1;
	ConfineRect.h = Bounds->bottom - Bounds->top - 1;

	// TODO. For some reason the values from Bounds seem not to cover the SDL's window area.
	// The cursor exceeds the right and bottom border of the window. This can be troublesome
	// when the cursor is outside the window and the user clicks the mouse button even the
	// cursor is confined. To prevent that the workaround is to add -amount additionally to the 
	// values used before.
	ConfineRect.w -= FMath::CeilToInt(ConfineRect.w * 0.002);
	ConfineRect.h -= FMath::CeilToInt(ConfineRect.h * 0.002);

	return ConfineRect;
}

void FLinuxCursor::Lock( const RECT* const Bounds )
{
	if (!LinuxApplication)
	{
		return;
	}

	TSharedPtr< FLinuxWindow > CurrentFocusWindow = LinuxApplication->GetCurrentFocusWindow();
	if(!CurrentFocusWindow.IsValid())
	{
		return;
	}

	// Lock/Unlock the cursor
	if ( Bounds == nullptr )
	{
		SDL_SetWindowMouseRect(CurrentFocusWindow->GetHWnd(), nullptr);
	}
	else
	{
		CursorClipRect = GetConfinementRect(Bounds, CurrentFocusWindow);

		// We dont want to set a negative bounding region. If Top, Left, Bottom, Right are all 0
		if (CursorClipRect.x >= 0 && CursorClipRect.y >= 0 && CursorClipRect.w > 0 && CursorClipRect.h > 0)
		{
			SDL_SetWindowMouseRect(CurrentFocusWindow->GetHWnd(), &CursorClipRect);
		}
	}
}

bool FLinuxCursor::IsHidden()
{
	return bHidden;
}

void FLinuxCursor::SetTypeShape(EMouseCursor::Type InCursorType, void* InCursorHandle)
{
	checkf(InCursorType < EMouseCursor::TotalCursorCount, TEXT("Invalid cursor(%d) supplied"), (int)InCursorType);

	SDL_Cursor* CursorHandle = (SDL_Cursor*)InCursorHandle;
	CursorOverrideHandles[InCursorType] = CursorHandle;

	if (CurrentType == InCursorType)
	{
		SetType(CurrentType);
	}
}
