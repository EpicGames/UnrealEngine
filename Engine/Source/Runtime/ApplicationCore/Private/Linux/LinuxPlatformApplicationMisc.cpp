// Copyright Epic Games, Inc. All Rights Reserved.

#include "Linux/LinuxPlatformApplicationMisc.h"
#include "Null/NullPlatformApplicationMisc.h"
#include "Misc/CommandLine.h"
#include "Misc/App.h"
#include "HAL/ThreadHeartBeat.h"
#include "Modules/ModuleManager.h"
#include "Linux/LinuxConsoleOutputDevice.h"
#include "Unix/UnixApplicationErrorOutputDevice.h"
#include "Unix/UnixFeedbackContext.h"
#include "Linux/LinuxApplication.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeLock.h"
#include "HAL/PlatformTime.h"

THIRD_PARTY_INCLUDES_START
	#include <SDL3/SDL.h>
	#include <SDL3/SDL_vulkan.h>
THIRD_PARTY_INCLUDES_END

bool GInitializedSDL = false;

// Wall-clock backstop: if RHI refs are never released (e.g. a leaked surface), destroy the window
// anyway this long after the destroy request so it can't leak forever. Far longer than any real
// present drain -- the GPU is already idle by the time the RHI releases its ref.
static float GWindowDestroyBackstopSeconds = 5.0f;
static FAutoConsoleVariableRef CVarWindowDestroyBackstopSeconds(
	TEXT("r.Vulkan.WindowDestroyBackstopSeconds"),
	GWindowDestroyBackstopSeconds,
	TEXT("Time to wait before destroying a window that was never released by the RHI."),
	ECVF_RenderThreadSafe
);

namespace
{
	uint64 GWindowStyleSDL = SDL_WINDOW_VULKAN;

	FString GetHeadlessMessageBoxMessage(EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption, EAppReturnType::Type& Answer)
	{
		FString MessageSuffix;
		switch (MsgType)
		{
		case EAppMsgType::YesNo:
		case EAppMsgType::YesNoYesAllNoAll:
		case EAppMsgType::YesNoYesAll:
			Answer = EAppReturnType::No;
			MessageSuffix = FString("No is implied.");
			break;

		case EAppMsgType::OkCancel:
		case EAppMsgType::YesNoCancel:
		case EAppMsgType::CancelRetryContinue:
		case EAppMsgType::YesNoYesAllNoAllCancel:
			Answer = EAppReturnType::Cancel;
			MessageSuffix = FString("Cancel is implied.");
			break;
		}

		FString Message = UTF8_TO_TCHAR(SDL_GetError());
		if (Message != FString("No message system available"))
		{
			Message = FString::Printf(TEXT("MessageBox: %s: %s: %s: %s"), Caption, Text, *Message, *MessageSuffix);
		}
		else
		{
			Message = FString::Printf(TEXT("MessageBox: %s: %s: %s"), Caption, Text, *MessageSuffix);
		}
		return Message;
	}
}

extern CORE_API TFunction<void()> UngrabAllInputCallback;
extern CORE_API TFunction<EAppReturnType::Type(EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption)> MessageBoxExtCallback;

static bool IsSDLDummyDriver()
{
	char const* SdlVideoDriver = SDL_GetCurrentVideoDriver();
	return (SdlVideoDriver && !strcmp(SdlVideoDriver, "dummy"));
}

EAppReturnType::Type MessageBoxExtImpl(EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption)
{
	int NumberOfButtons = 0;

	// if multimedia cannot be initialized for messagebox, just fall back to default implementation
	if (!FPlatformApplicationMisc::InitSDL() || IsSDLDummyDriver()) //	will not initialize more than once
	{
		EAppReturnType::Type Answer = EAppReturnType::Type::Cancel;
		FString Message = GetHeadlessMessageBoxMessage(MsgType, Caption, Text, Answer);
		UE_LOGF(LogLinux, Warning, "%ls", *Message);
		return Answer;
	}


#if DO_CHECK
	uint32 InitializedSubsystems = SDL_WasInit(SDL_INIT_VIDEO);
	check(InitializedSubsystems & SDL_INIT_VIDEO);
#endif // DO_CHECK

	SDL_MessageBoxButtonData *Buttons = nullptr;

	switch (MsgType)
	{
	case EAppMsgType::Ok:
		NumberOfButtons = 1;
		Buttons = new SDL_MessageBoxButtonData[NumberOfButtons];
		Buttons[0].flags = SDL_MESSAGEBOX_BUTTON_RETURNKEY_DEFAULT;
		Buttons[0].text = "Ok";
		Buttons[0].buttonID = EAppReturnType::Ok;
		break;

	case EAppMsgType::YesNo:
		NumberOfButtons = 2;
		Buttons = new SDL_MessageBoxButtonData[NumberOfButtons];
		Buttons[0].flags = SDL_MESSAGEBOX_BUTTON_ESCAPEKEY_DEFAULT;
		Buttons[0].text = "Yes";
		Buttons[0].buttonID = EAppReturnType::Yes;
		Buttons[1].flags = SDL_MESSAGEBOX_BUTTON_ESCAPEKEY_DEFAULT;
		Buttons[1].text = "No";
		Buttons[1].buttonID = EAppReturnType::No;
		break;

	case EAppMsgType::OkCancel:
		NumberOfButtons = 2;
		Buttons = new SDL_MessageBoxButtonData[NumberOfButtons];
		Buttons[0].flags = SDL_MESSAGEBOX_BUTTON_ESCAPEKEY_DEFAULT;
		Buttons[0].text = "Ok";
		Buttons[0].buttonID = EAppReturnType::Ok;
		Buttons[1].flags = SDL_MESSAGEBOX_BUTTON_ESCAPEKEY_DEFAULT;
		Buttons[1].text = "Cancel";
		Buttons[1].buttonID = EAppReturnType::Cancel;
		break;

	case EAppMsgType::YesNoCancel:
		NumberOfButtons = 3;
		Buttons = new SDL_MessageBoxButtonData[NumberOfButtons];
		Buttons[0].flags = SDL_MESSAGEBOX_BUTTON_ESCAPEKEY_DEFAULT;
		Buttons[0].text = "Yes";
		Buttons[0].buttonID = EAppReturnType::Yes;
		Buttons[1].flags = SDL_MESSAGEBOX_BUTTON_ESCAPEKEY_DEFAULT;
		Buttons[1].text = "No";
		Buttons[1].buttonID = EAppReturnType::No;
		Buttons[2].flags = SDL_MESSAGEBOX_BUTTON_ESCAPEKEY_DEFAULT;
		Buttons[2].text = "Cancel";
		Buttons[2].buttonID = EAppReturnType::Cancel;
		break;

	case EAppMsgType::CancelRetryContinue:
		NumberOfButtons = 3;
		Buttons = new SDL_MessageBoxButtonData[NumberOfButtons];
		Buttons[0].flags = SDL_MESSAGEBOX_BUTTON_ESCAPEKEY_DEFAULT;
		Buttons[0].text = "Continue";
		Buttons[0].buttonID = EAppReturnType::Continue;
		Buttons[1].flags = SDL_MESSAGEBOX_BUTTON_ESCAPEKEY_DEFAULT;
		Buttons[1].text = "Retry";
		Buttons[1].buttonID = EAppReturnType::Retry;
		Buttons[2].flags = SDL_MESSAGEBOX_BUTTON_ESCAPEKEY_DEFAULT;
		Buttons[2].text = "Cancel";
		Buttons[2].buttonID = EAppReturnType::Cancel;
		break;

	case EAppMsgType::YesNoYesAllNoAll:
		NumberOfButtons = 4;
		Buttons = new SDL_MessageBoxButtonData[NumberOfButtons];
		Buttons[0].flags = SDL_MESSAGEBOX_BUTTON_ESCAPEKEY_DEFAULT;
		Buttons[0].text = "Yes";
		Buttons[0].buttonID = EAppReturnType::Yes;
		Buttons[1].flags = SDL_MESSAGEBOX_BUTTON_ESCAPEKEY_DEFAULT;
		Buttons[1].text = "No";
		Buttons[1].buttonID = EAppReturnType::No;
		Buttons[2].flags = SDL_MESSAGEBOX_BUTTON_ESCAPEKEY_DEFAULT;
		Buttons[2].text = "Yes to all";
		Buttons[2].buttonID = EAppReturnType::YesAll;
		Buttons[3].flags = SDL_MESSAGEBOX_BUTTON_ESCAPEKEY_DEFAULT;
		Buttons[3].text = "No to all";
		Buttons[3].buttonID = EAppReturnType::NoAll;
		break;

	case EAppMsgType::YesNoYesAllNoAllCancel:
		NumberOfButtons = 5;
		Buttons = new SDL_MessageBoxButtonData[NumberOfButtons];
		Buttons[0].flags = SDL_MESSAGEBOX_BUTTON_ESCAPEKEY_DEFAULT;
		Buttons[0].text = "Yes";
		Buttons[0].buttonID = EAppReturnType::Yes;
		Buttons[1].flags = SDL_MESSAGEBOX_BUTTON_ESCAPEKEY_DEFAULT;
		Buttons[1].text = "No";
		Buttons[1].buttonID = EAppReturnType::No;
		Buttons[2].flags = SDL_MESSAGEBOX_BUTTON_ESCAPEKEY_DEFAULT;
		Buttons[2].text = "Yes to all";
		Buttons[2].buttonID = EAppReturnType::YesAll;
		Buttons[3].flags = SDL_MESSAGEBOX_BUTTON_ESCAPEKEY_DEFAULT;
		Buttons[3].text = "No to all";
		Buttons[3].buttonID = EAppReturnType::NoAll;
		Buttons[4].flags = SDL_MESSAGEBOX_BUTTON_ESCAPEKEY_DEFAULT;
		Buttons[4].text = "Cancel";
		Buttons[4].buttonID = EAppReturnType::Cancel;
		break;

	case EAppMsgType::YesNoYesAll:
		NumberOfButtons = 3;
		Buttons = new SDL_MessageBoxButtonData[NumberOfButtons];
		Buttons[0].flags = SDL_MESSAGEBOX_BUTTON_ESCAPEKEY_DEFAULT;
		Buttons[0].text = "Yes";
		Buttons[0].buttonID = EAppReturnType::Yes;
		Buttons[1].flags = SDL_MESSAGEBOX_BUTTON_ESCAPEKEY_DEFAULT;
		Buttons[1].text = "No";
		Buttons[1].buttonID = EAppReturnType::No;
		Buttons[2].flags = SDL_MESSAGEBOX_BUTTON_ESCAPEKEY_DEFAULT;
		Buttons[2].text = "Yes to all";
		Buttons[2].buttonID = EAppReturnType::YesAll;
		break;
	}

	FTCHARToUTF8 CaptionUTF8(Caption);
	FTCHARToUTF8 TextUTF8(Text);
	SDL_MessageBoxData MessageBoxData =
	{
		SDL_MESSAGEBOX_INFORMATION,
		NULL, // No parent window
		CaptionUTF8.Get(),
		TextUTF8.Get(),
		NumberOfButtons,
		Buttons,
		NULL // Default color scheme
	};

	int ButtonPressed = -1;
	EAppReturnType::Type Answer = EAppReturnType::Type::Cancel;

	FSlowHeartBeatScope SuspendHeartBeat;
	if (!SDL_ShowMessageBox(&MessageBoxData, &ButtonPressed))
	{
		FString Message = GetHeadlessMessageBoxMessage(MsgType, Caption, Text, Answer);
		UE_LOGF(LogLinux, Warning, "%ls", *Message);
	}
	else
	{
		Answer = ButtonPressed == -1 ? EAppReturnType::Cancel : static_cast<EAppReturnType::Type>(ButtonPressed);
	}

	delete[] Buttons;

	return Answer;
}

void UngrabAllInputImpl()
{
	if (GInitializedSDL)
	{
		SDL_Window* GrabbedWindow = SDL_GetGrabbedWindow();
		if (GrabbedWindow)
		{
			SDL_SetWindowMouseGrab(GrabbedWindow, false);
			SDL_SetWindowKeyboardGrab(GrabbedWindow, false);
		}

		SDL_Window* MouseFocusedWindow = SDL_GetMouseFocus();
		if (MouseFocusedWindow)
		{
			SDL_SetWindowMouseRect(MouseFocusedWindow, nullptr);
		}

		SDL_SetWindowRelativeMouseMode(GrabbedWindow, false);
		SDL_ShowCursor();
		SDL_CaptureMouse(false);
	}
}

uint64 FLinuxPlatformApplicationMisc::WindowStyle()
{
	return GWindowStyleSDL;
}

void FLinuxPlatformApplicationMisc::PreInit()
{
	MessageBoxExtCallback = MessageBoxExtImpl;
	FApp::SetHasFocusFunction(&FLinuxPlatformApplicationMisc::IsThisApplicationForeground);
}

void FLinuxPlatformApplicationMisc::Init()
{
	// skip for servers and programs, unless they request later
	bool bIsNullRHI = !FApp::CanEverRender();
	if (!IS_PROGRAM && !bIsNullRHI)
	{
		InitSDL();
	}

	FGenericPlatformApplicationMisc::Init();

	UngrabAllInputCallback = UngrabAllInputImpl;
}

DEFINE_LOG_CATEGORY_STATIC( LogSDL3, Log, All );

static void SDLCALL SDLLogOutputFunction(void *userdata, int category, SDL_LogPriority priority, const char *message)
{
	UE_LOGF(LogSDL3, Log, "%ls", UTF8_TO_TCHAR(message));
}

static int UEX11ErrorHandler(const char *errortext, int error_code, int request_code, int minor_code, void *userdata)
{
	UE_LOGF(LogInit, Warning, "X11 Error: %ls.  Error Code: %d.  Request Code: %d.  Minor Code: %d.",
		UTF8_TO_TCHAR(errortext),
		error_code,
		request_code,
		minor_code);

	return 0;
}

static int UEX11IOErrorHandler(void *userdata)
{
	UE_LOGF(LogInit, Error, "Fatal X11 IO Error: Lost connection to X Server");
	ensure(false);   // send a crash report if we can
	FLinuxPlatformMisc::RequestExit(true);

	return 0;
}

bool FLinuxPlatformApplicationMisc::InitSDL()
{
	if (!GInitializedSDL)
	{
		UE_LOGF(LogInit, Log, "Initializing SDL.");

#if !UE_BUILD_SHIPPING || USE_LOGGING_IN_SHIPPING
		SDL_SetLogOutputFunction(SDLLogOutputFunction, NULL);
#else
		SDL_SetLogOutputFunction(NULL, NULL);
#endif

		// In debug mode, let's log all the SDL warnings + errors, etc.
#if UE_BUILD_DEBUG		
		SDL_SetLogPriorities(SDL_LOG_PRIORITY_VERBOSE);
#endif



		// pass the string as is (SDL will parse)
		FString EglDeviceHint;
		if (FParse::Value(FCommandLine::Get(), TEXT("-egldevice="), EglDeviceHint))
		{
			UE_LOGF(LogInit, Log, "Hinting SDL to choose EGL device '%ls'", *EglDeviceHint);
			SDL_SetHint("SDL_HINT_EGL_DEVICE", TCHAR_TO_UTF8(*EglDeviceHint));
		}

		// Initially-unfocused windows should not ignore clicks.
		SDL_SetHint(SDL_HINT_MOUSE_FOCUS_CLICKTHROUGH, "1");

		// Unreal has support for touches, so this is unnecessary, and on some X11 implementations causes a mismatched number of mouse down/up events
		// which confuses Slate, preventing touch-based clicks from registering.
		SDL_SetHint(SDL_HINT_TOUCH_MOUSE_EVENTS, "0"); 

		// The following hints are needed when FLinuxApplication::SetHighPrecisionMouseMode is called and Enable = true.
		// SDL_SetRelativeMouseMode when enabled is warping the mouse in default mode but we don't want that. 
		// Furthermore SDL hides the mouse which we prevent by setting SDL_HINT_MOUSE_RELATIVE_CURSOR_VISIBLE
		SDL_SetHint(SDL_HINT_MOUSE_RELATIVE_CURSOR_VISIBLE, "1"); // When relative mouse mode is active, don't hide cursor.

		// Check for SDL video driver override. Priority (highest to lowest):
		// 1. -RenderOffScreen (forces dummy driver)
		// 2. -sdlvideodriver= command line (ALWAYS wins, even over env vars)
		// 3. SDL_VIDEO_DRIVER / SDL_VIDEODRIVER environment variable
		// 4. [Linux.SDL] VideoDriver= INI setting
		// 5. SDL default (auto-detect)
		{
			FString VideoDriverOverride;
			if (FParse::Param(FCommandLine::Get(), TEXT("RenderOffScreen")))
			{
				UE_LOGF(LogInit, Log, "Hinting SDL to use 'dummy' video driver for offscreen rendering.");
				SDL_SetHint(SDL_HINT_VIDEO_DRIVER, "dummy");
			}
			else if (FParse::Value(FCommandLine::Get(), TEXT("-sdlvideodriver="), VideoDriverOverride))
			{
				// Command line always wins -- use SDL_HINT_OVERRIDE to override even env vars.
				UE_LOGF(LogInit, Log, "Command line override: SDL video driver set to '%ls'", *VideoDriverOverride);
				SDL_SetHintWithPriority(SDL_HINT_VIDEO_DRIVER, TCHAR_TO_UTF8(*VideoDriverOverride), SDL_HINT_OVERRIDE);
			}
			else if (!(getenv("SDL_VIDEO_DRIVER") || getenv("SDL_VIDEODRIVER")))
			{
				// No command line override and no env var -- check INI.
				if (GConfig)
				{
					FString VideoDriverConfig;
					if (GConfig->GetString(TEXT("Linux.SDL"), TEXT("VideoDriver"), VideoDriverConfig, GEngineIni)
						&& !VideoDriverConfig.IsEmpty())
					{
						UE_LOGF(LogInit, Log, "INI override: SDL video driver set to '%ls'", *VideoDriverConfig);
						SDL_SetHint(SDL_HINT_VIDEO_DRIVER, TCHAR_TO_UTF8(*VideoDriverConfig));
					}
				}
			}
		}

		// For editor builds, prevent SDL from requesting the X11 window manager to bypass
		// compositing. Bypassing is good for fullscreen games (less latency) but breaks
		// desktop compositors like KDE Plasma, causing rendering glitches in the editor UI.
		if (GIsEditor && !getenv("SDL_VIDEO_X11_NET_WM_BYPASS_COMPOSITOR"))
		{
			SDL_SetHint(SDL_HINT_VIDEO_X11_NET_WM_BYPASS_COMPOSITOR, "0");
		}

		// do we need SDL_INIT_SENSOR or SDL_INIT_CAMERA here?
		if (!SDL_Init(SDL_INIT_VIDEO | SDL_INIT_GAMEPAD | SDL_INIT_EVENTS | SDL_INIT_SENSOR | SDL_INIT_CAMERA | SDL_INIT_JOYSTICK ))
		{
			FString ErrorMessage = UTF8_TO_TCHAR(SDL_GetError());
			if(ErrorMessage != FString("No message system available"))
			{
				// do not fail at this point, allow caller handle failure
				UE_LOGF(LogInit, Warning, "Could not initialize SDL: %ls", *ErrorMessage);
			}
			return false;
		}

		// Engine will control enabling and disabling the screensaver itself but SDL init always defaults the screen saver to disabled.
		// We are going to enable the screen saver here just so the logic the rest of engine uses works
		SDL_EnableScreenSaver();

		// print out version information
		int CompileTimeSDLVersion = SDL_VERSION;
		int RunTimeSDLVersion = SDL_GetVersion();
		FString SdlRevision = UTF8_TO_TCHAR(SDL_GetRevision());
		UE_LOGF(LogInit, Log, "Initialized SDL %d.%d.%d revision: %ls (compiled against %d.%d.%d)",
			SDL_VERSIONNUM_MAJOR(RunTimeSDLVersion), SDL_VERSIONNUM_MINOR(RunTimeSDLVersion), SDL_VERSIONNUM_MICRO(RunTimeSDLVersion),
			*SdlRevision,
			SDL_VERSIONNUM_MAJOR(CompileTimeSDLVersion), SDL_VERSIONNUM_MINOR(CompileTimeSDLVersion), SDL_VERSIONNUM_MICRO(CompileTimeSDLVersion)
			);

		char const* SdlVideoDriver = SDL_GetCurrentVideoDriver();
		if (SdlVideoDriver)
		{
			UE_LOGF(LogInit, Log, "Using SDL video driver '%ls'",
				UTF8_TO_TCHAR(SdlVideoDriver)
			);

			if (!strcmp(SdlVideoDriver, "x11"))
			{
				// Set up an error handler for X11 so we don't get the default version which just calls exit()
				SDL_SetX11ErrorHandler(UEX11ErrorHandler, nullptr, UEX11IOErrorHandler, nullptr);
			}
		}


		GInitializedSDL = true;

		// needs to come after GInitializedSDL, otherwise it will recurse here
		if (!UE_BUILD_SHIPPING)
		{
			// dump information about screens for debug
			FDisplayMetrics DisplayMetrics;
			FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);
			DisplayMetrics.PrintToLog();
		}
	}
	return true;
}

bool FLinuxPlatformApplicationMisc::IsUsingWayland()
{
	static int IsUsingWayland = -1;

	// If we haven't initialized SDL yet, then we can't say if we are using Wayland
	if (!GInitializedSDL)
	{
		UE_LOGF(LogInit, Warning, "Querying IsUsingWayland before SDL is initialized");
		return false;
	}

	if (IsUsingWayland == -1)
	{
		char const* SdlVideoDriver = SDL_GetCurrentVideoDriver();
		if (SdlVideoDriver)
		{
			IsUsingWayland = !strcmp(SdlVideoDriver, "wayland");
		}
		else
		{
			UE_LOGF(LogInit, Warning, "SDL is not returning a video driver while querying IsUsingWayland");	
			return false;		
		}
	}
	return IsUsingWayland;
}

bool FLinuxPlatformApplicationMisc::IsUsingX11()
{
	static int IsUsingX11 = -1;

	// If we haven't initialized SDL yet, then we can't say if we are using X11
	if (!GInitializedSDL)
	{
		UE_LOGF(LogInit, Warning, "Querying IsUsingX11 before SDL is initialized");
		return false;
	}

	if (IsUsingX11 == -1)
	{
		char const* SdlVideoDriver = SDL_GetCurrentVideoDriver();
		if (SdlVideoDriver)
		{
			IsUsingX11 = !strcmp(SdlVideoDriver, "x11");
		}
		else
		{
			UE_LOGF(LogInit, Warning, "SDL is not returning a video driver while querying IsUsingX11");	
			return false;		
		}
	}
	return IsUsingX11;
}

bool FLinuxPlatformApplicationMisc::RequiresParentedNotifications()
{
	// Pure Wayland cannot position a top-level surface; parenting + the popup role lets Slate anchor toasts.
	return IsUsingWayland();
}

// Tracks RHI surface ownership of an SDL window so SDL_DestroyWindow can fire on an explicit
// RHI-release signal rather than a fixed frame delay. Keyed on SDL_GetWindowID (stable for the
// window's life, 0 after destroy) -- NOT the raw SDL_Window*, because SDL can reissue a freed
// pointer to a different window.
struct FRHIWindowRegistryEntry
{
	SDL_Window* Window = nullptr;       // raw ptr, valid while the window is alive
	int32 RefCount = 0;                 // live RHI surfaces backing this window
	bool bDestroyRequested = false;     // FLinuxWindow::Destroy() was called
	double DestroyRequestedTime = 0.0;  // FPlatformTime::Seconds() at request, for the backstop
};
static TMap<Uint32 /*SDL_WindowID*/, FRHIWindowRegistryEntry> GRHIWindowRegistry;
static FCriticalSection GRHIWindowRegistryCS;

struct FDeferredHideWindow
{
	SDL_Window* Window;
	int32 FramesRemaining;
};
static TArray<FDeferredHideWindow> GWindowsPendingHide;

void FLinuxPlatformApplicationMisc::TearDown()
{
	FGenericPlatformApplicationMisc::TearDown();

	if (GInitializedSDL)
	{
		UE_LOGF(LogInit, Log, "Tearing down SDL.");

		// SDL_DestroyWindow handles role + buffer teardown directly; pending hides are redundant.
		GWindowsPendingHide.Empty();

		// Drain destroys before SDL_Quit. SDL_Quit would clean these up itself, but on Wayland
		// that triggers a Mutter use-after-free in wl_client_destroy signal cleanup; explicit
		// destroys give an orderly protocol teardown instead. At shutdown we destroy every tracked
		// window regardless of refcount -- the ultimate backstop against a leaked RHI ref.
		{
			FScopeLock Lock(&GRHIWindowRegistryCS);
			for (auto& Pair : GRHIWindowRegistry)
		{
				UE_LOGF(LogInit, Log, "TearDown: Destroying tracked window %p (refs=%d)", Pair.Value.Window, Pair.Value.RefCount);
				SDL_DestroyWindow(Pair.Value.Window);
			}
			GRHIWindowRegistry.Empty();
		}

		if (IsUsingX11())
		{
			SDL_SetX11ErrorHandler(nullptr, nullptr, nullptr, nullptr);
		}
		SDL_Quit();
		GInitializedSDL = false;

		MessageBoxExtCallback = nullptr;
		UngrabAllInputCallback = nullptr;
	}
}


void FLinuxPlatformApplicationMisc::LoadPreInitModules()
{
#if WITH_EDITOR
	FModuleManager::Get().LoadModule(TEXT("OpenGLDrv"));
#endif // WITH_EDITOR
}

void FLinuxPlatformApplicationMisc::LoadStartupModules()
{
#if !IS_PROGRAM && !UE_SERVER
	FModuleManager::Get().LoadModule(TEXT("AudioMixerSDL"));	// added in Launch.Build.cs for non-server targets
	FModuleManager::Get().LoadModule(TEXT("HeadMountedDisplay"));
#endif // !IS_PROGRAM && !UE_SERVER

#if defined(WITH_STEAMCONTROLLER) && WITH_STEAMCONTROLLER
	FModuleManager::Get().LoadModule(TEXT("SteamController"));
#endif // WITH_STEAMCONTROLLER

#if WITH_EDITOR
	FModuleManager::Get().LoadModule(TEXT("SourceCodeAccess"));
#endif	//WITH_EDITOR
}

class FOutputDeviceConsole* FLinuxPlatformApplicationMisc::CreateConsoleOutputDevice()
{
	// this is a slightly different kind of singleton that gives ownership to the caller and should not be called more than once
	return new FLinuxConsoleOutputDevice();
}

class FOutputDeviceError* FLinuxPlatformApplicationMisc::GetErrorOutputDevice()
{
	static FUnixApplicationErrorOutputDevice Singleton;
	return &Singleton;
}

class FFeedbackContext* FLinuxPlatformApplicationMisc::GetFeedbackContext()
{
	static FUnixFeedbackContext Singleton;
	return &Singleton;
}

GenericApplication* FLinuxPlatformApplicationMisc::CreateApplication()
{
	if (FParse::Param(FCommandLine::Get(), TEXT("RenderOffScreen")))
	{
		return FNullPlatformApplicationMisc::CreateApplication();
	}

	return FLinuxApplication::CreateLinuxApplication();
}

bool FLinuxPlatformApplicationMisc::IsThisApplicationForeground()
{
	return (LinuxApplication != nullptr) ? LinuxApplication->IsForeground() : true;
}

uint32 FLinuxPlatformApplicationMisc::AcquireWindowForRHI(void* SdlWindow)
{
	SDL_Window* Window = static_cast<SDL_Window*>(SdlWindow);
	if (!Window)
	{
		return 0;
	}

	// The window is guaranteed alive here: the RHI only acquires right after creating a surface
	// for it, which happens long before the window can be destroyed. Capture the id now and hand
	// it back to the caller -- release will key on this id, never on the (later possibly stale or
	// reused) pointer.
	const Uint32 Id = SDL_GetWindowID(Window);
	if (Id == 0)
{
		UE_LOGF(LogLinuxWindow, Verbose, "AcquireWindowForRHI failed with stale Window pointer");
		return 0;
	}

	FScopeLock Lock(&GRHIWindowRegistryCS);
	FRHIWindowRegistryEntry& Entry = GRHIWindowRegistry.FindOrAdd(Id);
	Entry.Window = Window;
	++Entry.RefCount;
	UE_LOGF(LogLinuxWindow, Verbose, "AcquireWindowForRHI id=%u win=%p refs=%d", Id, Window, Entry.RefCount);
	return Id;
}

void FLinuxPlatformApplicationMisc::ReleaseWindowForRHI(uint32 WindowId)
{
	if (WindowId == 0)
	{
		return;
	}

	// Key strictly on the id captured at acquire time. We deliberately do NOT re-derive the id from
	// any pointer here: after the destroy backstop force-destroys a still-referenced window and drops
	// its entry, a late release could otherwise resolve a reused SDL_Window* to a *different* live
	// window's id and decrement the wrong refcount. SDL window ids are monotonic and never recycled,
	// so a stale release simply finds no entry and no-ops.
	FScopeLock Lock(&GRHIWindowRegistryCS);
	FRHIWindowRegistryEntry* Entry = GRHIWindowRegistry.Find(WindowId);
	if (Entry && Entry->RefCount > 0)
		{
		--Entry->RefCount;
		}
	UE_LOGF(LogLinuxWindow, Verbose, "ReleaseWindowForRHI id=%u refs=%d", WindowId, Entry ? Entry->RefCount : -1);
	// Do NOT destroy here -- SDL_DestroyWindow must run on the game thread in PumpMessages.
	}

void FLinuxPlatformApplicationMisc::DeferDestroyWindow(SDL_Window* Window)
	{
	if (!Window)
		{
			return;
		}

	// Destroy() runs on the game thread while the window is still alive, so the id is valid.
	const Uint32 Id = SDL_GetWindowID(Window);

	FScopeLock Lock(&GRHIWindowRegistryCS);
	FRHIWindowRegistryEntry& Entry = GRHIWindowRegistry.FindOrAdd(Id);
	Entry.Window = Window;
	Entry.bDestroyRequested = true;
	Entry.DestroyRequestedTime = FPlatformTime::Seconds();

	// Destroy supersedes hide: SDL_DestroyWindow tears down the role itself, and a stale hide
	// entry firing on a destroyed pointer is unsafe (SDL may have reissued the pointer).
	for (int32 i = GWindowsPendingHide.Num() - 1; i >= 0; --i)
	{
		if (GWindowsPendingHide[i].Window == Window)
		{
			GWindowsPendingHide.RemoveAt(i);
		}
	}
}

void FLinuxPlatformApplicationMisc::DeferHideWindow(SDL_Window* Window)
{
	// 3-frame defer drains the RHI thread before SDL_HideWindow destroys the xdg role.
	// Caller must set UE.window.present_enabled=false before queueing so IsWindowValid
	// gates RHI off immediately.
	// :todo: replace fixed delay with an explicit RHI sync point.

	// Dedupe -- Slate may call Hide() repeatedly during hover transitions.
	for (const FDeferredHideWindow& Pending : GWindowsPendingHide)
	{
		if (Pending.Window == Window)
		{
			return;
		}
	}
	GWindowsPendingHide.Add({Window, 3});
}

void FLinuxPlatformApplicationMisc::CancelDeferredHideWindow(SDL_Window* Window)
{
	// FLinuxWindow::Show() calls this to abort a pending hide for the rapid Hide/Show cycle.
	for (int32 i = GWindowsPendingHide.Num() - 1; i >= 0; --i)
	{
		if (GWindowsPendingHide[i].Window == Window)
		{
			GWindowsPendingHide.RemoveAt(i);
		}
	}
}

void FLinuxPlatformApplicationMisc::PumpMessages( bool bFromMainLoop )
{
	if (GInitializedSDL && bFromMainLoop)
	{
		// Tick the deferred-hide queue. Order matters: hide before destroy so a window
		// queued for both fires hide first while still alive.
		for (int32 i = GWindowsPendingHide.Num() - 1; i >= 0; --i)
		{
			if (--GWindowsPendingHide[i].FramesRemaining <= 0)
			{
				UE_LOGF(LogLinuxWindow, Verbose, "Deferred SDL_HideWindow(%p)", GWindowsPendingHide[i].Window);
				SDL_HideWindow(GWindowsPendingHide[i].Window);
				GWindowsPendingHide.RemoveAt(i);
			}
		}

		// Destroy windows whose RHI surface refs have all been released (the explicit sync point),
		// or whose backstop timeout has elapsed. Edge-triggered on the refcount/flag, NOT a
		// per-call decrement -- PumpMessages(true) is not reliably once-per-frame, which is exactly
		// why the old fixed-frame countdown was unsound.
		{
			TArray<SDL_Window*> DestroyedWindows;
		{
				FScopeLock Lock(&GRHIWindowRegistryCS);
				const double Now = FPlatformTime::Seconds();
				for (auto It = GRHIWindowRegistry.CreateIterator(); It; ++It)
				{
					FRHIWindowRegistryEntry& Entry = It.Value();
					if (!Entry.bDestroyRequested)
					{
						continue;
					}

					const bool bRefsClear = (Entry.RefCount <= 0);
					const bool bTimedOut = (Now - Entry.DestroyRequestedTime) >= GWindowDestroyBackstopSeconds;
					if (bRefsClear || bTimedOut)
					{
						UE_CLOGF(bTimedOut && !bRefsClear, LogLinuxWindow, Warning,
							"SDL_DestroyWindow(%p) firing on BACKSTOP timeout with %d RHI ref(s) still held -- possible leaked surface.",
							Entry.Window, Entry.RefCount);
						UE_LOGF(LogLinuxWindow, Verbose, "SDL_DestroyWindow(%p) (refs=%d, requested)", Entry.Window, Entry.RefCount);
						DestroyedWindows.Add(Entry.Window);
						It.RemoveCurrent();
					}
				}
			}
			// Destroy windows outside of the lock
			for (SDL_Window* SDLWindow : DestroyedWindows)
			{
				SDL_DestroyWindow(SDLWindow);
			}
		}
		if( LinuxApplication )
		{
			LinuxApplication->SaveWindowPropertiesForEventLoop();

			SDL_Event event;

			while (SDL_PollEvent(&event))
			{
				LinuxApplication->AddPendingEvent( event );
			}

			LinuxApplication->CheckIfApplicatioNeedsDeactivation();
			LinuxApplication->ClearWindowPropertiesAfterEventLoop();
		}
		else
		{
			// No application to send events to. Just flush out the
			// queue.
			SDL_Event event;
			while (SDL_PollEvent(&event))
			{
				// noop
			}
		}

		bool bHasFocus = FApp::HasFocus();

		// if its our window, allow sound, otherwise apply multiplier
		FApp::SetVolumeMultiplier( bHasFocus ? 1.0f : FApp::GetUnfocusedVolumeMultiplier() );
	}
}

bool FLinuxPlatformApplicationMisc::IsScreensaverEnabled()
{
	return SDL_ScreenSaverEnabled();
}

bool FLinuxPlatformApplicationMisc::ControlScreensaver(EScreenSaverAction Action)
{
	if (Action == FGenericPlatformApplicationMisc::EScreenSaverAction::Disable)
	{
		SDL_DisableScreenSaver();
	}
	else
	{
		SDL_EnableScreenSaver();
	}
	return true;
}

namespace LinuxPlatformApplicationMisc
{
	/**
	 * Round the scale to 0.5, 1, 1.5, etc (note - step coarser than 0.25 is needed because a lot of monitors are 107-108 DPI and not 96).
	 */
	float QuantizeScale(float Scale)
	{
		float NewScale = FMath::FloorToFloat((64.0f * Scale / 32.0f) + 0.5f) / 2.0f;
		return NewScale > 0.0f ? NewScale : 1.0f;
	}
}

float FLinuxPlatformApplicationMisc::GetDPIScaleFactorAtPoint(float X, float Y)
{
	SDL_Point Point = {(int)X, (int)Y};
	SDL_DisplayID DisplayID = SDL_GetDisplayForPoint(&Point);
	float Scale = LinuxPlatformApplicationMisc::QuantizeScale(SDL_GetDisplayContentScale(DisplayID));

	UE_LOGF(LogLinux, Verbose, "Scale at X=%f, Y=%f: %f (monitor=#%d)", X, Y, Scale, DisplayID);
	return Scale;

}

void FLinuxPlatformApplicationMisc::ClipboardCopy(const TCHAR* Str)
{
	if (SDL_WasInit(SDL_INIT_VIDEO) && !SDL_SetClipboardText(TCHAR_TO_UTF8(Str)))
	{
		UE_LOGF(LogInit, Warning, "Error copying clipboard contents: %ls\n", UTF8_TO_TCHAR(SDL_GetError()));
	}
}

void FLinuxPlatformApplicationMisc::ClipboardPaste(class FString& Result)
{
	char* ClipContent;
	ClipContent = SDL_GetClipboardText();

	if (!ClipContent)
	{
		UE_LOGF(LogInit, Fatal, "Error pasting clipboard contents: %ls\n", UTF8_TO_TCHAR(SDL_GetError()));
		// unreachable
		Result = TEXT("");
	}
	else
	{
		Result = FString(UTF8_TO_TCHAR(ClipContent));
	}
	SDL_free(ClipContent);
}

void FLinuxPlatformApplicationMisc::EarlyUnixInitialization(FString& OutCommandLine)
{
}

void FLinuxPlatformApplicationMisc::UsingVulkan()
{
	UE_LOGF(LogInit, Log, "Using SDL_WINDOW_VULKAN");
	GWindowStyleSDL = SDL_WINDOW_VULKAN;
}

void FLinuxPlatformApplicationMisc::UsingOpenGL()
{
	UE_LOGF(LogInit, Log, "Using SDL_WINDOW_OPENGL");
	GWindowStyleSDL = SDL_WINDOW_OPENGL;
}
