// Copyright Epic Games, Inc. All Rights Reserved.

#include "Linux/LinuxPlatformSplash.h"

#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Misc/EngineVersionBase.h"
#include "Containers/Array.h"
#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "Logging/LogMacros.h"
#include "CoreGlobals.h"
#include "Misc/Parse.h"
#include "Misc/FileHelper.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Internationalization/Text.h"
#include "Internationalization/Internationalization.h"
#include "Misc/App.h"
#include "Misc/EngineVersion.h"
#include "Misc/EngineBuildSettings.h"
#include "Modules/ModuleManager.h"
#include "HAL/PlatformApplicationMisc.h"

#include <SDL3/SDL.h>

#if WITH_EDITOR

#include "IImageWrapper.h"
#include "IImageWrapperModule.h"

#include "ft2build.h"
#include FT_FREETYPE_H

#endif // WITH_EDITOR

/**
 * Splash screen functions and static globals
 */

struct Rect
{
	Rect(const int32 t = 0,const int32 l = 0, const int32 r = 0, const int32 b = 0) : Top(t), Left(l), Right(r), Bottom(b) {}
	int32 Top;
	int32 Left;
	int32 Right;
	int32 Bottom;
};


class FLinuxSplashState
{
public:
	~FLinuxSplashState();
	bool InitSplashResources(const FText &AppName, const FString &SplashPath, const FString &IconPath);
	void SetSplashText(const SplashTextType::Type InType, const FText& InText);
	void Pump();

private:
	static SDL_Surface* LoadImage(const FString &InImagePath);
	void Redraw();

#if WITH_EDITOR
	void DrawCharacter(int32 penx, int32 peny, FT_GlyphSlot Glyph, int32 CurTypeIndex, float Red, float Green, float Blue);
	void RenderStrings();
	bool OpenFonts();

	FT_Library FontLibrary = nullptr;
	FT_Face FontSmall = nullptr;
	FT_Face FontNormal = nullptr;
	FT_Face FontLarge = nullptr;

	SDL_Renderer *SplashRenderer = nullptr;
	SDL_Texture *SplashTexture = nullptr;
	FText SplashText[ SplashTextType::NumTextTypes ];
	Rect SplashTextRects[ SplashTextType::NumTextTypes ];

	unsigned char *ScratchSpace = nullptr;
	bool bStringsChanged = false;
#endif // WITH_EDITOR

	SDL_Surface *SplashSurface = nullptr;
	SDL_Window *SplashWindow = nullptr;

	bool bNeedsRedraw = false;
};

static FLinuxSplashState *GSplashState = nullptr;


//---------------------------------------------------------
FLinuxSplashState::~FLinuxSplashState()
{
#if WITH_EDITOR
	if (SplashTexture)
	{
		SDL_DestroyTexture(SplashTexture);
	}

	if (SplashRenderer)
	{
		SDL_DestroyRenderer(SplashRenderer);
	}
#endif // WITH_EDITOR

	if (SplashSurface)
	{
		SDL_DestroySurface(SplashSurface);
	}

	if (SplashWindow)
	{
		SDL_DestroyWindow(SplashWindow);
	}

#if WITH_EDITOR
	if (ScratchSpace)
	{
		FMemory::Free(ScratchSpace);
	}

	if (FontSmall)
	{
		FT_Done_Face(FontSmall);
	}

	if (FontNormal)
	{
		FT_Done_Face(FontNormal);
	}

	if (FontLarge)
	{
		FT_Done_Face(FontLarge);
	}

	if (FontLibrary)
	{
		FT_Done_FreeType(FontLibrary);
	}
#endif // WITH_EDITOR

	// do not deinit SDL here
}

#if WITH_EDITOR

//---------------------------------------------------------
bool FLinuxSplashState::OpenFonts()
{
	if (FT_Init_FreeType(&FontLibrary))
	{
		UE_LOGF(LogHAL, Error, "*** Unable to initialize font library.");
		return false;
	}

	// small font face
	FString FontPath = FPaths::ConvertRelativePathToFull(FPaths::EngineContentDir() / TEXT("Slate/Fonts/Roboto-Light.ttf"));

	if (FT_New_Face(FontLibrary, TCHAR_TO_UTF8(*FontPath), 0, &FontSmall))
	{
		UE_LOGF(LogHAL, Error, "*** Unable to open small font face for splash screen.");
	}
	else
	{
		FT_Set_Pixel_Sizes(FontSmall, 0, 10);
	}

	// normal font face
	FontPath = FPaths::ConvertRelativePathToFull(FPaths::EngineContentDir() / TEXT("Slate/Fonts/Roboto-Regular.ttf"));

	if (FT_New_Face(FontLibrary, TCHAR_TO_UTF8(*FontPath), 0, &FontNormal))
	{
		UE_LOGF(LogHAL, Error, "*** Unable to open normal font face for splash screen.");
	}
	else
	{
		FT_Set_Pixel_Sizes(FontNormal, 0, 16);
	}

	// large font face
	FontPath = FPaths::ConvertRelativePathToFull(FPaths::EngineContentDir() / TEXT("Slate/Fonts/Roboto-Bold.ttf"));

	if (FT_New_Face(FontLibrary, TCHAR_TO_UTF8(*FontPath), 0, &FontLarge))
	{
		UE_LOGF(LogHAL, Error, "*** Unable to open large font face for splash screen.");
	}
	else
	{
		FT_Set_Pixel_Sizes(FontLarge, 0, 32);
	}

	return true;
}


//---------------------------------------------------------
void FLinuxSplashState::DrawCharacter(int32 PenX, int32 PenY, FT_GlyphSlot Glyph, int32 CurTypeIndex, float Red, float Green, float Blue)
{
	// drawing boundaries
	const int32 MinX = SplashTextRects[CurTypeIndex].Left;
	const int32 MaxX = SplashTextRects[CurTypeIndex].Right;
	const int32 MaxY = SplashTextRects[CurTypeIndex].Bottom;
	const int32 MinY = SplashTextRects[CurTypeIndex].Top;

	// glyph dimensions
	const int32 GlyphWidth = Glyph->bitmap.width;
	const int32 GlyphHeight = Glyph->bitmap.rows;
	const int32 GlyphPitch = Glyph->bitmap.pitch;

	unsigned char *Pixels = Glyph->bitmap.buffer;

	const int SplashWidth = SplashSurface->w;
	const SDL_PixelFormatDetails *PixelFormat = SDL_GetPixelFormatDetails(SplashSurface->format);
	const int SplashBPP = PixelFormat->bytes_per_pixel;

	// draw glyph raster to texture
	for (int GlyphY = 0; GlyphY < GlyphHeight; GlyphY++)
	{
		for (int GlyphX = 0; GlyphX < GlyphWidth; GlyphX++)
		{
			// find pixel position in splash image
			const int PosX = PenX + GlyphX + (Glyph->metrics.horiBearingX >> 6);
			const int PosY = PenY + GlyphY - (Glyph->metrics.horiBearingY >> 6);

			// make sure pixel is in drawing rectangle
			if (PosX < MinX || PosX >= MaxX || PosY < MinY || PosY >= MaxY)
				continue;

			// get index of pixel in glyph bitmap
			const int32 SourceIndex = (GlyphY * GlyphPitch) + GlyphX;
			int32 DestIndex = (PosY * SplashWidth + PosX) * SplashBPP;

			// write pixel
			const float Alpha = Pixels[SourceIndex] / 255.0f;

			ScratchSpace[DestIndex] = (uint8)(ScratchSpace[DestIndex]*(1.0 - Alpha) + Alpha*Red);
			DestIndex++;
			ScratchSpace[DestIndex] = (uint8)(ScratchSpace[DestIndex]*(1.0 - Alpha) + Alpha*Green);
			DestIndex++;
			ScratchSpace[DestIndex] = (uint8)(ScratchSpace[DestIndex]*(1.0 - Alpha) + Alpha*Blue);
		}
	}
}


//---------------------------------------------------------
void FLinuxSplashState::RenderStrings()
{
	FT_UInt LastGlyph = 0;
	FT_Vector Kerning;
	bool bRightJustify = false;
	float Red, Blue, Green; // font color

	if (!bStringsChanged)
	{
		return;
	}

	bStringsChanged = false;
	bNeedsRedraw = true;

	const int SplashWidth = SplashSurface->w;
	const int SplashHeight =  SplashSurface->h;
	const SDL_PixelFormatDetails *PixelFormat = SDL_GetPixelFormatDetails(SplashSurface->format);
	const int SplashBPP = PixelFormat->bytes_per_pixel;
	unsigned char *SplashPixels = reinterpret_cast<unsigned char *>(SplashSurface->pixels);

	// clear the rendering scratch pad.
	for (int y = 0; y < SplashHeight; ++y)
	{
		unsigned char *Src = SplashPixels + y * SplashSurface->pitch;
		unsigned char *Dst = ScratchSpace + y * SplashWidth * SplashBPP;

		FMemory::Memcpy(Dst, Src, SplashWidth * SplashBPP);
	}

	// draw each type of string
	for (int CurTypeIndex=0; CurTypeIndex<SplashTextType::NumTextTypes; CurTypeIndex++)
	{
		FT_Face Font = nullptr;

		// initial pen position
		uint32 PenX = SplashTextRects[ CurTypeIndex].Left;
		uint32 PenY = SplashTextRects[ CurTypeIndex].Bottom;
		Kerning.x = 0;
		bRightJustify = false;

		// Set font color and text position
		if (CurTypeIndex == SplashTextType::StartupProgress)
		{
			Red = Green = Blue = 200.0f;
			Font = FontSmall;
		}
		else if (CurTypeIndex == SplashTextType::VersionInfo1)
		{
			Red = Green = Blue = 240.0f;
			Font = FontSmall;
		}
		else if (CurTypeIndex == SplashTextType::GameName)
		{
			Red = Green = Blue = 240.0f;
			if (GIsEditor)
			{
				Font = FontNormal;
			}
			else
			{
				Font = FontLarge;
			}
		}
		else if (CurTypeIndex == SplashTextType::CopyrightInfo)
		{
			Red = Green = Blue = 160.0f;
			Font = FontSmall;
			PenX = SplashTextRects[ CurTypeIndex].Right;
			bRightJustify = true;
		}
		else
		{
			Red = Green = Blue = 160.0f;
			Font = FontSmall;
		}

		// sanity check: make sure we have a font loaded.
		if (!Font)
		{
			continue;
		}

		// adjust verticle pos to allow for descenders
		PenY += Font->descender >> 6;

		// convert strings to glyphs and place them in bitmap.
		{
			FString Text = SplashText[CurTypeIndex].ToString();

			for (int i=0; i<Text.Len(); i++)
			{
				FT_ULong CharacterCode;

				// fetch next glyph
				if (bRightJustify)
				{
					CharacterCode = (Uint32)(Text[Text.Len() - i - 1]);
				}
				else
				{
					CharacterCode = (Uint32)(Text[i]);
				}

				FT_UInt GlyphIndex = FT_Get_Char_Index(Font, CharacterCode);
				FT_Load_Glyph(Font, GlyphIndex, FT_LOAD_DEFAULT);

				if (Font->glyph->format != FT_GLYPH_FORMAT_BITMAP)
				{
					FT_Render_Glyph(Font->glyph, FT_RENDER_MODE_NORMAL);
				}

				// pen advance and kerning
				if (bRightJustify)
				{
					if (LastGlyph != 0)
					{
						FT_Get_Kerning(Font, GlyphIndex, LastGlyph, FT_KERNING_DEFAULT, &Kerning);
					}

					PenX -= (Font->glyph->metrics.horiAdvance - Kerning.x) >> 6;
				}
				else
				{
					if (LastGlyph != 0)
					{
						FT_Get_Kerning(Font, LastGlyph, GlyphIndex, FT_KERNING_DEFAULT, &Kerning);
					}
				}

				LastGlyph = GlyphIndex;

				// draw character
				DrawCharacter(PenX, PenY, Font->glyph, CurTypeIndex, Red, Green, Blue);

				if (!bRightJustify)
				{
					PenX += (Font->glyph->metrics.horiAdvance - Kerning.x) >> 6;
				}
			}
		}

		// store rendered text as texture
		SDL_UpdateTexture(SplashTexture, NULL, ScratchSpace, SplashWidth * SplashBPP);
	}
}

#endif // WITH_EDITOR


/**
 * @brief Helper function to load an image in any format.
 *
 * @param ImagePath an (absolute) path to the image
 *
 * @return Splash surface
 */
SDL_Surface* FLinuxSplashState::LoadImage(const FString &ImagePath)
{
#if WITH_EDITOR
	TArray<uint8> RawFileData;

	// Load the image buffer first (unless it's BMP)
	if (!ImagePath.EndsWith(TEXT("bmp"), ESearchCase::IgnoreCase) && FFileHelper::LoadFileToArray(RawFileData, *ImagePath))
	{
		auto& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(FName("ImageWrapper"));
		auto Format = ImageWrapperModule.DetectImageFormat(RawFileData.GetData(), RawFileData.Num());
		TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(Format);

		if (ImageWrapper.IsValid() && ImageWrapper->SetCompressed(RawFileData.GetData(), RawFileData.Num()))
		{
			TArray<uint8> RawData;
			if (ImageWrapper->GetRaw(ERGBFormat::BGRA, 8, RawData))
			{
				SDL_Surface *Surface = SDL_CreateSurface(ImageWrapper->GetWidth(), ImageWrapper->GetHeight(), SDL_PIXELFORMAT_BGRA32);
				if (Surface)
				{
					const int Width = Surface->w;
					const int Height =  Surface->h;
					const SDL_PixelFormatDetails *PixelFormat = SDL_GetPixelFormatDetails(Surface->format);
					const int BytesPerPixel = PixelFormat->bytes_per_pixel;
					FMemory::Memcpy(Surface->pixels, (void*)RawData.GetData(), Width * Height * BytesPerPixel);
					return Surface;
				}
			}
		}
	}
#endif // WITH_EDITOR

	// If for some reason the image cannot be loaded, use the default BMP function
	return SDL_LoadBMP(TCHAR_TO_UTF8(*ImagePath));
}


/** Helper function to init resources used by the splash window */
bool FLinuxSplashState::InitSplashResources(const FText &AppName, const FString &SplashPath, const FString &IconPath)
{
	checkf(SplashWindow == nullptr, TEXT("FLinuxSplashState::InitSplashResources() has been called multiple times."));

	if (!FPlatformApplicationMisc::InitSDL()) //	will not initialize more than once
	{
		UE_LOGF(LogInit, Warning, "FLinuxSplashState::InitSplashResources() : InitSDL() failed, there will be no splash.");
		return false;
	}

	// load splash .bmp image
	SplashSurface = LoadImage(SplashPath);
	if (SplashSurface == nullptr)
	{
		UE_LOGF(LogHAL, Warning, "FLinuxSplashState::InitSplashResources() : Could not load splash BMP! SDL_Error: %ls", UTF8_TO_TCHAR(SDL_GetError()));
		return false;
	}

	const int SplashWidth = SplashSurface->w;
	const int SplashHeight =  SplashSurface->h;
	const SDL_PixelFormatDetails *PixelFormat = SDL_GetPixelFormatDetails(SplashSurface->format);
	const int SplashBPP = PixelFormat->bytes_per_pixel;

	if (SplashWidth <= 0 || SplashHeight <= 0)
	{
		UE_LOGF(LogHAL, Warning, "Invalid splash image dimensions.");
		return false;
	}

	// on modern X11, your windows might turn gray if they don't pump the event queue fast enough.
	// But this is because they opt-in to an optional window manager protocol by default; legacy
	// apps and those that know they'll be slow to respond to events--like splash screens--can
	// just choose to not support the protocol. Since we're a splash screen, it doesn't matter if
	// we would be unresponsive, since we accept no input. So don't opt-in.
	// We pump the event queue during SetSplashText, but it might be at a slow rate.
	const char *_OriginalHint = SDL_GetHint(SDL_HINT_VIDEO_X11_NET_WM_PING);  // (SDL_SetHint will free this pointer.)
	char *OriginalHint = _OriginalHint ? SDL_strdup(_OriginalHint) : NULL;
	SDL_SetHint(SDL_HINT_VIDEO_X11_NET_WM_PING, "0");
	SDL_PropertiesID Props = SDL_CreateProperties();
	if (Props == 0)
	{
		if (OriginalHint)
		{
			SDL_free(OriginalHint);
		}
		UE_LOGF(LogHAL, Error, "Failed to create SDL properties");
		return false;
	}

	SDL_DisplayID DisplayID = SDL_GetPrimaryDisplay();
	float SplashScale = SDL_GetDisplayContentScale(DisplayID);

	if (SplashScale == 0.0f)
	{
		SplashScale = 1.0f;
	}

	const int ScaledSplashWidth = (int)((float)SplashWidth * SplashScale);
	const int ScaledSplashHeight = (int)((float)SplashHeight * SplashScale);

	SDL_SetStringProperty(Props, SDL_PROP_WINDOW_CREATE_TITLE_STRING, TCHAR_TO_UTF8(*AppName.ToString()));
	SDL_SetNumberProperty(Props, SDL_PROP_WINDOW_CREATE_WIDTH_NUMBER, ScaledSplashWidth);
	SDL_SetNumberProperty(Props, SDL_PROP_WINDOW_CREATE_HEIGHT_NUMBER, ScaledSplashHeight);
	SDL_SetNumberProperty(Props, SDL_PROP_WINDOW_CREATE_X_NUMBER, SDL_WINDOWPOS_CENTERED_DISPLAY(0));
	SDL_SetNumberProperty(Props, SDL_PROP_WINDOW_CREATE_Y_NUMBER, SDL_WINDOWPOS_CENTERED_DISPLAY(0));
	SDL_SetBooleanProperty(Props, SDL_PROP_WINDOW_CREATE_BORDERLESS_BOOLEAN, true);
	SDL_SetBooleanProperty(Props, SDL_PROP_WINDOW_CREATE_HIDDEN_BOOLEAN, true);
	
	SplashWindow = SDL_CreateWindowWithProperties(Props);

	if (SplashWindow == nullptr)
	{
		UE_LOGF(LogHAL, Error, "FLinuxSplashState::InitSplashResources() : Splash screen window could not be created! SDL_Error: %ls", UTF8_TO_TCHAR(SDL_GetError()));
		SDL_SetHint(SDL_HINT_VIDEO_X11_NET_WM_PING, OriginalHint ? OriginalHint : "1");
		SDL_free(OriginalHint);
		return false;
	}

	if (!IconPath.IsEmpty())
	{
		SDL_Surface *SplashIconImage = LoadImage(IconPath);

		if (SplashIconImage == nullptr)
		{
			UE_LOGF(LogHAL, Warning, "FLinuxSplashState::InitSplashResources() : Splash icon could not be created! SDL_Error: %ls", UTF8_TO_TCHAR(SDL_GetError()));
		}
		else
		{
			SDL_SetWindowIcon(SplashWindow, SplashIconImage);  // SDL_SetWindowIcon() makes a copy of this.
			SDL_DestroySurface(SplashIconImage);
		}
	}

#if WITH_EDITOR
	SplashRenderer = SDL_CreateRenderer(SplashWindow, nullptr);
#endif

	// it's safe to set the hint back once the Renderer is created (since it might recreate the window to get a GL or whatever visual).
	SDL_SetHint(SDL_HINT_VIDEO_X11_NET_WM_PING, OriginalHint ? OriginalHint : "1");
	SDL_free(OriginalHint);

#if WITH_EDITOR

	if (SplashRenderer == nullptr)
	{
		UE_LOGF(LogHAL, Error, "FLinuxSplashState::InitSplashResources() : Splash screen renderer could not be created! SDL_Error: %ls", UTF8_TO_TCHAR(SDL_GetError()));
		return false;
	}

	SplashTexture = SDL_CreateTexture(SplashRenderer, SDL_GetPixelFormatDetails(SplashSurface->format)->format, SDL_TEXTUREACCESS_STATIC, SplashSurface->w, SplashSurface->h);
	if (SplashTexture == nullptr)
	{
		UE_LOGF(LogHAL, Error, "FLinuxSplashState::InitSplashResources() : Splash screen texture could not be created! SDL_Error: %ls", UTF8_TO_TCHAR(SDL_GetError()));
		return false;
	}

	// allocate scratch space for rendering text
	ScratchSpace = reinterpret_cast<unsigned char *>(FMemory::Malloc(SplashHeight*SplashWidth*SplashBPP));

	// Setup bounds for game name
	if (GIsEditor)
	{
		SplashTextRects[ SplashTextType::GameName ].Top = SplashHeight - 60;
		SplashTextRects[ SplashTextType::GameName ].Bottom = SplashHeight - 40;
	}
	else
	{
		SplashTextRects[ SplashTextType::GameName ].Top = SplashHeight - 60;
		SplashTextRects[ SplashTextType::GameName ].Bottom = SplashHeight - 20;
	}

	SplashTextRects[ SplashTextType::GameName ].Left = 10;
	SplashTextRects[ SplashTextType::GameName ].Right = SplashWidth - 10;

	// Setup bounds for version info text 1
	SplashTextRects[ SplashTextType::VersionInfo1 ].Top = SplashHeight - 35;
	SplashTextRects[ SplashTextType::VersionInfo1 ].Bottom = SplashHeight - 20;
	SplashTextRects[ SplashTextType::VersionInfo1 ].Left = 10;
	SplashTextRects[ SplashTextType::VersionInfo1 ].Right = SplashWidth - 10;

	// Setup bounds for copyright info text
	SplashTextRects[ SplashTextType::CopyrightInfo ].Top = SplashHeight - 16;
	SplashTextRects[ SplashTextType::CopyrightInfo ].Bottom = SplashHeight;
	SplashTextRects[ SplashTextType::CopyrightInfo ].Left = 0;
	SplashTextRects[ SplashTextType::CopyrightInfo ].Right = SplashWidth - 12;

	// Setup bounds for startup progress text
	SplashTextRects[ SplashTextType::StartupProgress ].Top = SplashHeight - 20;
	SplashTextRects[ SplashTextType::StartupProgress ].Bottom = SplashHeight;
	SplashTextRects[ SplashTextType::StartupProgress ].Left = 10;
	SplashTextRects[ SplashTextType::StartupProgress ].Right = SplashWidth - 20;

	OpenFonts();

	bStringsChanged = true;
	RenderStrings();
	SDL_ShowWindow(SplashWindow);
	Redraw();

#else

	SDL_Surface *WindowSurface = SDL_GetWindowSurface( SplashWindow );
	if (WindowSurface == nullptr)
	{
		UE_LOGF(LogHAL, Error, "FLinuxSplashState::InitSplashResources() : Splash window surface could not be created! SDL_Error: %ls", UTF8_TO_TCHAR(SDL_GetError()));
		return false;
	}

	SDL_BlitSurface(SplashSurface, NULL, WindowSurface, NULL);

	SDL_ShowWindow(SplashWindow);
	Pump();

#endif

	return true;
}


/**
 * Sets the text displayed on the splash screen (for startup/loading progress)
 *
 * @param	InType		Type of text to change
 * @param	InText		Text to display
 */
void FLinuxSplashState::SetSplashText( const SplashTextType::Type InType, const FText& InText )
{
#if WITH_EDITOR
	if (!InText.IdenticalTo(SplashText[ InType ], ETextIdenticalModeFlags::LexicalCompareInvariants))
	{
		SplashText[ InType ] = InText;
		bStringsChanged = true;
	}
#endif
}


//---------------------------------------------------------
void FLinuxSplashState::Redraw()
{
#if WITH_EDITOR
	if (bNeedsRedraw || bStringsChanged)
	{
		RenderStrings();
		SDL_RenderTexture(SplashRenderer, SplashTexture, NULL, NULL);
		SDL_RenderPresent(SplashRenderer);
		bNeedsRedraw = false;
	}
#else
	if (bNeedsRedraw)
	{
		SDL_UpdateWindowSurface(SplashWindow);
		bNeedsRedraw = false;
	}
#endif
}


void FLinuxSplashState::Pump()
{
	if (SplashWindow)
	{
		SDL_Event Event;
		while (SDL_PollEvent(&Event))
		{
			if ((Event.type == SDL_EVENT_WINDOW_EXPOSED) && (SDL_GetWindowID(SplashWindow) == Event.window.windowID))
			{
				bNeedsRedraw = true;
			}
		}

		Redraw();
	}
}


/**
 * Open a splash screen if there's not one already and if it's not disabled.
 *
 */
void FLinuxPlatformSplash::Show( )
{
	// need to do a splash screen?
	if(GSplashState || !FApp::CanEverRender() || FParse::Param(FCommandLine::Get(), TEXT("NOSPLASH")) == true)
	{
		return;
	}

	// decide on which splash screen to show
	FString FinalGameName = FApp::GetProjectName();
	if (GIsEditor)
	{
		if (!FApp::HasProjectName())
		{
			FinalGameName = FString("Unreal Editor");
		}
		else
		{
			FinalGameName = FString("Unreal Editor - ") + FinalGameName;
		}
	}

	const FText GameName = FText::FromString(FinalGameName);

	bool IsCustom = false;

	// first look for the splash, do not init anything if not found
	FString SplashPath;
	{
		const TCHAR* SplashImage = GIsEditor ? TEXT("EdSplash") : TEXT("Splash");
		if (!GetSplashPath(SplashImage, SplashPath, IsCustom))
		{
			UE_LOGF(LogHAL, Warning, "Splash screen image not found.");
			return;	// early out
		}
	}

	// look for the icon separately, also avoid initialization if not found
	FString IconPath;
	{
		const TCHAR* EditorIcons[] =
		{
			TEXT("EdIcon"),
			TEXT("EdIconDefault"),
			nullptr
		};

		const TCHAR* GameIcons[] =
		{
			TEXT("Icon"),
			TEXT("IconDefault"),
			nullptr
		};

		const TCHAR** IconsToTry = GIsEditor? EditorIcons : GameIcons;
		bool bIconFound = false;
		for (const TCHAR* IconImage = *IconsToTry; IconImage != nullptr && !bIconFound; IconImage = *(++IconsToTry))
		{
			bool bDummy;
			if (GetSplashPath(IconImage, IconPath, bDummy))
			{
				bIconFound = true;
			}
		}

		if (!bIconFound)
		{
			UE_LOGF(LogHAL, Warning, "Game icon not found.");
			IconPath.Reset();
		}
	}

	GSplashState = new FLinuxSplashState;

	// In the editor, we'll display loading info
	FText AppName;
	if ( GIsEditor )
	{
		// Set initial startup progress info
		{
			GSplashState->SetSplashText( SplashTextType::StartupProgress,
				NSLOCTEXT("UnrealEd", "SplashScreen_InitialStartupProgress", "Loading..." ) );
		}

		// Set version info
		{
			const FText Version = FText::FromString( FEngineVersion::Current().ToString( FEngineBuildSettings::IsPerforceBuild() ? EVersionComponent::Branch : EVersionComponent::Patch ) );

			FText VersionInfo = FText::Format( NSLOCTEXT( "UnrealEd", "UnrealEdTitleWithVersionNoGameName_F", "Unreal Editor {0}" ), Version );
			AppName = NSLOCTEXT( "UnrealEd", "UnrealEdTitleNoGameName_F", "Unreal Editor" );

			GSplashState->SetSplashText( SplashTextType::VersionInfo1, VersionInfo );
		}

		// Display copyright information in editor splash screen
		{
			const FText CopyrightInfo = NSLOCTEXT( "UnrealEd", "SplashScreen_CopyrightInfo", "Copyright \u00a9   Epic Games, Inc.   All rights reserved." );
			GSplashState->SetSplashText( SplashTextType::CopyrightInfo, CopyrightInfo );
		}

		// Set GameName for Editor always
		{
			GSplashState->SetSplashText( SplashTextType::GameName, GameName );
		}
	}
	// Don't set the game name if the splash screen is custom.
	else if ( !IsCustom )
	{
		GSplashState->SetSplashText( SplashTextType::GameName, GameName );
	}

	if (!GSplashState->InitSplashResources(AppName, SplashPath, IconPath))
	{
		delete GSplashState;
		GSplashState = nullptr;
	}
}


/**
* Done with splash screen. Close it and clean up.
*/
void FLinuxPlatformSplash::Hide()
{
	delete GSplashState;
	GSplashState = nullptr;
}


/**
* Sets the text displayed on the splash screen (for startup/loading progress)
*
* @param	InType		Type of text to change
* @param	InText		Text to display
*/
void FLinuxPlatformSplash::SetSplashText( const SplashTextType::Type InType, const FText& InText )
{
	if (GSplashState)
	{
		// We only want to bother drawing startup progress in the editor, since this information is
		// not interesting to an end-user (also, it's not usually localized properly.)
		if (InType == SplashTextType::CopyrightInfo || GIsEditor)
		{
			GSplashState->SetSplashText(InType, InText);
		}
		GSplashState->Pump();
	}
}
