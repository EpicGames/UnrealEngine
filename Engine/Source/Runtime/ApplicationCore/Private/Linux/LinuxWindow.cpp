// Copyright Epic Games, Inc. All Rights Reserved.

#include "Linux/LinuxWindow.h"

#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Internationalization/Text.h"
#include "Math/Vector2D.h"
#include "GenericPlatform/GenericWindowDefinition.h"
#include "Misc/App.h"
#include "Linux/LinuxApplication.h"
#include "Linux/LinuxPlatformApplicationMisc.h"
#include "Internationalization/Internationalization.h" // LOCTEXT

#define LOCTEXT_NAMESPACE "LinuxWindow"

DEFINE_LOG_CATEGORY( LogLinuxWindow );
DEFINE_LOG_CATEGORY( LogLinuxWindowType );
DEFINE_LOG_CATEGORY( LogLinuxWindowEvent );

// SDL 2.0.4 as of 10374:dccf51aee79b will account for border width/height automatically (see SDL_x11window.c)
// might need to be a function in case SDL gets overriden at runtime
#define UE_USING_BORDERS_AWARE_SDL					1

namespace 
{
	constexpr int DefaultMinWindowWidth = 100;
	constexpr int DefaultMinWindowHeight = 50;
}

FLinuxWindow::~FLinuxWindow()
{
	// NOTE: The HWnd is invalid here!
	//       Doing stuff with HWnds will fail silently.
	//       Use Destroy() instead.
}

TSharedRef< FLinuxWindow > FLinuxWindow::Make()
{
	// First, allocate the new native window object.  This doesn't actually create a native window or anything,
	// we're simply instantiating the object so that we can keep shared references to it.
	return MakeShareable( new FLinuxWindow() );
}

FLinuxWindow::FLinuxWindow()
	: HWnd(NULL)
	, WindowMode( EWindowMode::Windowed )
	, bIsVisible( false )
	, bWasFullscreen( false )
	, bIsPopupWindow(false)
	, bIsTooltipWindow(false)
	, bIsConsoleWindow(false)
	, bIsDialogWindow(false)
	, bIsNotificationWindow(false)
	, bIsTopLevelWindow(false)
	, bIsDragAndDropWindow(false)
	, bIsUtilityWindow(false)
	, bIsPointerInsideWindow(false)
	, LeftBorderWidth(0)
	, TopBorderHeight(0)
	, bValidNativePropertiesCache(false)
	, DPIScaleFactor(1.0f)
{
	PreFullscreenWindowRect.left = PreFullscreenWindowRect.top = PreFullscreenWindowRect.right = PreFullscreenWindowRect.bottom = 0;
}

SDL_HWindow FLinuxWindow::GetHWnd() const
{
	return HWnd;
}

//	HINSTANCE InHInstance,
void FLinuxWindow::Initialize( FLinuxApplication* const Application, const TSharedRef< FGenericWindowDefinition >& InDefinition, const TSharedPtr< FLinuxWindow >& InParent, const bool bShowImmediately )
{
	Definition = InDefinition;
	OwningApplication = Application;
	ParentWindow = InParent;

	if (!FPlatformApplicationMisc::InitSDL()) //	will not initialize more than once
	{
		UE_LOGF(LogInit, Fatal, "FLinuxWindow::Initialize() : InitSDL() failed, cannot initialize window.");
		// unreachable
		return;
	}

#if DO_CHECK
	uint32 InitializedSubsystems = SDL_WasInit(SDL_INIT_VIDEO);
	check(InitializedSubsystems & SDL_INIT_VIDEO);
#endif // DO_CHECK

	// Finally, let's initialize the new native window object.  Calling this function will often cause OS
	// window messages to be sent! (such as activation messages)

	RegionWidth = RegionHeight = INDEX_NONE;

	const float XInitialRect = Definition->XDesiredPositionOnScreen;
	const float YInitialRect = Definition->YDesiredPositionOnScreen;

	const float WidthInitial = Definition->WidthDesiredOnScreen;
	const float HeightInitial = Definition->HeightDesiredOnScreen;

	// calculate the DPI at the centerpoint
	float XCenter = XInitialRect + WidthInitial / 2.0f;
	float YCenter = YInitialRect + HeightInitial / 2.0f;
	DPIScaleFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(XCenter, YCenter);

	int32 X = FMath::TruncToInt( XInitialRect + 0.5f );
	int32 Y = FMath::TruncToInt( YInitialRect + 0.5f );
	int32 ClientWidth = FMath::TruncToInt( WidthInitial + 0.5f );
	int32 ClientHeight = FMath::TruncToInt( HeightInitial + 0.5f );
	// Clamp to 1x1 to avoid a degenerate Vulkan swapchain on tooltips/autosized windows that
	// Slate creates at 0x0 before measuring content. NVIDIA loses the device after a few
	// OUT_OF_DATE recreates on a 0x0 surface. ReshapeWindow applies the same clamp.
	ClientWidth = FMath::Max(ClientWidth, 1);
	ClientHeight = FMath::Max(ClientHeight, 1);
	int32 WindowWidth = ClientWidth;
	int32 WindowHeight = ClientHeight;


        //      The SDL window doesn't need to be reshaped.
        //      the size of the window you input is the sizeof the client.
	bool bIsResizable = false;

	SDL_PropertiesID Props = SDL_CreateProperties();
	SDL_SetStringProperty( Props, SDL_PROP_WINDOW_CREATE_TITLE_STRING, TCHAR_TO_ANSI( *Definition->Title ) );
	SDL_SetNumberProperty( Props, SDL_PROP_WINDOW_CREATE_WIDTH_NUMBER, ClientWidth );
	SDL_SetNumberProperty( Props, SDL_PROP_WINDOW_CREATE_HEIGHT_NUMBER, ClientHeight );

	SDL_SetBooleanProperty(Props, SDL_PROP_WINDOW_CREATE_HIDDEN_BOOLEAN, true);

	// Slate computes popup/tooltip placement using the correct per-anchor work area; let it win.
	// SDL's clamp picks the wrong monitor when the toplevel straddles two displays.
	SDL_SetBooleanProperty(Props, SDL_PROP_WINDOW_CREATE_CONSTRAIN_POPUP_BOOLEAN, false);

	UE_LOGF(LogLinuxWindowType, Verbose, "Linux Window: X=%d Y=%d, Width=%d, Height=%d", X, Y, ClientWidth, ClientHeight);
	switch(FLinuxPlatformApplicationMisc::WindowStyle())
	{
		case SDL_WINDOW_VULKAN:
			SDL_SetBooleanProperty( Props, SDL_PROP_WINDOW_CREATE_VULKAN_BOOLEAN, true );
			break;
		case SDL_WINDOW_OPENGL:
			SDL_SetBooleanProperty( Props, SDL_PROP_WINDOW_CREATE_OPENGL_BOOLEAN, true );
			break;
	}

	if (Definition->IsRegularWindow && Definition->HasSizingFrame)
	{
		SDL_SetBooleanProperty( Props, SDL_PROP_WINDOW_CREATE_RESIZABLE_BOOLEAN, true );
		bIsResizable = true;
	}

	const bool bShouldActivate = Definition->ActivationPolicy != EWindowActivationPolicy::Never;
	const char *SDLWindowType = nullptr;			// use this to set the X11 Window Type if necessary

	// Tooltip window. Prefer Slate's explicit owner (passed as InParent via
	// AddWindowAsNativeChild on Wayland) over the mouse-focused window -- the cursor can
	// be over a different toplevel than the widget that requested the tooltip.
	if (!Definition->HasOSWindowBorder &&
		(Definition->Type == EWindowType::ToolTip || (!InParent.IsValid() && !Definition->AcceptsInput)) && // tooltips can be interactive
		Definition->IsTopmostWindow && !Definition->AppearsInTaskbar &&
		!Definition->HasSizingFrame && !Definition->IsModalWindow &&
		!Definition->IsRegularWindow && Definition->SizeWillChangeOften)
	{
		// Both backends need the tooltip flag + parent: Wayland positions via xdg_positioner;
		// SDL3's X11 backend only applies override-redirect and X11_ConstrainPopup to flagged
		// popups, and 26.04 XWayland refuses to display unflagged parentless tooltips.
		SDL_Window* TooltipParent = InParent.IsValid() ? InParent->GetHWnd() : SDL_GetMouseFocus();
		if (TooltipParent)
		{
			SDL_SetBooleanProperty( Props, SDL_PROP_WINDOW_CREATE_TOOLTIP_BOOLEAN, true );
			SDL_SetPointerProperty( Props, SDL_PROP_WINDOW_CREATE_PARENT_POINTER, TooltipParent );
		}
		SDL_SetBooleanProperty( Props, SDL_PROP_WINDOW_CREATE_FOCUSABLE_BOOLEAN, false );
		bIsTooltipWindow = true;
		UE_LOGF(LogLinuxWindowType, Verbose, "*** New Window is a Tooltip Window ***");
	}
	// This is a notification window.
	else if (InParent.IsValid() && !Definition->HasOSWindowBorder &&
		Definition->AcceptsInput && !Definition->IsTopmostWindow && 
		!Definition->AppearsInTaskbar && !Definition->HasSizingFrame &&
		!Definition->IsModalWindow && !Definition->IsRegularWindow &&
		!bShouldActivate && Definition->SizeWillChangeOften)
	{
		SDL_SetPointerProperty( Props, SDL_PROP_WINDOW_CREATE_PARENT_POINTER, InParent->GetHWnd() );

		// On Wayland only xdg_popup surfaces can be positioned relative to their parent, so give the
		// notification the popup-menu role. UTILITY and MENU are mutually exclusive at creation (swap,
		// not add); keep bIsNotificationWindow so focus-revert handling stays on the notification path.
		if (FLinuxPlatformApplicationMisc::IsUsingWayland())
		{
			SDL_SetBooleanProperty( Props, SDL_PROP_WINDOW_CREATE_MENU_BOOLEAN, true );
			bIsPopupWindow = true;
		}
		else
		{
			SDL_SetBooleanProperty( Props, SDL_PROP_WINDOW_CREATE_UTILITY_BOOLEAN, true );
		}
		bIsNotificationWindow = true;
		UE_LOGF(LogLinuxWindowType, Verbose, "*** New Window is a Notification Window ***");
	}
	// Is it another notification window?
	else if (InParent.IsValid() && !Definition->HasOSWindowBorder &&
		Definition->AcceptsInput && !Definition->IsTopmostWindow && 
		!Definition->AppearsInTaskbar && !Definition->HasSizingFrame &&
		Definition->IsModalWindow && !Definition->IsRegularWindow &&
		bShouldActivate && !Definition->SizeWillChangeOften)
	{
		SDL_SetBooleanProperty( Props, SDL_PROP_WINDOW_CREATE_MODAL_BOOLEAN, true );
		SDL_SetPointerProperty( Props, SDL_PROP_WINDOW_CREATE_PARENT_POINTER, InParent->GetHWnd() );
		bIsNotificationWindow = true;
		UE_LOGF(LogLinuxWindowType, Verbose, "*** New Window is another Notification Window ***");
	}
	else if (!InParent.IsValid() && !Definition->HasOSWindowBorder &&
		Definition->AcceptsInput && !Definition->IsTopmostWindow && 
		!Definition->AppearsInTaskbar && !Definition->HasSizingFrame &&
		!Definition->IsModalWindow && !Definition->IsRegularWindow &&
		!bShouldActivate && Definition->SizeWillChangeOften && 
		Definition->Type == EWindowType::Notification)
	{
		// Even parentless notification windows should be use the right window type
		SDL_SetBooleanProperty( Props, SDL_PROP_WINDOW_CREATE_UTILITY_BOOLEAN, true );
		bIsNotificationWindow = true;
		UE_LOGF(LogLinuxWindowType, Verbose, "*** New Window is a parentless Notification Window ***");
	}
	// This is a popup menu window?
	else if (InParent.IsValid() && !Definition->HasOSWindowBorder &&
		Definition->AcceptsInput && !Definition->IsTopmostWindow && 
		!Definition->AppearsInTaskbar && !Definition->HasSizingFrame &&
		!Definition->IsModalWindow && !Definition->IsRegularWindow &&
		bShouldActivate && !Definition->SizeWillChangeOften)
	{
		SDL_SetBooleanProperty( Props, SDL_PROP_WINDOW_CREATE_TRANSPARENT_BOOLEAN, true );
		SDL_SetBooleanProperty( Props, SDL_PROP_WINDOW_CREATE_BORDERLESS_BOOLEAN, true );
		SDL_SetPointerProperty( Props, SDL_PROP_WINDOW_CREATE_PARENT_POINTER, InParent->GetHWnd() );

		// Note that on Wayland, only popup menus can be repositioned, and they are positioned relative to their parents.
		// Without this adjustment, all popup menus will be spawned at the center of their parent window and cannot be moved.
		// This means we /must/ set the SDL_PROP_WINDOW_CREATE_MENU_BOOLEAN which sets the SDL_WINDOW_POPUP_MENU flag
		// on these windows, or menus won't be usable on Wayland.
		// To make this work, we transparently translate any setting of positions of parent-relative windows from outside
		// LinuxWindow to parent-relative positions. (i.e. places that use SDL_SetWindowPosition().)
		SDL_SetBooleanProperty( Props, SDL_PROP_WINDOW_CREATE_MENU_BOOLEAN, true );
		bIsPopupWindow = true;
		UE_LOGF(LogLinuxWindowType, Verbose, "*** New Window is a Popup Menu Window ***");
	}
	// Is it a console window?
	else if( InParent.IsValid() && !Definition->HasOSWindowBorder &&
		Definition->AcceptsInput && !Definition->IsTopmostWindow &&
		!Definition->AppearsInTaskbar && !Definition->HasSizingFrame &&
		!Definition->IsModalWindow && !Definition->IsRegularWindow &&
		!bShouldActivate && !Definition->SizeWillChangeOften)
	{
		SDL_SetBooleanProperty( Props, SDL_PROP_WINDOW_CREATE_TRANSPARENT_BOOLEAN, true );
		SDL_SetBooleanProperty( Props, SDL_PROP_WINDOW_CREATE_BORDERLESS_BOOLEAN, true );
		// Toolbar-dropdown-style popups need parent + menu flag so SDL picks the right
		// shell role: xdg_popup on Wayland, override-redirect with X11_ConstrainPopup on X11.
		SDL_SetPointerProperty( Props, SDL_PROP_WINDOW_CREATE_PARENT_POINTER, InParent->GetHWnd() );
		SDL_SetBooleanProperty( Props, SDL_PROP_WINDOW_CREATE_MENU_BOOLEAN, true );
		bIsConsoleWindow = true;
		bIsPopupWindow = true;
		UE_LOGF(LogLinuxWindowType, Verbose, "*** New Window is a Console Window ***");
	}
	// Is it a drag and drop window?
	else if (!InParent.IsValid() && !Definition->HasOSWindowBorder &&
		!Definition->AcceptsInput && Definition->IsTopmostWindow && 
		!Definition->AppearsInTaskbar && !Definition->HasSizingFrame &&
		!Definition->IsModalWindow && !Definition->IsRegularWindow &&
		!bShouldActivate && !Definition->SizeWillChangeOften)
	{
		// TODO Experimental (The SDL_WINDOW_DND sets focus)
		SDLWindowType = "_NET_WM_WINDOW_TYPE_DND";
		bIsDragAndDropWindow = true;
		UE_LOGF(LogLinuxWindowType, Verbose, "*** New Window is a Drag and Drop Window ***");
	}
	// Is modal dialog window?
	else if (InParent.IsValid() && !Definition->HasOSWindowBorder &&
		Definition->AcceptsInput && !Definition->IsTopmostWindow && 
		Definition->AppearsInTaskbar && Definition->IsModalWindow &&
		Definition->IsRegularWindow && bShouldActivate &&
		!Definition->SizeWillChangeOften)
	{
		SDLWindowType = "_NET_WM_WINDOW_TYPE_DIALOG";
		SDL_SetBooleanProperty( Props, SDL_PROP_WINDOW_CREATE_MODAL_BOOLEAN, true );
		SDL_SetPointerProperty( Props, SDL_PROP_WINDOW_CREATE_PARENT_POINTER, InParent->GetHWnd() );
		bIsDialogWindow = true;
		UE_LOGF(LogLinuxWindowType, Verbose, "*** New Window is a Modal Dialog Window ***");
	}
	// Is a Blueprint, Cascade, etc. utility window.
	else if (InParent.IsValid() && !Definition->HasOSWindowBorder &&
		Definition->AcceptsInput && !Definition->IsTopmostWindow && 
		Definition->AppearsInTaskbar && Definition->HasSizingFrame &&
		!Definition->IsModalWindow && Definition->IsRegularWindow &&
		bShouldActivate && !Definition->SizeWillChangeOften)
	{
		bIsUtilityWindow = true;
		UE_LOGF(LogLinuxWindowType, Verbose, "*** New Window is a BP, Cascade, etc. Window ***");
	}
	else
	{
		UE_LOGF(LogLinuxWindowType, Verbose, "*** New Window is TopLevel Window ***");
		bIsTopLevelWindow = true;
	}

	if ( !Definition->HasOSWindowBorder )
	{
		SDL_SetBooleanProperty( Props, SDL_PROP_WINDOW_CREATE_BORDERLESS_BOOLEAN, true );

		if (Definition->IsTopmostWindow)
		{
			SDL_SetBooleanProperty( Props, SDL_PROP_WINDOW_CREATE_ALWAYS_ON_TOP_BOOLEAN, true );
		}

		// MODAL, NOTIFICATION, MENU POPUP, and TOOLTIP windows will already not be shown in the task bar
		// (Creating windows with SDL_PROP_WINDOW_CREATE(_MENU_BOOLEAN|_TOOLTIP_BOOLEAN) will fail if UTILITY is added.)
		if (!Definition->AppearsInTaskbar && !Definition->IsModalWindow && !bIsNotificationWindow && 
			!SDL_GetBooleanProperty( Props, SDL_PROP_WINDOW_CREATE_MENU_BOOLEAN, false ) &&
			!SDL_GetBooleanProperty( Props, SDL_PROP_WINDOW_CREATE_TOOLTIP_BOOLEAN, false ))
		{

			SDL_SetBooleanProperty( Props, SDL_PROP_WINDOW_CREATE_UTILITY_BOOLEAN, true );
		}
	}

	/* 
			Do not set for Notification Windows the transient flag because the WM's usually raise the the parent window
			if the Notificaton Window gets raised. That behaviour is to aggresive and disturbs users doing other things 
			while UnrealEngine calculates lights and other things and pop ups notifications. Notifications will be handled so that 
			they are some sort of independend but will be raised if the TopLevel Window gets focused or activated.
	*/
	
	// Set parent and make the Window modal if necessary
	if (bIsUtilityWindow || bIsDialogWindow || bIsConsoleWindow)
	{
		SDL_SetPointerProperty( Props, SDL_PROP_WINDOW_CREATE_PARENT_POINTER, InParent->GetHWnd() );
		if (Definition->IsModalWindow)
		{
			SDL_SetBooleanProperty( Props, SDL_PROP_WINDOW_CREATE_MODAL_BOOLEAN, true );
		}
	}

	// This lets us override the X11 window type
	if (SDLWindowType != nullptr)
	{
		SDL_SetHint(SDL_HINT_X11_WINDOW_TYPE, SDLWindowType);
	}

	// If the props indicate this window is parent-relative, we need to get the offset.
	int32 ToParentX = 0;
	int32 ToParentY = 0;
	SDL_Window* ParentHWnd = InParent.IsValid() ? InParent->GetHWnd() : nullptr;
	if (!ParentHWnd)
	{
		// Tooltips on Wayland have a parent set directly in Props (via SDL_GetMouseFocus)
		// but InParent is not valid since Slate doesn't provide one.
		ParentHWnd = static_cast<SDL_Window*>(SDL_GetPointerProperty(Props, SDL_PROP_WINDOW_CREATE_PARENT_POINTER, nullptr));
	}
	if (ParentHWnd)
	{
		GetParentRelativeOffset(Props, ParentHWnd, ToParentX, ToParentY);
	}

	const int32 CreateX = X + ToParentX;
	const int32 CreateY = Y + ToParentY;
	SDL_SetNumberProperty( Props, SDL_PROP_WINDOW_CREATE_X_NUMBER, CreateX );
	SDL_SetNumberProperty( Props, SDL_PROP_WINDOW_CREATE_Y_NUMBER, CreateY );

	// Diagnostic trace for popup and tooltip placement: requested abs position, parent
	// origin used to convert to parent-relative, and the final create-time properties.
	if (UE_LOG_ACTIVE(LogLinuxWindowType, Verbose))
	{
		if (ParentHWnd)
		{
			int32 ParentScreenX = 0;
			int32 ParentScreenY = 0;
			SDL_GetWindowPosition(ParentHWnd, &ParentScreenX, &ParentScreenY);
			UE_LOGF(LogLinuxWindowType, Verbose,
				"Window create coords: req_abs=(%d,%d) parent=%p parent_pos=(%d,%d) to_parent_offset=(%d,%d) create_xy=(%d,%d)",
				X, Y, ParentHWnd, ParentScreenX, ParentScreenY, ToParentX, ToParentY, CreateX, CreateY);
		}
		else
		{
			UE_LOGF(LogLinuxWindowType, Verbose,
				"Window create coords: req_abs=(%d,%d) no parent, create_xy=(%d,%d)",
				X, Y, CreateX, CreateY);
		}
	}

	HWnd = SDL_CreateWindowWithProperties( Props );
	SDL_DestroyProperties( Props );

	if (HWnd && UE_LOG_ACTIVE(LogLinuxWindowType, Verbose))
	{
		int32 PostCreateX = 0;
		int32 PostCreateY = 0;
		SDL_GetWindowPosition(HWnd, &PostCreateX, &PostCreateY);
		UE_LOGF(LogLinuxWindowType, Verbose,
			"Window create coords: post-create SDL_GetWindowPosition=(%d,%d) (sdl_id=%u)",
			PostCreateX, PostCreateY, SDL_GetWindowID(HWnd));
	}

	// Window is created hidden -- disable Vulkan presenting until Show() is called.
	// This prevents Vulkan from committing buffers before the shell surface role is assigned.
	if (HWnd)
	{
		SDL_SetBooleanProperty(SDL_GetWindowProperties(HWnd), "UE.window.present_enabled", false);
	}

	// Restore the X11 Window Type
	if (SDLWindowType != nullptr)
	{
		SDL_ResetHint(SDL_HINT_X11_WINDOW_TYPE);
	}
	
	// produce a helpful message for common driver errors
	if (HWnd == nullptr)
	{
		FString ErrorMessage;

		ErrorMessage = FText::Format(LOCTEXT("SDLWindowCreationFailedLinuxFmt", "Window creation failed (SDL error: '{0}'')"), FText::FromString(UTF8_TO_TCHAR(SDL_GetError()))).ToString();
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ErrorMessage,
									 *LOCTEXT("SDLWindowCreationFailedLinuxTitle", "Unable to create an SDL window.").ToString());

		checkf(false, TEXT("%s"), *ErrorMessage);
		// unreachable
		return;
	}

	// SDL_StartTextInput is no longer a global state, so we need to set it on every
	// window we create to have it work like it used to
	if (Definition->AcceptsInput)
	{
		SDL_StartTextInput(HWnd);
	}

	// SDL_SetWindow[Minimum/Maximum]Size() requires a valid value for BOTH width and height, we can't do one or the other without magic numbers
	const bool bIsMinSizeSet = Definition->SizeLimits.GetMinWidth().IsSet() && Definition->SizeLimits.GetMinHeight().IsSet();
	const bool bIsMaxSizeSet = Definition->SizeLimits.GetMaxWidth().IsSet() && Definition->SizeLimits.GetMaxHeight().IsSet();

	if (bIsResizable && !bIsMinSizeSet)
	{
		// Set a reasonable minimum size if none are set
		// Vulkan will fail to create a swap chain if the back buffer is 0x0
		SDL_SetWindowMinimumSize(HWnd, DefaultMinWindowWidth, DefaultMinWindowHeight);
	}
	else if (bIsMinSizeSet)
	{
		SDL_SetWindowMinimumSize(HWnd, (int)*Definition->SizeLimits.GetMinWidth(), (int)*Definition->SizeLimits.GetMinHeight());
	}

	if (bIsMaxSizeSet)
	{
		SDL_SetWindowMaximumSize(HWnd, (int)*Definition->SizeLimits.GetMaxWidth(), (int)*Definition->SizeLimits.GetMaxHeight());
	}

	if (Definition->AppearsInTaskbar)
	{
		// Try to find an icon for the window
		const FText GameName = FText::FromString(FApp::GetProjectName());
		FString Iconpath;
		SDL_Surface *IconImage = nullptr;

		if (!GameName.IsEmpty())
		{
			if (GIsEditor)
			{
				Iconpath = FPaths::ProjectContentDir() / TEXT("Splash/EdIcon.bmp");
			}
			else
			{
				Iconpath = FPaths::ProjectContentDir() / TEXT("Splash/Icon.bmp");
			}

			Iconpath = FPaths::FPaths::ConvertRelativePathToFull(Iconpath);
			if (IFileManager::Get().FileSize(*Iconpath) != -1)
			{
				IconImage = SDL_LoadBMP(TCHAR_TO_UTF8(*Iconpath));
			}
		}

		if (IconImage == nullptr)
		{
			// no game specified or there's no custom icons for the game. use default icons.
			if (GIsEditor)
			{
				Iconpath = FPaths::EngineContentDir() / TEXT("Splash/EdIconDefault.bmp");
			}
			else
			{
				Iconpath = FPaths::EngineContentDir() / TEXT("Splash/IconDefault.bmp");
			}

			Iconpath = FPaths::FPaths::ConvertRelativePathToFull(Iconpath);
			if (IFileManager::Get().FileSize(*Iconpath) != -1)
			{
				IconImage = SDL_LoadBMP(TCHAR_TO_UTF8(*Iconpath));
			}
		}

		if (IconImage != nullptr)
		{
			SDL_SetWindowIcon(HWnd, IconImage);

			SDL_DestroySurface(IconImage);
			IconImage = nullptr;
		}
	}

	SDL_SetWindowHitTest( HWnd, FLinuxWindow::HitTest, this );

	VirtualWidth  = ClientWidth;
	VirtualHeight = ClientHeight;

	// attempt to early cache native properties
	CacheNativeProperties();

	// We call reshape window here because we didn't take into account the non-client area
	// in the initial creation of the window. Slate should only pass client area dimensions.
	// Reshape window may resize the window if the non-client area is encroaching on our
	// desired client area space.
	ReshapeWindow( X, Y, ClientWidth, ClientHeight );

	if ( Definition->TransparencySupport == EWindowTransparency::PerWindow )
	{
		SetOpacity( Definition->Opacity );
	}

	// TODO This can be removed later - for debugging purposes.
	WindowSDLID = SDL_GetWindowID( HWnd );
}

SDL_HitTestResult FLinuxWindow::HitTest( SDL_Window *SDLwin, const SDL_Point *point, void *data )
{
	FLinuxWindow *pself = static_cast<FLinuxWindow *>( data );
	TSharedPtr< FLinuxWindow > self = pself->OwningApplication->FindWindowBySDLWindow( SDLwin );
	if ( !self.IsValid() ) 
	{
		// When a window is being destroyed it can receive events before the window is actually removed.
		// At this point it is no longer in UE's Window list so fails the IsValid test
		if (pself->GetHWnd() == SDLwin)
		{
			return SDL_HITTEST_NORMAL;
		}

		UE_LOGF(LogLinuxWindow, Warning, "BAD HIT TEST EVENT: SDL window = %p\n", SDLwin);
		return SDL_HITTEST_NORMAL;
	}

	EWindowZone::Type eventZone = pself->OwningApplication->WindowHitTest( self, point->x, point->y );

	static const SDL_HitTestResult Results[] =
	{
		SDL_HITTEST_NORMAL,
		SDL_HITTEST_NORMAL,
		SDL_HITTEST_RESIZE_TOPLEFT,
		SDL_HITTEST_RESIZE_TOP,
		SDL_HITTEST_RESIZE_TOPRIGHT,
		SDL_HITTEST_RESIZE_LEFT,
		SDL_HITTEST_RESIZE_RIGHT,
		SDL_HITTEST_RESIZE_BOTTOMLEFT,
		SDL_HITTEST_RESIZE_BOTTOM,
		SDL_HITTEST_RESIZE_BOTTOMRIGHT,
		SDL_HITTEST_DRAGGABLE,
		SDL_HITTEST_NORMAL,
		SDL_HITTEST_NORMAL,
		SDL_HITTEST_NORMAL,
		SDL_HITTEST_NORMAL
	};

	return Results[eventZone];
}

void FLinuxWindow::ConvertToParentRelativePosition(SDL_Window* ThisWindow, int32& X, int32& Y)
{
	// SDL3 popup positioning (X11 and Wayland) expects parent-relative coords. Walk up,
	// subtracting each popup ancestor's position; stop at the first non-popup (toplevel).
	SDL_Window* Current = ThisWindow;
	while (SDL_GetWindowFlags(Current) & (SDL_WINDOW_POPUP_MENU | SDL_WINDOW_TOOLTIP))
	{
		SDL_Window* Parent = SDL_GetWindowParent(Current);
		if (!Parent)
		{
			break;
		}

		int32 ParentX, ParentY;
		SDL_GetWindowPosition(Parent, &ParentX, &ParentY);
		X -= ParentX;
		Y -= ParentY;
		Current = Parent;
	}
}

void FLinuxWindow::GetParentRelativeOffset(SDL_PropertiesID Props, SDL_Window* StartParent, int32& X, int32& Y)
{
	X = 0;
	Y = 0;

	// Pre-creation variant of ConvertToParentRelativePosition: only popup/tooltip windows
	// take the offset, and we have to walk from StartParent since the new window doesn't
	// exist yet.
	if (StartParent != nullptr &&
		(SDL_GetBooleanProperty(Props, SDL_PROP_WINDOW_CREATE_MENU_BOOLEAN, false) ||
		 SDL_GetBooleanProperty(Props, SDL_PROP_WINDOW_CREATE_TOOLTIP_BOOLEAN, false)))
	{
		int32 ParentX, ParentY;
		SDL_GetWindowPosition(StartParent, &ParentX, &ParentY);
		X -= ParentX;
		Y -= ParentY;
		ConvertToParentRelativePosition(StartParent, X, Y);
	}
}

/** Native windows should implement MoveWindowTo by relocating the platform-specific window to (X,Y). */
void FLinuxWindow::MoveWindowTo( int32 X, int32 Y )
{
	// Wayland: the compositor owns toplevel placement; force SDL and Slate's cached origin
	// to (0,0) so popup positioner math and IsScreenspaceMouseWithin hit-testing agree.
	if (FLinuxPlatformApplicationMisc::IsUsingWayland() &&
		!(SDL_GetWindowFlags(HWnd) & (SDL_WINDOW_POPUP_MENU | SDL_WINDOW_TOOLTIP)))
	{
		SDL_SetWindowPosition(HWnd, 0, 0);
		TSharedPtr<FLinuxWindow> LinuxWindow = OwningApplication->FindWindowBySDLWindow(HWnd);
		if (LinuxWindow)
		{
			OwningApplication->GetMessageHandler()->OnMovedWindow(LinuxWindow.ToSharedRef(), 0, 0);
		}
		return;
	}

	const int32 RequestedAbsX = X;
	const int32 RequestedAbsY = Y;

	// NOTE: For SDL parent-relative windows, this modifies X and Y!
	ConvertToParentRelativePosition(HWnd, X, Y);

	// Diagnostic trace for popup and tooltip moves: requested abs position vs the parent-
	// relative value handed to SDL_SetWindowPosition, plus the parent origin used for the walk.
	if (UE_LOG_ACTIVE(LogLinuxWindow, Verbose))
	{
		int32 ParentScreenX = 0;
		int32 ParentScreenY = 0;
		SDL_Window* WalkParent = SDL_GetWindowParent(HWnd);
		if (WalkParent)
		{
			SDL_GetWindowPosition(WalkParent, &ParentScreenX, &ParentScreenY);
		}
		const SDL_WindowFlags HwndFlags = SDL_GetWindowFlags(HWnd);
		UE_LOGF(LogLinuxWindow, Verbose,
			"MoveWindowTo wid=%u flags=0x%llx req_abs=(%d,%d) parent_pos=(%d,%d) parent_relative=(%d,%d)",
			SDL_GetWindowID(HWnd),
			static_cast<unsigned long long>(HwndFlags),
			RequestedAbsX, RequestedAbsY,
			ParentScreenX, ParentScreenY,
			X, Y);
	}

	if (UE_USING_BORDERS_AWARE_SDL)
	{
		SDL_SetWindowPosition( HWnd, X, Y );
	}
	else
	{
		// we are passed coordinates of a client area, so account for decorations
		checkf(bValidNativePropertiesCache, TEXT("Attempted to use border sizes too early, native properties aren't yet cached. Review the flow"));

		SDL_SetWindowPosition( HWnd, X - LeftBorderWidth, Y - TopBorderHeight );
	}
}

/** Native windows should implement BringToFront by making this window the top-most window (i.e. focused).
 *
 * @param bForce	Forces the window to the top of the Z order, even if that means stealing focus from other windows
 *					In general do not pass true for this.  It is really only useful for some windows, like game windows where not forcing it to the front
 *					could cause mouse capture and mouse lock to happen but without the window visible
 */
void FLinuxWindow::BringToFront( bool bForce )
{
	// TODO Forces the the window to top of z order? Only that? SDL is using XMapRaised which changes the z order
	// so we do not steal focus here I guess.
	if(bForce)
	{
		SDL_RaiseWindow(HWnd);
	}
	else
	{
		Show();
	}
}

/** Native windows should implement this function by asking the OS to destroy OS-specific resource associated with the window (e.g. Win32 window handle) */
void FLinuxWindow::Destroy()
{
	if (HWnd)
	{
		OwningApplication->RemoveRevertFocusWindow( HWnd );
		OwningApplication->RemoveEventWindow( HWnd );
		OwningApplication->RemoveNotificationWindow( HWnd );

		UE_LOGF(LogLinuxWindow, Verbose, "Requesting SDL_DestroyWindow(%p) once RHI surface refs are released", HWnd);

		// Gate the RHI thread off the surface immediately; the actual SDL_DestroyWindow waits
		// until the RHI releases its last surface ref (or a backstop elapses). The gate stops
		// *new* present work; the refcount tracks *existing* surface ownership.
		SDL_SetBooleanProperty(SDL_GetWindowProperties(HWnd), "UE.window.present_enabled", false);

		// Hit test callback can fire during wl_display_roundtrip on other windows; clear
		// it before the FLinuxWindow goes away.
		SDL_SetWindowHitTest(HWnd, nullptr, nullptr);

		FLinuxPlatformApplicationMisc::DeferDestroyWindow(HWnd);
		HWnd = nullptr;
	}
}

/** Native window should implement this function by performing the equivalent of the Win32 minimize-to-taskbar operation */
void FLinuxWindow::Minimize()
{
	SDL_MinimizeWindow( HWnd );
}

/** Native window should implement this function by performing the equivalent of the Win32 maximize operation */
void FLinuxWindow::Maximize()
{
	SDL_MaximizeWindow( HWnd );
}

/** Native window should implement this function by performing the equivalent of the Win32 maximize operation */
void FLinuxWindow::Restore()
{
	// X11 WMs can auto-promote a borderless window that fills the monitor into _NET_WM_STATE_FULLSCREEN.
	// SDL_RestoreWindow is a no-op in that state, so leave fullscreen explicitly.
	if (SDL_GetWindowFlags(HWnd) & SDL_WINDOW_FULLSCREEN)
	{
		SDL_SetWindowFullscreen(HWnd, false);
	}
	else
	{
		SDL_RestoreWindow(HWnd);
	}
}

/** Show the native window.
 *
 *  Source of truth for visibility is SDL_WINDOW_HIDDEN, not a UE-side flag -- it stays in
 *  sync with the deferred-hide queue automatically. Three cases:
 *    - Hidden  -> SDL_ShowWindow + request swapchain recreate (xdg role lost during hide).
 *    - Pending hide not yet fired -> cancel the queue entry; surface state intact.
 *    - Already shown -> no-op.
 */
void FLinuxWindow::Show()
{
	if ( IsMinimized() )
	{
		Restore();
	}

	if (!HWnd)
	{
		bIsVisible = true;
		return;
	}

	FLinuxPlatformApplicationMisc::CancelDeferredHideWindow(HWnd);

	if ((SDL_GetWindowFlags(HWnd) & SDL_WINDOW_HIDDEN) != 0)
	{
		SDL_ShowWindow(HWnd);
		// Force a swapchain recreate; Vulkan WSI's OUT_OF_DATE detection across an xdg
		// role destroy/recreate isn't reliable across vendor stacks.
		SDL_SetBooleanProperty(SDL_GetWindowProperties(HWnd), "UE.window.swapchain_needs_recreate", true);
	}

	SDL_SetBooleanProperty(SDL_GetWindowProperties(HWnd), "UE.window.present_enabled", true);

	// Vulkan owns surface commits after first Show; keep SDL out of the buffer detach path.
	if (FLinuxPlatformApplicationMisc::IsUsingWayland())
	{
		SDL_SetBooleanProperty(SDL_GetWindowProperties(HWnd), "UE.wayland.skip_surface_detach", true);
	}

	bIsVisible = true;
}

/** Hide the native window.
 *
 *  SDL_HideWindow destroys the Wayland xdg role; running it synchronously would race the
 *  RHI thread mid-present. Instead: gate RHI off via UE.window.present_enabled, then queue
 *  the SDL_HideWindow for 3 PumpMessages ticks (drain time for in-flight presents). Slate
 *  treats Hide as eventual-consistency.
 */
void FLinuxWindow::Hide()
{
	if (!HWnd)
	{
		return;
	}

	if (SDL_GetWindowFlags(HWnd) & SDL_WINDOW_HIDDEN)
	{
		bIsVisible = false;
		return;
	}

	SDL_SetBooleanProperty(SDL_GetWindowProperties(HWnd), "UE.window.present_enabled", false);
	FLinuxPlatformApplicationMisc::DeferHideWindow(HWnd);

	bIsVisible = false;
}


static void GetBestFullscreenResolution( SDL_HWindow hWnd, int32 *pWidth, int32 *pHeight )
{
	uint32 InitializedMode = false;
	uint32 BestWidth = 0;
	uint32 BestHeight = 0;
	uint32 ModeIndex = 0;

	int32 DisplayIndex = SDL_GetDisplayForWindow( hWnd );
	if ( DisplayIndex < 0 )
	{	
		DisplayIndex = 0;
	}

	int NumModes = 0;
	SDL_DisplayMode** Modes = SDL_GetFullscreenDisplayModes(DisplayIndex, &NumModes);
	if (Modes)
	{
		for (int i = 0; i < NumModes; ++i)
		{
			SDL_DisplayMode* Mode = Modes[i];
			bool IsEqualOrBetterWidth  = FMath::Abs((int32)Mode->w - (int32)(*pWidth))  <= FMath::Abs((int32)BestWidth  - (int32)(*pWidth ));
			bool IsEqualOrBetterHeight = FMath::Abs((int32)Mode->h - (int32)(*pHeight)) <= FMath::Abs((int32)BestHeight - (int32)(*pHeight));
			if (!InitializedMode || (IsEqualOrBetterWidth && IsEqualOrBetterHeight))
			{
				BestWidth = Mode->w;
				BestHeight = Mode->h;
				InitializedMode = true;
			}

			
		}
		SDL_free(Modes);
	}

	check(InitializedMode);

	*pWidth  = BestWidth;
	*pHeight = BestHeight;
}

void FLinuxWindow::ReshapeWindow( int32 NewX, int32 NewY, int32 NewWidth, int32 NewHeight )
{
	// If we have set our self to 0,0 Width/Height it will not be allowed we will still show the window
	// this is a work around to at least reduce the visibile impact of a window that is lingering
	NewWidth  = FMath::Max(NewWidth, 1);
	NewHeight = FMath::Max(NewHeight, 1);

	// X11 will take until the next frame to send a SizeChanged event. This means the X11 window
	// will most likely have resized already by the time we render but the slate renderer will
	// not have been updated leading to an incorrect frame.
	//
	// For now tell the owning application we are going to be this size. When the SizeChanged
	// event comes through for X11 it'll confirm our size is the request one or resize to what
	// the WM has forced as the size.
	TSharedPtr< FLinuxWindow > LinuxWindow = OwningApplication->FindWindowBySDLWindow(HWnd);
	if ( LinuxWindow )
	{
		OwningApplication->GetMessageHandler()->OnResizingWindow( LinuxWindow.ToSharedRef() );
	}

	switch( WindowMode )
	{
		// Fullscreen and WindowedFullscreen both use SDL_WINDOW_FULLSCREEN_DESKTOP now
		//  and code elsewhere handles the backbufer blit properly. This solves several
		//  problems that actual mode switches cause, and a GPU scales better than LCD display.
		// If this is changed, change SetWindowMode() and FSystemResolution::RequestResolutionChange() as well.
		case EWindowMode::Fullscreen:
		case EWindowMode::WindowedFullscreen:
		{
			// these are asynchronous calls now, so we need to sync to ensure they've completed
			SDL_SetWindowFullscreen( HWnd, true );
			SDL_SyncWindow( HWnd );
			bWasFullscreen = true;
		}
		break;

		case EWindowMode::Windowed:
		{
			// NOTE: For SDL parent-relative windows, this modifies NewX and NewY!
			ConvertToParentRelativePosition(HWnd, NewX, NewY);

			if (UE_USING_BORDERS_AWARE_SDL == 0 && Definition->HasOSWindowBorder)
			{
				// we are passed coordinates of a client area, so account for decorations
				checkf(bValidNativePropertiesCache, TEXT("Attempted to use border sizes too early, native properties aren't yet cached. Review the flow"));
				NewX -= LeftBorderWidth;
				NewY -= TopBorderHeight;
			}
			// On Wayland, don't set position for toplevel windows -- the compositor manages placement.
			// Setting it would poison SDL's cached position used for popup offset calculations.
			const bool bIsWaylandToplevel = FLinuxPlatformApplicationMisc::IsUsingWayland() &&
				!(SDL_GetWindowFlags(HWnd) & (SDL_WINDOW_POPUP_MENU | SDL_WINDOW_TOOLTIP));

			// Apply the size before setting the position. A Wayland popup's position is derived by the
			// compositor from an xdg_positioner that SDL builds using the popup's current size, so a
			// stale size positions it for the wrong dimensions -- the popup appears to grow without
			// moving and only snaps into place once the size catches up (notably on Mutter).
			SDL_SetWindowSize( HWnd, NewWidth, NewHeight );

			// Only the Wayland popup case needs the size flushed before the position: the reposition's
			// positioner reads the now-current size. X11 re-applies position inside its own resize path
			// and Wayland toplevels don't set a position here, so neither needs this extra round-trip
			// (avoids a second blocking SDL_SyncWindow on the per-frame notification reshape).
			const bool bIsWaylandPopup = FLinuxPlatformApplicationMisc::IsUsingWayland() && !bIsWaylandToplevel;
			if (bIsWaylandPopup)
			{
				SDL_SyncWindow( HWnd );
			}

			if (!bIsWaylandToplevel)
			{
				SDL_SetWindowPosition( HWnd, NewX, NewY );
			}
			else
			{
				// SWindow::ReshapeWindow speculatively set Slate's cached screen position to
				// NewX/NewY, but on Wayland the compositor owns toplevel placement and SDL's
				// global-mouse implementation returns surface-local coords plus window origin
				// (which is (0,0) here). Slate's cache must match or IsScreenspaceMouseWithin
				// will reject every pointer event that lands on the window.
				if (LinuxWindow)
				{
					OwningApplication->GetMessageHandler()->OnMovedWindow(LinuxWindow.ToSharedRef(), 0, 0);
				}
			}
			SDL_SetWindowFullscreen(HWnd, false);
			SDL_SyncWindow( HWnd );

			bWasFullscreen = false;

		}
		break;
	}

	RegionWidth   = NewWidth;
	RegionHeight  = NewHeight;
	VirtualWidth  = NewWidth;
	VirtualHeight = NewHeight;

	// Avoid broadcasting we have set a zero size as it will attempt to resize the backbuffer which on some RHI is invalid per the spec (ie. Vulkan)
	// OnSizeChanged will be called already since SDL_EVENT_WINDOW_RESIZED is now always sent even when SDL_SetWindowSize() is called
	// on SDL3
}

/** Toggle native window between fullscreen and normal mode */
void FLinuxWindow::SetWindowMode( EWindowMode::Type NewWindowMode )
{
	if( NewWindowMode != WindowMode )
	{
		switch( NewWindowMode )
		{
			// Fullscreen and WindowedFullscreen both use SDL_WINDOW_FULLSCREEN_DESKTOP now
			//  and code elsewhere handles the backbufer blit properly. This solves several
			//  problems that actual mode switches cause, and a GPU scales better than LCD display.
			// If this is changed, change ReshapeWindow() and FSystemResolution::RequestResolutionChange() as well.
			case EWindowMode::Fullscreen:
			case EWindowMode::WindowedFullscreen:
			{
				if ( bWasFullscreen != true )
				{
					TSharedPtr< FLinuxWindow > LinuxWindow = OwningApplication->FindWindowBySDLWindow(HWnd);
					if ( LinuxWindow )
					{
						OwningApplication->GetMessageHandler()->OnResizingWindow( LinuxWindow.ToSharedRef() );
					}

					SDL_SetWindowFullscreen( HWnd, true );
					SDL_SyncWindow( HWnd );
					bWasFullscreen = true;

					if ( LinuxWindow )
					{
						OwningApplication->GetMessageHandler()->OnSizeChanged(
							LinuxWindow.ToSharedRef(),
							VirtualWidth,
							VirtualHeight,
							//  bWasMinimized
							false
						);
					}
				}
			}
			break;

			case EWindowMode::Windowed:
			{
				// when going back to windowed from desktop, make window smaller (but not too small),
				// since some too smart window managers (Compiz) will maximize the window if it's set to desktop size.
				// @FIXME: [RCL] 2015-02-10: this is a hack.
				int SmallerWidth = FMath::Max(100, VirtualWidth - 100);
				int SmallerHeight = FMath::Max(100, VirtualHeight - 100);
				SDL_SetWindowSize(HWnd, SmallerWidth, SmallerHeight);

				SDL_SetWindowFullscreen( HWnd, false );
				SDL_SetWindowBordered( HWnd, true );

				SDL_SetWindowMouseGrab( HWnd, false );

				SDL_SyncWindow( HWnd );

				bWasFullscreen = false;
			}
			break;
		}


		WindowMode = NewWindowMode;
	}
}


/** @return	Gives the native window a chance to adjust our stored window size before we cache it off */

void FLinuxWindow::AdjustCachedSize( FVector2D& Size ) const
{
	if	( Definition.IsValid() && Definition->SizeWillChangeOften )
	{
		Size = FVector2D( VirtualWidth, VirtualHeight );
	}
	else if	( HWnd )
	{
		int SizeW, SizeH;

		SDL_GetWindowSize( HWnd, &SizeW, &SizeH );

		/*
		 * Currently we are not correctly supporting up-scaling on all RHIs. For now disable this
		 * until all RHIs are working with up-scaling
		 *
		if ( WindowMode == EWindowMode::Windowed )
		{
			SDL_GetWindowSize( HWnd, &SizeW, &SizeH );
		}
		else // windowed fullscreen or fullscreen
		{
			SizeW = VirtualWidth ;
			SizeH = VirtualHeight;

			GetBestFullscreenResolution( HWnd, &SizeW, &SizeH );
		}
		*/

		Size = FVector2D( SizeW, SizeH );
	}
}

bool FLinuxWindow::GetFullScreenInfo( int32& X, int32& Y, int32& Width, int32& Height ) const
{
	SDL_Rect DisplayRect;

	int DisplayIdx = SDL_GetDisplayForWindow(HWnd);
	if (DisplayIdx >= 0 && SDL_GetDisplayBounds(DisplayIdx, &DisplayRect))
	{
		X = DisplayRect.x;
		Y = DisplayRect.y;
		Width = DisplayRect.w;
		Height = DisplayRect.h;

		return true;
	}

	return false;
}

/** @return true if the native window is maximized, false otherwise */
bool FLinuxWindow::IsMaximized() const
{
	// Treat WM-promoted fullscreen as maximized so title-bar restore/maximize toggles work.
	const SDL_WindowFlags Flags = SDL_GetWindowFlags(HWnd);
	return (Flags & (SDL_WINDOW_MAXIMIZED | SDL_WINDOW_FULLSCREEN)) != 0;
}

/** @return true if the native window is minimized, false otherwise */
bool FLinuxWindow::IsMinimized() const
{
	return SDL_GetWindowFlags(HWnd) & SDL_WINDOW_MINIMIZED || SDL_GetWindowFlags(HWnd) & SDL_WINDOW_HIDDEN;
}

/** @return true if the native window is visible, false otherwise */
bool FLinuxWindow::IsVisible() const
{
	return bIsVisible;
}

/** Returns the size and location of the window when it is restored */
bool FLinuxWindow::GetRestoredDimensions(int32& X, int32& Y, int32& Width, int32& Height)
{
	SDL_GetWindowPosition(HWnd, &X, &Y);
	SDL_GetWindowSize(HWnd, &Width, &Height);

	return true;
}

/** Sets focus on the native window */
void FLinuxWindow::SetWindowFocus()
{
	// Setting focus here is troublesome at least when running on X11, since unlike other platforms
	// it is asynchronous and SetWindowFocus() may happen at an inappropriate time (e.g. on an unmapped window).
	// Instead of allowing this silently fail this is abstracted away from Slate, actual focus change will happen later (when handling window messages).
	// @todo: should we queue the focus change if this function gets called long after the window creation?
//	SDL_SetWindowInputFocus( HWnd );
}

/**
 * Sets the opacity of this window
 *
 * @param	InOpacity	The new window opacity represented as a floating point scalar
 */
void FLinuxWindow::SetOpacity( const float InOpacity )
{
	SDL_SetWindowOpacity(HWnd, InOpacity);
}

/**
 * Enables or disables this window.  When disabled, a window will receive no input.       
 *
 * @param bEnable	true to enable the window, false to disable it.
 */
void FLinuxWindow::Enable( bool bEnable )
{
	// Different WMs handle this in different way.
	// TODO: figure out if ignoring this causes problems for Slate
}

/** @return true if native window exists underneath the coordinates */
bool FLinuxWindow::IsPointInWindow( int32 X, int32 Y ) const
{
	int32 width = 0, height = 0;

	SDL_GetWindowSize( HWnd, &width, &height );
	
	return X >= 0 && Y >= 0 && X < width && Y < height;
}

int32 FLinuxWindow::GetWindowBorderSize() const
{
	return 0;
}

bool FLinuxWindow::IsForegroundWindow() const
{
	if (OwningApplication->GetCurrentActiveWindow().IsValid())
	{
		return OwningApplication->GetCurrentActiveWindow()->GetHWnd() == HWnd;
	}
	else
	{
		return false;
	}
}

void FLinuxWindow::SetText( const TCHAR* const Text )
{
	SDL_SetWindowTitle( HWnd, TCHAR_TO_ANSI(Text));
}

bool FLinuxWindow::IsRegularWindow() const
{
	return Definition->IsRegularWindow;
}

bool FLinuxWindow::IsPopupMenuWindow() const
{
	return bIsPopupWindow;
}

bool FLinuxWindow::IsTooltipWindow() const
{
	return bIsTooltipWindow;
}

bool FLinuxWindow::IsNotificationWindow() const
{
	return bIsNotificationWindow;
}

bool FLinuxWindow::IsTopLevelWindow() const
{
	return bIsTopLevelWindow;
}

bool FLinuxWindow::IsModalWindow() const
{
	return Definition->IsModalWindow;
}

bool FLinuxWindow::IsDialogWindow() const
{
	return bIsDialogWindow;
}

bool FLinuxWindow::IsDragAndDropWindow() const
{
	return bIsDragAndDropWindow;
}

bool FLinuxWindow::IsUtilityWindow() const
{
	return bIsUtilityWindow;
}

EWindowActivationPolicy FLinuxWindow::GetActivationPolicy() const
{
	return Definition->ActivationPolicy;
}

bool FLinuxWindow::IsFocusWhenFirstShown() const
{
	return Definition->FocusWhenFirstShown;
}

uint32 FLinuxWindow::GetID() const
{
	return WindowSDLID;
}

void FLinuxWindow::LogInfo() 
{
	UE_LOGF(LogLinuxWindowType, Verbose, "---------- Windows ID: %d Properties -----------)", GetID());
	UE_LOGF(LogLinuxWindowType, Verbose, "InParent: %d Parent Window ID: %d", ParentWindow.IsValid(), ParentWindow.IsValid() ? ParentWindow->GetID() : -1);
	UE_LOGF(LogLinuxWindowType, Verbose, "HasOSWindowBorder: %d", Definition->HasOSWindowBorder);
	UE_LOGF(LogLinuxWindowType, Verbose, "IsTopmostWindow: %d", Definition->IsTopmostWindow);
	UE_LOGF(LogLinuxWindowType, Verbose, "HasSizingFrame: %d", Definition->HasSizingFrame);
	UE_LOGF(LogLinuxWindowType, Verbose, "AppearsInTaskbar: %d", Definition->AppearsInTaskbar);
	UE_LOGF(LogLinuxWindowType, Verbose, "AcceptsInput: %d", Definition->AcceptsInput);
	UE_LOGF(LogLinuxWindowType, Verbose, "IsModalWindow: %d", Definition->IsModalWindow);
	UE_LOGF(LogLinuxWindowType, Verbose, "IsRegularWindow: %d", Definition->IsRegularWindow);
	UE_LOGF(LogLinuxWindowType, Verbose, "ActivationPolicy: %d", (int)Definition->ActivationPolicy);
	UE_LOGF(LogLinuxWindowType, Verbose, "FocusWhenFirstShown: %d", Definition->FocusWhenFirstShown);
	UE_LOGF(LogLinuxWindowType, Verbose, "SizeWillChangeOften: %d", Definition->SizeWillChangeOften);
}

const TSharedPtr< FLinuxWindow >& FLinuxWindow::GetParent() const
{
	return ParentWindow;
}

void FLinuxWindow::GetNativeBordersSize(int32& OutLeftBorderWidth, int32& OutTopBorderHeight) const
{
	checkf(bValidNativePropertiesCache, TEXT("Attempted to get border sizes too early, native properties aren't yet cached. Review the flow"));
	OutLeftBorderWidth = LeftBorderWidth;
	OutTopBorderHeight = TopBorderHeight;
}

void FLinuxWindow::CacheNativeProperties()
{
	// cache border sizes
	int Top, Left;
	if (SDL_GetWindowBordersSize(HWnd, &Top, &Left, nullptr, nullptr) == 0)
	{
		LeftBorderWidth = Left;
		TopBorderHeight = Top;
	}

	bValidNativePropertiesCache = true;
}

#undef LOCTEXT_NAMESPACE

