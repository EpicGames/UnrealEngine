// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_ACCESSIBILITY
#include "Mac/Accessibility/CocoaAccessibilityView.h"
#include "Mac/Accessibility/MacAccessibilityManager.h"
#include "Mac/Accessibility/MacAccessibilityElement.h"

@implementation FCocoaAccessibilityView
@synthesize AccessibleWindow;

- (id)initWithFrame: (NSRect) Frame
{
	if(self = [super initWithFrame:Frame])
	{
		//init code goes here
	}
	return self;
}

- (void)dealloc
{
	self.AccessibleWindow = Nil;
	[super dealloc];
}

- (void)SetAccessibilityWindowAsAccessibilityChild:(AccessibleWidgetId)InWindowId
{
	checkf([NSThread isMainThread], TEXT("Updating accessibility view in FCocoaWindow from wrong Thread! Accessibility can only be done on Main Thread!"));
	checkf(InWindowId != IAccessibleWidget::InvalidAccessibleWidgetId, TEXT("Cannot update view with invalid accessible widget Id! use RemoveAccessibilityWindow if trying to remove window."));
	FMacAccessibilityElement* InAccessibleWindow = [[FMacAccessibilityManager AccessibilityManager]GetAccessibilityElement:InWindowId];
	check(InAccessibleWindow != Nil);
	//This ensures  that accessibility parenting is done properly and gives VoiceOVer the hierarchy it wants
	// app > window (the NSView) > accessible elements
	self.AccessibleWindow.accessibilityParent = self;
	self.AccessibleWindow = InAccessibleWindow;
}

- (void)RemoveAccessibilityWindow
{
	checkf([NSThread isMainThread], TEXT("Updating accessibility view in FCocoaWindow from wrong Thread! Accessibility can only be done on Main Thread!"));
	if(self.AccessibleWindow != Nil)
	{
		self.AccessibleWindow.accessibilityParent = Nil;
		self.AccessibleWindow = Nil;
	}
}

- (BOOL)isAccessibilityElement
{
	//we don't want the view to be interactable
	return NO;
}

- (NSArray*)accessibilityChildren
{
	if(self.AccessibleWindow != Nil)
	{
		return self.AccessibleWindow.accessibilityChildren;
	}
	return Nil;
}

- (id)accessibilityFocusedUIElement
{
	return self.AccessibleWindow.accessibilityFocusedUIElement;
}

//@TODO: Implement hit testing properly for the window. Not sure if this is necessary though
//- (id)accessibilityHitTest:(NSPoint) InPoint
//{
//	return [self.AccessibleWindow accessibilityHitTest: InPoint];
//}

@end

#endif
