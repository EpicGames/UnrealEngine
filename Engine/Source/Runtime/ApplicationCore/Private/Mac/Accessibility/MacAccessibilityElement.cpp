// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_ACCESSIBILITY

#include "Mac/Accessibility/MacAccessibilityElement.h"
#include "Apple/AppleStringUtils.h"
#include "Mac/CocoaThread.h"
#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"
#include "Mac/Accessibility/MacAccessibilityManager.h"
#include "Mac/MacApplication.h"
#include "Mac/CocoaTextView.h"

#import <Foundation/Foundation.h>

@implementation FMacAccessibilityElement

@synthesize Id;
@synthesize ParentId;
@synthesize OwningWindowId;
@synthesize ChildIds;
@synthesize Bounds;
@synthesize Label;
@synthesize Help;
@synthesize Value;
@synthesize LastCachedStringTime;

- (id)initWithId:(AccessibleWidgetId)InId
{
	// Creating FMacAccessibilityElements shoulld only be done in MainThread
	check([NSThread isMainThread]);
	self = [super init];
	if(self == Nil)
	{
		return self;
	}
	
	self.Id = InId;
	GameThreadCall(^{
		TSharedPtr<IAccessibleWidget> Widget = [FMacAccessibilityManager AccessibilityManager].MacApplication->GetAccessibleMessageHandler()->GetAccessibleWidgetFromId(InId);
		if (Widget.IsValid())
		{
			FString InitialLabel = Widget->GetWidgetName();
			FString InitialHelp = Widget->GetHelpText();
			FVariant InitialValue;
			//@TODO: This ensures accessibility doesn't read out a person's password by giving an empty value. Do it properly
			if (Widget->AsProperty() && !Widget->AsProperty()->IsPassword())
			{
				InitialValue = Widget->AsProperty()->GetValueAsVariant();
			}
			AccessibleWidgetId InitialParentId = IAccessibleWidget::InvalidAccessibleWidgetId;
			TSharedPtr<IAccessibleWidget> InitialParent = Widget->GetParent();
			if(InitialParent.IsValid())
			{
				InitialParentId = InitialParent->GetId();
			}
			AccessibleWidgetId InitialOwningWindowId = IAccessibleWidget::InvalidAccessibleWidgetId;
			TSharedPtr<IAccessibleWidget> OwningWindow = Widget->GetWindow();
			if(OwningWindow.IsValid())
			{
				InitialOwningWindowId = OwningWindow->GetId();
			}
			
			EAccessibleWidgetType WidgetType = Widget->GetWidgetType();
			MainThreadCall(^{
				FMacAccessibilityElement* Self = [[FMacAccessibilityManager AccessibilityManager] GetAccessibilityElement:InId];
				Self.Label = InitialLabel;
				Self.Help = InitialHelp;
				Self.Value = InitialValue;
				Self.LastCachedStringTime = FPlatformTime::Seconds();
				Self.ParentId = InitialParentId;
				if(InitialParentId != IAccessibleWidget::InvalidAccessibleWidgetId)
				{
					self.accessibilityParent = [[FMacAccessibilityManager AccessibilityManager] GetAccessibilityElement:InitialParentId];
				}
				self.OwningWindowId = InitialOwningWindowId;
				if(InitialOwningWindowId != IAccessibleWidget::InvalidAccessibleWidgetId)
				{
					self.accessibilityWindow = [[FMacAccessibilityManager AccessibilityManager] GetAccessibilityElement:InitialOwningWindowId];
				}
				
				// The role and Subrole of an NSElement basically never changes, so set it here
				//The Role and SubRoles together change the role descriptions for Voiceover to announce
				//MOst of Slate Widgets don't need to support subroles so set to Nil here .
				self.accessibilitySubrole = Nil;
				switch(WidgetType)
				{
					case EAccessibleWidgetType::Button:
					{
						_AccessibilityRole = NSAccessibilityButtonRole;
						break;
					}
					case EAccessibleWidgetType::CheckBox:
					{
						_AccessibilityRole = NSAccessibilityCheckBoxRole;
						break;
					}
					case EAccessibleWidgetType::Image:
					{
						_AccessibilityRole = NSAccessibilityImageRole;
						break;
					}
					case EAccessibleWidgetType::Slider:
					{
						_AccessibilityRole = NSAccessibilitySliderRole;
						break;
					}
					case EAccessibleWidgetType::Text:
					{
						_AccessibilityRole = NSAccessibilityStaticTextRole;
						break;
					}
					case EAccessibleWidgetType::TextEdit:
					{
						if(Widget->AsProperty()->IsReadOnly())
						{
							_AccessibilityRole = NSAccessibilityStaticTextRole;
						}
						else
						{
							_AccessibilityRole = NSAccessibilityTextFieldRole;
							if(Widget->AsProperty()->IsPassword())
							{
								self.accessibilitySubrole = NSAccessibilitySecureTextFieldSubrole;
							}
						}
						break;
					}
					case EAccessibleWidgetType::Window:
					{
						_AccessibilityRole = NSAccessibilityWindowRole;
						break;
					}
					case EAccessibleWidgetType::Hyperlink:
					{
						_AccessibilityRole = NSAccessibilityLinkRole;
						self.accessibilitySubrole = NSAccessibilityTextLinkSubrole;
						break;
					}
					case EAccessibleWidgetType::Layout:
					{
						_AccessibilityRole = NSAccessibilityLayoutAreaRole;
						break;
					}
					case EAccessibleWidgetType::ComboBox:
					{
						_AccessibilityRole = NSAccessibilityComboBoxRole;
						break;
					}
					default:
					{
						//we use the NSAccessibilityGroupRole to forward VOiceover focus to children
						//NSAccessibilityUnknownRole will intercept Voiceover Focus
						//_AccessibilityRole = NSAccessibilityGroupRole;
						_AccessibilityRole = NSAccessibilityUnknownRole;
						break;
					}
				} // switch WidgetType
				self.accessibilityRole = _AccessibilityRole;
			}, false);
		} // if widget valid
	}, false);
	
	return self;
}

- (void)dealloc
{
	// Destroying of elements should only happen in Main Thread
	check([NSThread isMainThread]);
	if(_AccessibilityChildren != Nil)
	{
		[_AccessibilityChildren release];
	}
	NSAccessibilityPostNotification(self, NSAccessibilityUIElementDestroyedNotification);
	[super dealloc];
}

- (bool)ShouldCacheStrings
{
	//@TODO: Magic number could be changed dynamically to suit responsiveness and performance
	return FPlatformTime::Seconds() - LastCachedStringTime > 0.5;
}

- (void)CacheStrings
{
	// Updating of elements should only be done in Main Thread
	check([NSThread isMainThread]);
	if(![self ShouldCacheStrings])
	{
		return;
	}
	const AccessibleWidgetId TempId = Id;
	GameThreadCall(^{
		TSharedPtr<IAccessibleWidget> Widget = [FMacAccessibilityManager AccessibilityManager].MacApplication->GetAccessibleMessageHandler()->GetAccessibleWidgetFromId(TempId);
		if(Widget.IsValid())
		{
			FString TempLabel = Widget->GetWidgetName();
			FString TempHelp = Widget->GetHelpText();
			FVariant TempValue;
			//@TODO: Same as above.
			if (Widget->AsProperty() && !Widget->AsProperty()->IsPassword())
			{
				TempValue = Widget->AsProperty()->GetValueAsVariant();
			}
			MainThreadCall(^{
				FMacAccessibilityElement* Self = [[FMacAccessibilityManager AccessibilityManager] GetAccessibilityElement:TempId];
				Self.Label = TempLabel;
				Self.Help = TempHelp;
				Self.Value = TempValue;
				Self.LastCachedStringTime = FPlatformTime::Seconds();
			}, false);
		} // if widget valid
	}, false);
}

- (void)ExposeToVoiceOver:(bool)bShouldExpose;
{
	// Editing of accessibility values should only happen in Main Thread
	check([NSThread isMainThread]);
	//Setting accessibilityRole to @"" makes VO ignore the element. It's a way to remove invisible elements from the accessibility tree. Not documented by Apple, so this may go away...
	// https://stackoverflow.com/questions/31523333/disable-hide-accessibility-element
	self.accessibilityRole = bShouldExpose ? _AccessibilityRole : @"";
}

- (BOOL)accessibilityNotifiesWhenDestroyed
{
	//this allows posting of accessibility notifications
	return YES;
}

- (NSRect)accessibilityFrame
{
	//Bounds max and min are slate screen coordinates converted into cocoa screen coordinates
	// Slate's Coordinate System is origin bottom left,+X right, +Y up
	//Cocoa's Coordinate System is origin top left, +X right, +Y down
	//Thus BOunds is not a valid AABB by cocoa coordinate space. We fix that here
	return 	NSMakeRect(Bounds.Min.X, Bounds.Max.Y, Bounds.Max.X - Bounds.Min.X, Bounds.Min.Y - Bounds.Max.Y);
}

-(BOOL) isAccessibilityElement
{
	//this affects the element's exposure to the trackpad commander for Voiceover
	return YES;
}

- (BOOL)isAccessibilityFocused
{
	//just check if the application focused element is this element
	id ApplicationFocusedElement = NSApp.accessibilityApplicationFocusedUIElement;
	if ([ApplicationFocusedElement isKindOfClass:[FMacAccessibilityElement class]])
	{
		FMacAccessibilityElement* MacElement = (FMacAccessibilityElement*) ApplicationFocusedElement;
		return self.Id == MacElement.Id;
	}
	return NO;
}

- (void)setAccessibilityFocused:(BOOL)InIsFocused
{
	if(InIsFocused == YES)
	{
		// THis somehow contributes towards the ability to type in text fiellds...
		//note that this holds a strong reference to the element and so the element won't be automatically released 
		NSApp.accessibilityApplicationFocusedUIElement = self;
		const AccessibleWidgetId TempId = self.Id;
		GameThreadCall(^(){
			TSharedPtr<IAccessibleWidget> Widget = [FMacAccessibilityManager AccessibilityManager].MacApplication->GetAccessibleMessageHandler()->GetAccessibleWidgetFromId(TempId);
			if (Widget.IsValid() && Widget->IsEnabled() && !Widget->IsHidden())
			{
				// Voiceover only recognizes 1 user, the primary accessible user.
				Widget->SetUserFocus(FGenericAccessibleUserRegistry::GetPrimaryUserIndex());
			}
		}, false, @[ NSDefaultRunLoopMode, UnrealIMEEventMode, UnrealShowEventMode, UnrealResizeEventMode, UnrealFullscreenEventMode]); // GameThreadCall
	}
	[super setAccessibilityFocused:InIsFocused];
}

- (NSString*)accessibilityLabel
{
	if (!self.Label.IsEmpty())
	{
		return [NSString stringWithFString:self.Label];
	}
	return nil;
}

- (NSString*)accessibilityHelp
{
	if (!self.Help.IsEmpty())
	{
		return [NSString stringWithFString:self.Help];
	}
	return nil;
}

- (id)accessibilityValue
{
	if (Value.IsEmpty())
	{
		return Nil;
	}
	EVariantTypes VariantType = Value.GetType();
	switch(VariantType)
	{
			//for checkboxes 
		case EVariantTypes::Bool:
		{
			return Value.GetValue<bool>() ? @YES : @NO;
		}
			//for sliders
		case EVariantTypes::Float:
		{
			return @(Value.GetValue<float>());
		}
			// for textedits and textboxes etc
		case EVariantTypes::String:
		{
			//Passwords should be covered up
			if(self.accessibilitySubrole == NSAccessibilitySecureTextFieldSubrole)
			{
				//@TODO: Consider just having the accessibility value in IAccessible Property be hidden
				int32 PasswordLength = Value.GetValue<FString>().Len();
				FString HiddenString;
				HiddenString.Reserve(PasswordLength);
				for(uint32 Index = 0; Index < PasswordLength; ++Index)
				{
					//the password black dot character
					HiddenString += TCHAR(0x2202);
				}
				return HiddenString.GetNSString();
			}
			//return [NSString stringWithFString:Value.GetValue<FString>()];
			return Value.GetValue<FString>().GetNSString();
		}
	} // switch VariantType
	return Nil;
}

- (NSString*)accessibilityRoleDescription
{
	return NSAccessibilityRoleDescription(self.accessibilityRole, self.accessibilitySubrole);
}

- (NSArray*)accessibilityChildren
{
	if(ChildIds.Num() == 0)
	{
		return Nil;
	}
	//@TODO: Rather wasteful to clear it everytime it's requested. Probably an optimiazation available.
	if(_AccessibilityChildren == Nil)
	{
		_AccessibilityChildren = [[NSMutableArray alloc] init];
	}
	else
	{
		[_AccessibilityChildren removeAllObjects];
	}
	for( const AccessibleWidgetId ChildId : ChildIds)
	{
		FMacAccessibilityElement* Child = [[FMacAccessibilityManager AccessibilityManager] GetAccessibilityElement:ChildId];
		[_AccessibilityChildren addObject: Child];
	}
	return _AccessibilityChildren;
}

//Text
- (NSInteger)accessibilityNumberOfCharacters
{
	//@TODO
	return 0;
}

- (NSInteger)accessibilityInsertionPointLineNumber
{
	//@TODO
	return 0;
}

- (NSRange)accessibilityVisibleCharacterRange
{
	//@TODO
	return {NSNotFound, 0};;
}

- (NSString*)accessibilitySelectedText
{
	FMacAccessibilityElement* AccessibleWindow = (FMacAccessibilityElement*)self.accessibilityWindow;
	id ParentView = AccessibleWindow.accessibilityParent;
	if(![ParentView isKindOfClass: [FCocoaTextView class]])
	{
		return Nil;
	}
	FCocoaTextView* TextView = (FCocoaTextView*)ParentView;
	NSAttributedString* AttributeString = [TextView attributedSubstringForProposedRange: [TextView selectedRange] actualRange: NULL];
	return [AttributeString string];
}

- (NSRange)accessibilitySelectedTextRange
{
	FMacAccessibilityElement* AccessibleWindow = (FMacAccessibilityElement*)self.accessibilityWindow;
	id ParentView = AccessibleWindow.accessibilityParent;
	if(![ParentView isKindOfClass: [FCocoaTextView class]])
	{
		return {NSNotFound, 0};;
	}
	FCocoaTextView* TextView = (FCocoaTextView*)ParentView;
	return [TextView selectedRange];
}

- (NSInteger)accessibilityLineForIndex:(NSInteger)index
{
	//TODO
	return 0;
}
//Actions
- (BOOL)accessibilityPerformPress
{
	const AccessibleWidgetId TempId = self.Id;
	
	// All IAccessibleWidget functions must be run on Game Thread
	GameThreadCall(^(){
		TSharedPtr<IAccessibleWidget> Widget = [FMacAccessibilityManager AccessibilityManager].MacApplication->GetAccessibleMessageHandler()->GetAccessibleWidgetFromId(TempId);
		if (Widget.IsValid() && Widget->IsEnabled())
		{
			IAccessibleActivatable* Activatable = Widget->AsActivatable();
			if(Activatable)
			{
				Activatable->Activate();
				//if the activatable is checkable, it's a checkbox, post value updated notification
				if(Activatable->IsCheckable())
				{
					MainThreadCall(^{
						FMacAccessibilityElement* Self = [[FMacAccessibilityManager AccessibilityManager] GetAccessibilityElement:TempId];
						NSAccessibilityPostNotification(Self, NSAccessibilityValueChangedNotification);
					}, false);
				}
			}
		} // if widget valid and enabled
	}, false, @[ NSDefaultRunLoopMode, UnrealIMEEventMode, UnrealShowEventMode, UnrealResizeEventMode, UnrealFullscreenEventMode]); // GameThreadCall
	return YES;
}

- (BOOL) accessibilityPerformIncrement
{
	const AccessibleWidgetId TempId = self.Id;
	// All IAccessibleWidget functions must be run on Game Thread
	GameThreadCall(^{
		TSharedPtr<IAccessibleWidget> Widget = [FMacAccessibilityManager AccessibilityManager].MacApplication->GetAccessibleMessageHandler()->GetAccessibleWidgetFromId(TempId);
		if (Widget.IsValid())
		{
			IAccessibleProperty* Property = Widget->AsProperty();
			if (Property && !FMath::IsNearlyZero(Property->GetStepSize()) && !Property->IsReadOnly())
			{
				const float CurrentValue = FCString::Atof(*Property->GetValue());
				float NewValue = CurrentValue + Property->GetStepSize();
				if(NewValue > Property->GetMaximum())
				{
					NewValue = Property->GetMaximum();
				}
				Property->SetValue(FString::SanitizeFloat(NewValue));
				MainThreadCall(^{
					FMacAccessibilityElement* Self = [[FMacAccessibilityManager AccessibilityManager] GetAccessibilityElement:TempId];
					NSAccessibilityPostNotification(Self, NSAccessibilityValueChangedNotification);
				}, false);
			}
		} // widget valid
	}, false);
	return YES;
}

- (BOOL)accessibilityPerformDecrement
{
	const AccessibleWidgetId TempId = self.Id;
	// All IAccessibleWidget functions must be run on Game Thread
	GameThreadCall(^{
		TSharedPtr<IAccessibleWidget> Widget = [FMacAccessibilityManager AccessibilityManager].MacApplication->GetAccessibleMessageHandler()->GetAccessibleWidgetFromId(TempId);
		if (Widget.IsValid())
		{
			IAccessibleProperty* Property = Widget->AsProperty();
			if (Property && !FMath::IsNearlyZero(Property->GetStepSize()) && !Property->IsReadOnly())
			{
				const float CurrentValue = FCString::Atof(*Property->GetValue());
				float NewValue = CurrentValue - Property->GetStepSize();
				if(NewValue < Property->GetMinimum())
				{
					NewValue = Property->GetMinimum();
				}
				Property->SetValue(FString::SanitizeFloat(NewValue));
				MainThreadCall(^{
					FMacAccessibilityElement* Self = [[FMacAccessibilityManager AccessibilityManager] GetAccessibilityElement:TempId];
					NSAccessibilityPostNotification(Self, NSAccessibilityValueChangedNotification);
				}, false);
			}
		} // widget valid
	}, false);
	return YES;
}

//Returns YES if the selector can be called on this accessibility element
- (BOOL)isAccessibilitySelectorAllowed:(SEL)selector
{
	if(selector == @selector(accessibilityValue))
	{
		return _AccessibilityRole == NSAccessibilityCheckBoxRole
		|| _AccessibilityRole == NSAccessibilityTextFieldRole
		|| _AccessibilityRole == NSAccessibilityStaticTextRole
		|| _AccessibilityRole == NSAccessibilitySliderRole;
		
	}
	//Text
	else if(selector == @selector(accessibilityInsertionPointLineNumber))
	{
		return _AccessibilityRole == NSAccessibilityTextFieldRole;
	}
	else if(selector == @selector(accessibilityLineForIndex:))
	{
		return _AccessibilityRole == NSAccessibilityTextFieldRole;
	}
	else if(selector == @selector(accessibilityRangeForLine:))
	{
		return _AccessibilityRole == NSAccessibilityTextFieldRole;
	}
	else if(selector == @selector(accessibilityFrameForRange))
	{
		return _AccessibilityRole == NSAccessibilityTextFieldRole;
	}
	//Actions
	else if(selector == @selector(accessibilityPerformPress))
	{
		return _AccessibilityRole == NSAccessibilityButtonRole
		|| _AccessibilityRole == NSAccessibilityCheckBoxRole;
	}
	else if (selector == @selector(accessibilityPerformDecrement))
	{
		return _AccessibilityRole == NSAccessibilitySliderRole;
	}
	else if (selector == @selector(accessibilityPerformIncrement))
	{
		return _AccessibilityRole == NSAccessibilitySliderRole;
	}
	
	return YES;
}
@end

#endif 
