// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_ACCESSIBILITY

#include "Mac/Accessibility/MacAccessibilityManager.h"
#include "Mac/Accessibility/MacAccessibilityElement.h"
#include "Mac/MacApplication.h"
#include "HAL/IConsoleManager.h"
#include "Mac/CocoaThread.h"
#include "Apple/AppleStringUtils.h"

#import <AppKit/AppKit.h>

//used for registering for NSWorkspace's KVO notifications when Voiceover is turned on
static void* MacAccessibilityManagerContext = &MacAccessibilityManagerContext;

@implementation FMacAccessibilityManager
@synthesize MacApplication;

#if !UE_BUILD_SHIPPING
static void DumpAccessibilityStatsForwarder()
{
	[[FMacAccessibilityManager AccessibilityManager] DumpAccessibilityStats];
}
#endif

- (id)init
{
	// This should only be created in Main Thread
	check([NSThread isMainThread]);
	if(self = [super init])
	{
		Cache = [[NSMutableDictionary alloc] init];
		//register KVO for notifications about changes in  Voiceover status.
		// A notification is received every time Voiceover status changes.
		//@see: observeValueForKeyPath
		//NOTE: Don't use the NSKeyValueObservingOptionInitial option.
		// If VoiceOVer were already enabled before the engine launched,
		//It would send a KVO notification before the FSlateAccessibilityMessageHandler is set in FMacApplication.
		[[NSWorkspace sharedWorkspace] addObserver:self
										forKeyPath: @"voiceOverEnabled"
										   options: NSKeyValueObservingOptionNew
										   context: MacAccessibilityManagerContext];
	}
	return self;
}

- (void)dealloc
{
	[super dealloc];
}

- (void)observeValueForKeyPath:(NSString *)keyPath
					 ofObject:(id)object
					   change:(NSDictionary*)change
					  context:(void*)context
{
	if(context == MacAccessibilityManagerContext)
	{
		if ([keyPath isEqualToString: @"voiceOverEnabled"])
		{
			NSNumber* newValue = [change objectForKey:NSKeyValueChangeNewKey];
			bool bVoiceoverEnabled = [newValue boolValue];
			if(bVoiceoverEnabled)
			{
				//@TODO: This can cause VoiceOVer to hang sometimes. Disabling for now.
				//MacApplication->OnVoiceoverEnabled();
			}
			else
			{
				//@TODO: This causes VoiceOVer to hang when turning it on and off sometimes. Disabling for now
				//This means that the accessibility cache is only cleared on shutdown and will persist even if VoiceOVer is off.
				//MacApplication->OnVoiceoverDisabled();
			}
		} // if keypath
	} //if context
	else
	{
		[super observeValueForKeyPath:keyPath ofObject:object change:change context:context];
	}
}

- (FMacAccessibilityElement*)GetAccessibilityElement:(AccessibleWidgetId)Id
{
	// Getting and creating FMacAccessibilityElements should only be done fromm Main Thread
	check([NSThread isMainThread]);
	if (Id == IAccessibleWidget::InvalidAccessibleWidgetId)
	{
		return nil;
	}
	FMacAccessibilityElement* ExistingElement = [Cache objectForKey:@(Id)];
	if (ExistingElement == nil)
	{
		ExistingElement = [[[FMacAccessibilityElement alloc] initWithId:Id] autorelease];
		[Cache setObject:ExistingElement forKey:@(Id)];
	}
	return ExistingElement;
}

- (bool)AccessibilityElementExists:(AccessibleWidgetId)Id
{
	return [Cache objectForKey:@(Id)] != nil;
}

- (void)RemoveAccessibilityElement:(AccessibleWidgetId)Id
{
	//Modifications to the cache should only be done in ain Thread
	check([NSThread isMainThread]);
	[Cache removeObjectForKey:@(Id)];
}

- (void)RemoveAccessibilitySubtree:(AccessibleWidgetId)RootId
{
	check([NSThread isMainThread]);
	//BFS to remove entire subtree
	TArray<AccessibleWidgetId> Queue;
	Queue.Push(RootId);
	while(Queue.Num() > 0)
	{
		AccessibleWidgetId CurrentId = Queue.Pop(EAllowShrinking::No);
		FMacAccessibilityElement* CurrentElement = [Cache objectForKey:@(RootId)];
		if(CurrentElement.ChildIds.Num() == 0)
		{
			[self RemoveAccessibilityElement:CurrentId];
			continue;
		}
		for(AccessibleWidgetId ChildId : CurrentElement.ChildIds)
		{
			Queue.Push(ChildId);
		}
		[self RemoveAccessibilityElement:CurrentId];
	}
}

- (void)Clear
{
	check([NSThread isMainThread]);
	[Cache removeAllObjects];
}

- (int32)GetAccessibilityCacheSize
{
	return [Cache count];
}

- (bool)IsAccessibilityCacheEmpty
{
	return [Cache count] == 0;
}

- (void)TearDown
{
	//This should only be called in ~FMacApplication() in a Main Thread Call
	check([NSThread isMainThread]);
	checkf([self IsAccessibilityCacheEmpty], TEXT("Accessibility Manager is being torn down and still has %u elements!"), [self GetAccessibilityCacheSize]);
	[Cache release];
	Cache = Nil;
	//unregister KVO for Voiceover changes
	[[NSWorkspace sharedWorkspace] removeObserver:self
									   forKeyPath:@"voiceOverEnabled"];
	//Note: do not set the appllication to null as multithreading could cause the MacApplication destructor to be called before the update timer is destroyed.
}

+ (FMacAccessibilityManager*)AccessibilityManager
{
	static FMacAccessibilityManager* manager = nil;
	if (manager == nil)
	{
		manager = [[self alloc] init];
	}
	return manager;
}

- (void)UpdateAllCachedProperties
{
	//Updating of FMacAccessibilityElements should only be done from Main Thread
	check([NSThread isMainThread]);
	TArray<AccessibleWidgetId> Ids;
	for (NSString* Key in Cache)
	{
		Ids.Add(Key.intValue);
	}
	if(Ids.Num() == 0)
	{
		return;
	}
	//all requests to slate accessible widget cache must be done from game thread
	//Do not update parent here as the Mac Application is supposed to handle a EAccessibleEvent::ParentChanged event to do that
	GameThreadCall(^{
		for (const AccessibleWidgetId Id : Ids)
		{
			TSharedPtr<IAccessibleWidget> Widget = self.MacApplication->GetAccessibleMessageHandler()->GetAccessibleWidgetFromId(Id);
			if (Widget.IsValid())
			{
				// Children
				TArray<int32> ChildIds;
				for (int32 i = 0; i < Widget->GetNumberOfChildren(); ++i)
				{
					TSharedPtr<IAccessibleWidget> Child = Widget->GetChildAt(i);
					if (Child.IsValid())
					{
						ChildIds.Add(Child->GetId());
					}
				}
				// Bounding rect in Slate screen space  .
				//Slate origin is bottom left, +X right, +Y up
				// Cocoa origin is top left, +X right, +Y down
				FBox2D Bounds = Widget->GetBounds();
				
				//NOTE: This is just converting Slate screen coordinates to Cocoa screen coordinates
				//Because Slate and Cocoa have different coordinate systems, Bounds is no longer a valid AABB
				Bounds.Min = FMacApplication::ConvertSlatePositionToCocoa(Bounds.Min.X, Bounds.Min.Y);
				Bounds.Max = FMacApplication::ConvertSlatePositionToCocoa(Bounds.Max.X, Bounds.Max.Y);
				// Visibility
				const bool bIsEnabled = Widget->IsEnabled();
				const bool bIsVisible = !Widget->IsHidden();
				// All AppKit functions must be run on Main Thread
				MainThreadCall(^{
					FMacAccessibilityElement* Element = [[FMacAccessibilityManager AccessibilityManager] GetAccessibilityElement:Id];
					Element.ChildIds = ChildIds;
					Element.Bounds = Bounds;
					Element.accessibilityEnabled = bIsEnabled ? YES : NO;
					//tells Voiceover to ignore elements based on visibility
					[Element ExposeToVoiceOver:bIsVisible];
					//@TODO: Updating the label, help and value of ALL elements is suboptimal...but it works
					//Tested with EngineTest in Debug Editor in the Editor with ~650 elements and still fairly responsive
					[Element CacheStrings];
				});
			} // if Widget valid
		} // for Ids loop
	}, true, @[ NSDefaultRunLoopMode, UnrealIMEEventMode, UnrealShowEventMode, UnrealResizeEventMode, UnrealFullscreenEventMode, UnrealNilEventMode ]);
}

#if !UE_BUILD_SHIPPING
- (void)DumpAccessibilityStats
{
	const uint32 NumElements = [Cache count];
	uint32 SizeOfElement = 0;
	uint32 CacheSize = 0;
	
	NSArray* Keys = [Cache allKeys];
	for (NSString* Key in Keys)
	{
		FMacAccessibilityElement* Element = [Cache objectForKey: Key];
		
		SizeOfElement = malloc_size(Element);
		CacheSize += sizeof(NSString*) + sizeof(FMacAccessibilityElement*) + malloc_size(Key) + SizeOfElement
		+ malloc_size([NSString stringWithFString : Element.Label])
		+ malloc_size([NSString stringWithFString : Element.Help])
		+ malloc_size([NSString stringWithFString : Element.Value]);
		
	}
	
	UE_LOGF(LogAccessibility, Log, "Number of Accessibility Elements: %i", NumElements );
	UE_LOGF(LogAccessibility, Log, "Size of FMacAccessibilityElement: %u", SizeOfElement);
	UE_LOGF(LogAccessibility, Log, "Memory stored in cache: %u kb", CacheSize / 1000);
}

- (void)PrintCache
{
	TArray<AccessibleWidgetId> Ids;
	for (NSString* Key in Cache)
	{
		Ids.Add(Key.intValue);
	}
	if(Ids.Num() == 0)
	{
		UE_LOGF(LogAccessibility, Log, "No Cache is empty.");
		return;
	}
	Ids.Sort();
	for(AccessibleWidgetId Id : Ids)
	{
		[self PrintAccessibilityElementById:Id];
	}
}

- (void)PrintAccessibilityElement: (FMacAccessibilityElement*) Element
{
	UE_LOGF(LogAccessibility, Log, "ID: %i", Element.Id);
	UE_LOGF(LogAccessibility, Log, "Parent ID: %i", Element.ParentId);
	UE_LOGF(LogAccessibility, Log, "Label: %ls", *Element.Label);
	UE_LOGF(LogAccessibility, Log, "Children cOunt: %u", Element.ChildIds.Num());
}

- (void)PrintAccessibilityElementById:(AccessibleWidgetId)Id
{
	FMacAccessibilityElement* Element = [Cache objectForKey:@(Id)];
	if(Element == Nil)
	{
		UE_LOGF(LogAccessibility, Log, "Element with ID %i is not in the cache.", Id);
	}
	[self PrintAccessibilityElement:Element];
}

- (id)GetAccessibilityParent:(FMacAccessibilityElement*)Element
{
	return (Element != Nil) ? Element.accessibilityParent : Nil;
}

- (FMacAccessibilityElement*)GetAccessibilityRoot
{
	return Nil;
}

- (bool)isElementOrphaned:(FMacAccessibilityElement*) Element
{
	return Element.accessibilityParent == Nil;
}
#endif

@end

#endif
