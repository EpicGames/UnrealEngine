// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/CocoaMenu.h"
#include "Mac/MacApplication.h"
#include "Apple/ScopeAutoreleasePool.h"

@implementation FCocoaMenu

- (id)initWithTitle:(NSString *)Title
{
	id Self = [super initWithTitle:Title];
	if (Self)
	{
		bHighlightingKeyEquivalent = false;
	}
	return Self;
}

- (bool)isHighlightingKeyEquivalent
{
	SCOPED_AUTORELEASE_POOL;
	FCocoaMenu* SuperMenu = [[self supermenu] isKindOfClass:[FCocoaMenu class]] ? (FCocoaMenu*)[self supermenu] : nil;
	if ( SuperMenu )
	{
		return [SuperMenu isHighlightingKeyEquivalent];
	}
	else
	{
		return bHighlightingKeyEquivalent;
	}
}

- (bool)highlightKeyEquivalent:(NSEvent *)Event
{
	SCOPED_AUTORELEASE_POOL;
	bHighlightingKeyEquivalent = true;
	bool bHighlighted = [super performKeyEquivalent:Event];
	bHighlightingKeyEquivalent = false;
	return bHighlighted;
}

@end
