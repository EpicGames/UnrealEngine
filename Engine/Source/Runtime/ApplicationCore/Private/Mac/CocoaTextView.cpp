// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/CocoaTextView.h"
#include "Mac/CocoaWindow.h"
#include "Mac/CocoaThread.h"
#include "Apple/ScopeAutoreleasePool.h"

@implementation FCocoaTextView

- (id)initWithFrame:(NSRect)frame
{
	SCOPED_AUTORELEASE_POOL;
	self = [super initWithFrame:frame];
	if (self)
	{
		// Initialization code here.
		IMMContext = nil;
		markedRange = {NSNotFound, 0};
		reallyHandledEvent = false;
	}
	return self;
}

- (bool)imkKeyDown:(NSEvent *)theEvent
{
	if (IMMContext.IsValid())
	{
		SCOPED_AUTORELEASE_POOL;
		reallyHandledEvent = true;
		return [[self inputContext] handleEvent:theEvent] && reallyHandledEvent;
	}
	else
	{
		return false;
	}
}

/**
 * Forward mouse events up to the window rather than through the responder chain.
 */
- (BOOL)acceptsFirstMouse:(NSEvent *)Event
{
	return YES;
}

- (void)mouseDown:(NSEvent *)theEvent
{
	SCOPED_AUTORELEASE_POOL;
	if (IMMContext.IsValid())
	{
		[[self inputContext] handleEvent:theEvent];
	}

	FCocoaWindow* CocoaWindow = [[self window] isKindOfClass:[FCocoaWindow class]] ? (FCocoaWindow*)[self window] : nil;
	if (CocoaWindow)
	{
		[CocoaWindow mouseDown:theEvent];
	}

	[NSApp preventWindowOrdering];
}

- (void)mouseDragged:(NSEvent *)theEvent
{
	if (IMMContext.IsValid())
	{
		SCOPED_AUTORELEASE_POOL;
		[[self inputContext] handleEvent:theEvent];
	}
}

- (void)mouseUp:(NSEvent *)theEvent
{
	SCOPED_AUTORELEASE_POOL;
	if (IMMContext.IsValid())
	{
		[[self inputContext] handleEvent:theEvent];
	}

	FCocoaWindow* CocoaWindow = [[self window] isKindOfClass:[FCocoaWindow class]] ? (FCocoaWindow*)[self window] : nil;
	if (CocoaWindow)
	{
		[CocoaWindow mouseUp:theEvent];
	}
}

- (void)rightMouseDown:(NSEvent*)Event
{
	SCOPED_AUTORELEASE_POOL;
	FCocoaWindow* CocoaWindow = [[self window] isKindOfClass:[FCocoaWindow class]] ? (FCocoaWindow*)[self window] : nil;
	if (CocoaWindow)
	{
		[CocoaWindow rightMouseDown:Event];
	}
	else
	{
		[super rightMouseDown:Event];
	}
}

- (void)otherMouseDown:(NSEvent*)Event
{
	SCOPED_AUTORELEASE_POOL;
	FCocoaWindow* CocoaWindow = [[self window] isKindOfClass:[FCocoaWindow class]] ? (FCocoaWindow*)[self window] : nil;
	if (CocoaWindow)
	{
		[CocoaWindow otherMouseDown:Event];
	}
	else
	{
		[super otherMouseDown:Event];
	}
}

- (void)rightMouseUp:(NSEvent*)Event
{
	SCOPED_AUTORELEASE_POOL;
	FCocoaWindow* CocoaWindow = [[self window] isKindOfClass:[FCocoaWindow class]] ? (FCocoaWindow*)[self window] : nil;
	if (CocoaWindow)
	{
		[CocoaWindow rightMouseUp:Event];
	}
	else
	{
		[super rightMouseUp:Event];
	}
}

- (void)otherMouseUp:(NSEvent*)Event
{
	SCOPED_AUTORELEASE_POOL;
	FCocoaWindow* CocoaWindow = [[self window] isKindOfClass:[FCocoaWindow class]] ? (FCocoaWindow*)[self window] : nil;
	if (CocoaWindow)
	{
		[CocoaWindow otherMouseUp:Event];
	}
	else
	{
		[super otherMouseUp:Event];
	}
}

// Returning YES allows SlateApplication to control if window should be activated on mouse down.
// This is used for drag and drop when we don't want to activate if the mouse cursor is over a draggable item.
- (BOOL)shouldDelayWindowOrderingForEvent:(NSEvent*)Event
{
	return YES;
}

- (void)activateInputMethod:(const TSharedRef<ITextInputMethodContext>&)InContext
{
	SCOPED_AUTORELEASE_POOL;
	if (IMMContext.IsValid())
	{
		[self unmarkText];
		[[self inputContext] deactivate];
		[[self inputContext] discardMarkedText];
	}
	IMMContext = InContext;
	[[self inputContext] activate];
}

- (void)deactivateInputMethod
{
	SCOPED_AUTORELEASE_POOL;
	if (IMMContext.IsValid())
	{
		[self unmarkText];
		IMMContext = NULL;
		[[self inputContext] deactivate];
		[[self inputContext] discardMarkedText];
	}
}

- (bool)isActiveInputMethod:(const TSharedRef<ITextInputMethodContext>&)InContext
{
	return IMMContext == InContext;
}

//@protocol NSTextInputClient
//@required
/* The receiver inserts aString replacing the content specified by replacementRange. aString can be either an NSString or NSAttributedString instance.
 */
- (void)insertText:(id)aString replacementRange:(NSRange)replacementRange
{
	SCOPED_AUTORELEASE_POOL;
	if (IMMContext.IsValid() && [self hasMarkedText])
	{
		__block uint32 SelectionLocation;
		__block uint32 SelectionLength;
		if (replacementRange.location == NSNotFound)
		{
			if (markedRange.location != NSNotFound)
			{
				SelectionLocation = markedRange.location;
				SelectionLength = markedRange.length;
			}
			else
			{
				GameThreadCall(^{
					if (IMMContext.IsValid())
					{
						ITextInputMethodContext::ECaretPosition CaretPosition;
						IMMContext->GetSelectionRange(SelectionLocation, SelectionLength, CaretPosition);
					}
				}, true, @[ NSDefaultRunLoopMode, UnrealIMEEventMode ]);
			}
		}
		else
		{
			SelectionLocation = replacementRange.location;
			SelectionLength = replacementRange.length;
		}

		NSString* TheString;
		if ([aString isKindOfClass:[NSAttributedString class]])
		{
			TheString = [(NSAttributedString*)aString string];
		}
		else
		{
			TheString = (NSString*)aString;
		}

		GameThreadCall(^{
			SCOPED_AUTORELEASE_POOL;
			if (IMMContext.IsValid())
			{
				FString TheFString(TheString);
				IMMContext->SetTextInRange(SelectionLocation, SelectionLength, TheFString);
				IMMContext->SetSelectionRange(SelectionLocation+TheFString.Len(), 0, ITextInputMethodContext::ECaretPosition::Ending);
			}
		}, true, @[ NSDefaultRunLoopMode, UnrealIMEEventMode ]);

		[self unmarkText];
		[[self inputContext] invalidateCharacterCoordinates]; // recentering
	}
	else
	{
		reallyHandledEvent = false;
	}
}

/* The receiver invokes the action specified by aSelector.
 */
- (void)doCommandBySelector:(SEL)aSelector
{
	reallyHandledEvent = false;
}

/* The receiver inserts aString replacing the content specified by replacementRange. aString can be either an NSString or NSAttributedString instance. selectedRange specifies the selection inside the string being inserted; hence, the location is relative to the beginning of aString. When aString is an NSString, the receiver is expected to render the marked text with distinguishing appearance (i.e. NSTextView renders with -markedTextAttributes).
 */
- (void)setMarkedText:(id)aString selectedRange:(NSRange)selectedRange replacementRange:(NSRange)replacementRange
{
	if (IMMContext.IsValid())
	{
		SCOPED_AUTORELEASE_POOL;
		__block uint32 SelectionLocation;
		__block uint32 SelectionLength;
		if (replacementRange.location == NSNotFound)
		{
			if (markedRange.location != NSNotFound)
			{
				SelectionLocation = markedRange.location;
				SelectionLength = markedRange.length;
			}
			else
			{
				GameThreadCall(^{
					if (IMMContext.IsValid())
					{
						ITextInputMethodContext::ECaretPosition CaretPosition;
						IMMContext->GetSelectionRange(SelectionLocation, SelectionLength, CaretPosition);
					}
				}, true, @[ NSDefaultRunLoopMode, UnrealIMEEventMode ]);
			}
		}
		else
		{
			SelectionLocation = replacementRange.location;
			SelectionLength = replacementRange.length;
		}

		if ([aString length] == 0)
		{
			GameThreadCall(^{
				if (IMMContext.IsValid())
				{
					IMMContext->SetTextInRange(SelectionLocation, SelectionLength, FString());
				}
			}, true, @[ NSDefaultRunLoopMode, UnrealIMEEventMode ]);
			[self unmarkText];
		}
		else
		{
			if(markedRange.location == NSNotFound)
			{
				GameThreadCall(^{
					if (IMMContext.IsValid())
					{
						IMMContext->BeginComposition();
					}
				}, true, @[ NSDefaultRunLoopMode, UnrealIMEEventMode ]);
			}
			markedRange = NSMakeRange(SelectionLocation, [aString length]);

			__block NSRange CompositionRange = markedRange;

			NSString* TheString;
			if ([aString isKindOfClass:[NSAttributedString class]])
			{
				// While the whole string is being composed NSUnderlineStyleAttributeName is 1 to show a single line below the whole string.
				// When using the pop-up glyph selection window in some IME's the NSAttributedString is broken up into separate glyph ranges, each with its own set of attributes.
				// Each range specifies NSMarkedClauseSegment, incrementing the NSNumber value from 0 as well as NSUnderlineStyleAttributeName, which makes the underlining show the different ranges.
				// The subrange being edited by the pop-up glyph selection window will set NSUnderlineStyleAttributeName to a value >1, while all other ranges will be set NSUnderlineStyleAttributeName to 1.
				NSAttributedString* AttributedString = (NSAttributedString*)aString;
				[AttributedString enumerateAttribute:NSUnderlineStyleAttributeName inRange:NSMakeRange(0, [aString length]) options:0 usingBlock:^(id Value, NSRange Range, BOOL* bStop) {
					if(Value && [Value isKindOfClass:[NSNumber class]])
					{
						NSNumber* NumberValue = (NSNumber*)Value;
						const int UnderlineValue = [NumberValue intValue];
						if(UnderlineValue > 1)
						{
							// Found the active range, stop enumeration.
							*bStop = YES;
							CompositionRange.location += Range.location;
							CompositionRange.length = Range.length;
						}
					}
				}];

				TheString = [AttributedString string];
			}
			else
			{
				TheString = (NSString*)aString;
			}

			GameThreadCall(^{
				SCOPED_AUTORELEASE_POOL;
				if (IMMContext.IsValid())
				{
					FString TheFString(TheString);
					IMMContext->SetTextInRange(SelectionLocation, SelectionLength, TheFString);
					IMMContext->UpdateCompositionRange(CompositionRange.location, CompositionRange.length);
					IMMContext->SetSelectionRange(markedRange.location + selectedRange.location, 0, ITextInputMethodContext::ECaretPosition::Ending);
				}
			}, true, @[ NSDefaultRunLoopMode, UnrealIMEEventMode ]);
		}
		[[self inputContext] invalidateCharacterCoordinates]; // recentering
	}
	else
	{
		reallyHandledEvent = false;
	}
}

/* The receiver unmarks the marked text. If no marked text, the invocation of this method has no effect.
 */
- (void)unmarkText
{
	if(markedRange.location != NSNotFound)
	{
		markedRange = {NSNotFound, 0};
		if (IMMContext.IsValid())
		{
			GameThreadCall(^{
				if (IMMContext.IsValid())
				{
					IMMContext->UpdateCompositionRange(0, 0);
					IMMContext->EndComposition();
				}
			}, true, @[ NSDefaultRunLoopMode, UnrealIMEEventMode ]);
		}
	}
}

/* Returns the selection range. The valid location is from 0 to the document length.
 */
- (NSRange)selectedRange
{
	FCocoaWindow* CocoaWindow = [[self window] isKindOfClass:[FCocoaWindow class]] ? (FCocoaWindow*)[self window] : nil;
	if (CocoaWindow && CocoaWindow->bIsBeingResized)
	{
		return {0, 0};
	}
	
	if (IMMContext.IsValid())
	{
		__block uint32 SelectionLocation;
		__block uint32 SelectionLength;
		__block ITextInputMethodContext::ECaretPosition CaretPosition;
		GameThreadCall(^{
			if (IMMContext.IsValid())
			{
				IMMContext->GetSelectionRange(SelectionLocation, SelectionLength, CaretPosition);
			}
		}, true, @[ NSDefaultRunLoopMode, UnrealIMEEventMode ]);
		return {SelectionLocation, SelectionLength};
	}
	else
	{
		return {NSNotFound, 0};
	}
}

/* Returns the marked range. Returns {NSNotFound, 0} if no marked range.
 */
- (NSRange)markedRange
{
	if (IMMContext.IsValid())
	{
		return markedRange;
	}
	else
	{
		return {NSNotFound, 0};
	}
}

/* Returns whether or not the receiver has marked text.
 */
- (BOOL)hasMarkedText
{
	return IMMContext.IsValid() && (markedRange.location != NSNotFound);
}

/* Returns attributed string specified by aRange. It may return nil. If non-nil return value and actualRange is non-NULL, it contains the actual range for the return value. The range can be adjusted from various reasons (i.e. adjust to grapheme cluster boundary, performance optimization, etc).
 */
- (NSAttributedString *)attributedSubstringForProposedRange:(NSRange)aRange actualRange:(NSRangePointer)actualRange
{
	// Deliberately no autorelease pool here - relies on the OS setting one up beforehand, otherwise the returned object won't be properly ref-counted
	NSAttributedString* AttributedString = nil;
	if (IMMContext.IsValid())
	{
		__block FString String;
		GameThreadCall(^{
			if (IMMContext.IsValid())
			{
				IMMContext->GetTextInRange(aRange.location, aRange.length, String);
			}
		}, true, @[ NSDefaultRunLoopMode, UnrealIMEEventMode ]);
		CFStringRef CFString = FPlatformString::TCHARToCFString(*String);
		if(CFString)
		{
			AttributedString = [[[NSAttributedString alloc] initWithString:(NSString*)CFString] autorelease];
			CFRelease(CFString);
			if(actualRange)
			{
				actualRange->location = aRange.location;
				actualRange->length = String.Len();
			}
		}
	}
	return AttributedString;
}

/* Returns an array of attribute names recognized by the receiver.
 */
- (NSArray*)validAttributesForMarkedText
{
	// We only allow these attributes to be set on our marked text (plus standard attributes)
	// NSMarkedClauseSegmentAttributeName is important for CJK input, among other uses
	// NSGlyphInfoAttributeName allows alternate forms of characters
	// Deliberately no autorelease pool here - relies on the OS setting one up beforehand, otherwise the returned object won't be properly ref-counted
	return [NSArray arrayWithObjects:NSMarkedClauseSegmentAttributeName, NSGlyphInfoAttributeName, nil];
}

/* Returns the first logical rectangular area for aRange. The return value is in the screen coordinate. The size value can be negative if the text flows to the left. If non-NULL, actuallRange contains the character range corresponding to the returned area.
 */
- (NSRect)firstRectForCharacterRange:(NSRange)aRange actualRange:(NSRangePointer)actualRange
{
	if (IMMContext.IsValid())
	{
		SCOPED_AUTORELEASE_POOL;
		__block FVector2D Position;
		__block FVector2D Size;
		GameThreadCall(^{
			if (IMMContext.IsValid())
			{
				IMMContext->GetTextBounds(aRange.location, aRange.length, Position, Size);
			}
		}, true, @[ NSDefaultRunLoopMode, UnrealIMEEventMode ]);

		if(actualRange)
		{
			*actualRange = aRange;
		}

		NSScreen* PrimaryScreen = [[self window] screen];
		const CGFloat PrimaryScreenHeight = [PrimaryScreen visibleFrame].size.height;
		Position.Y = -(Position.Y - PrimaryScreenHeight + 1);

		NSRect GlyphBox = {{Position.X,Position.Y},{Size.X,Size.Y}};
		return GlyphBox;
	}
	else
	{
		NSRect GlyphBox = {{0,0},{0,0}};
		return GlyphBox;
	}
}

/* Returns the index for character that is nearest to aPoint. aPoint is in the screen coordinate system.
 */
- (NSUInteger)characterIndexForPoint:(NSPoint)aPoint
{
	if (IMMContext.IsValid())
	{
		FVector2D Pos(aPoint.x, aPoint.y);
		int32 Index = GameThreadReturn(^{ return IMMContext.IsValid() ? IMMContext->GetCharacterIndexFromPoint(Pos) : INDEX_NONE; }, @[ NSDefaultRunLoopMode, UnrealIMEEventMode ]);
		NSUInteger Idx = Index == INDEX_NONE ? NSNotFound : (NSUInteger)Index;
		return Idx;
	}
	else
	{
		return NSNotFound;
	}
}

//@optional

/* Returns the window level of the receiver. An NSTextInputClient can implement this interface to specify its window level if it is higher than NSFloatingWindowLevel.
 */
- (NSInteger)windowLevel
{
	SCOPED_AUTORELEASE_POOL;
	return [[self window] level];
}

@end
