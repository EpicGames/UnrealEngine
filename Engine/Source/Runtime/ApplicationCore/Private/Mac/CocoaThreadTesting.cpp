// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "Mac/CocoaThread.h"

#if (WITH_EDITOR || IS_PROGRAM) && !UE_BUILD_SHIPPING

void RecursiveThreadCall(int Depth, bool bFromGameThread, int MaxDepth)
{
	if (Depth < MaxDepth)
	{
		if (bFromGameThread)
		{
			// Test other thread call
			MainThreadCall(^{
				RecursiveThreadCall(Depth + 1, false, MaxDepth);
			}, true, UnrealNilEventMode);
			
			// Test current thread call
			GameThreadCall(^{});
		}
		else
		{
			// Test other thread call
			GameThreadCall(^{
				RecursiveThreadCall(Depth + 1, true, MaxDepth);
			});
			
			// Test current thread call
			MainThreadCall(^{});
		}
	}
}

static void TestCocoaThread()
{
	RecursiveThreadCall(0, true, 100);
}

FAutoConsoleCommand TestCocoaCommand(TEXT("Mac.Tests.CocoaThread"), TEXT(""), FConsoleCommandDelegate::CreateStatic(&TestCocoaThread));

#endif
