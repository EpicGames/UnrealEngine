// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/CocoaWindow.h"

#include "Apple/ScopeAutoreleasePool.h"
#include "Mac/MacApplication.h"
#include "Mac/CocoaTextView.h"
#include "Mac/CocoaThread.h"
#include "Mac/MacCursor.h"
#include "HAL/IConsoleManager.h"
#if WITH_ACCESSIBILITY
#include "Mac/Accessibility/CocoaAccessibilityView.h"
#endif
NSString* NSDraggingExited = @"NSDraggingExited";
NSString* NSDraggingUpdated = @"NSDraggingUpdated";
NSString* NSPrepareForDragOperation = @"NSPrepareForDragOperation";
NSString* NSPerformDragOperation = @"NSPerformDragOperation";

/**
 * Custom window class used for input handling
 */
@implementation FCocoaWindow

@synthesize TargetWindowMode;

- (id)initWithContentRect:(NSRect)ContentRect styleMask:(NSUInteger)Style backing:(NSBackingStoreType)BufferingType defer:(BOOL)Flag
{
	WindowMode = EWindowMode::Windowed;
	bAcceptsInput = false;
	bRenderInitialized = false;
	bIsBeingOrderedFront = false;
	bIsBeingResized = false;
	Opacity = 0.0f;

	id NewSelf = [super initWithContentRect:ContentRect styleMask:Style backing:BufferingType defer:Flag];
	if (NewSelf)
	{
		bZoomed = [super isZoomed];
		bMinimized = [super isMiniaturized];
		bKeyWindow = [super isKeyWindow];
		CachedScreen = [super screen];
		CachedBackingScaleFactor = [super backingScaleFactor];
		bIsOnActiveSpace = [super isOnActiveSpace];
		self.TargetWindowMode = EWindowMode::Windowed;
		[super setAlphaValue:Opacity];
		[super setRestorable:NO];
		[super disableSnapshotRestoration];
	}
	return NewSelf;
}

- (NSRect)openGLFrame
{
	SCOPED_AUTORELEASE_POOL;
	return [[self contentView] frame];
}

- (NSView*)openGLView
{
	SCOPED_AUTORELEASE_POOL;
	return [self contentView];
}

- (void)setAcceptsInput:(bool)InAcceptsInput
{
	bAcceptsInput = InAcceptsInput;
}

- (NSApplicationPresentationOptions)window:(NSWindow *)window willUseFullScreenPresentationOptions:(NSApplicationPresentationOptions)proposedOptions {
		return proposedOptions;
	  }

- (void)setWindowMode:(EWindowMode::Type)NewWindowMode
{
	WindowMode = NewWindowMode;
	
	NSView* OpenGLView = [self openGLView];
	[[NSNotificationCenter defaultCenter] postNotificationName:NSViewFrameDidChangeNotification object:OpenGLView];
	[[NSNotificationCenter defaultCenter] postNotificationName:NSViewBoundsDidChangeNotification object:OpenGLView];
}

- (EWindowMode::Type)windowMode
{
	return WindowMode;
}

- (void)orderFrontAndMakeMain:(bool)bMain andKey:(bool)bKey
{
	SCOPED_AUTORELEASE_POOL;
	if ([NSApp isHidden] == NO)
	{
		bIsBeingOrderedFront = true;

		[self orderFront:nil];

		if (bMain && [self canBecomeMainWindow] && self != [NSApp mainWindow])
		{
			[self makeMainWindow];
		}
		if (bKey && [self canBecomeKeyWindow] && self != [NSApp keyWindow])
		{
			[self makeKeyWindow];
		}

		bIsBeingOrderedFront = false;
	}
}

// Following few methods overload NSWindow's methods from Cocoa API, so have to use Cocoa's BOOL (signed char), not bool (unsigned int)
- (BOOL)canBecomeMainWindow
{
	SCOPED_AUTORELEASE_POOL;
	return bAcceptsInput && ![self ignoresMouseEvents] && self.AllowMainWindow;
}

- (BOOL)canBecomeKeyWindow
{
	SCOPED_AUTORELEASE_POOL;
	return bAcceptsInput && ([self styleMask] != NSWindowStyleMaskBorderless);
}

- (BOOL)validateMenuItem:(NSMenuItem*)MenuItem
{
	SCOPED_AUTORELEASE_POOL;
	// Borderless windows we use do not automatically handle first responder's actions, so we force it here
	return ([MenuItem action] == @selector(performClose:) || [MenuItem action] == @selector(miniaturize:) || [MenuItem action] == @selector(zoom:)) ? YES : [super validateMenuItem:MenuItem];
}

- (void)setAlphaValue:(CGFloat)WindowAlpha
{
	Opacity = WindowAlpha;
	if (bRenderInitialized)
	{
		MainThreadCall(^{
			[super setAlphaValue:WindowAlpha];
		}, false);
	}
}

- (void)startRendering
{
	if (!bRenderInitialized)
	{
		bRenderInitialized = true;
		MainThreadCall(^{
			[super setAlphaValue:Opacity];
		}, false);
	}
}

- (bool)isRenderInitialized
{
	return bRenderInitialized;
}

- (void)performClose:(id)Sender
{
	GameThreadCall(^{
		if (MacApplication)
		{
			TSharedPtr<FMacWindow> Window = MacApplication->FindWindowByNSWindow(self);
			if (Window.IsValid())
			{
				MacApplication->CloseWindow(Window.ToSharedRef());
			}
		}
	}, false);
}

- (void)performZoom:(id)Sender
{
}

- (void)zoom:(id)Sender
{
	SCOPED_AUTORELEASE_POOL;
	bZoomed = !bZoomed;
	[super zoom:Sender];
}

- (void)toggleFullScreen:(id)Sender
{
	// Make sure we don't limit the window size for fullscreen toggle. Limits, if needed, will be restored by FMacWindow after fullscreen transition (in FMacApplication::OnCursorLock()).
	[self setMinSize:NSMakeSize(10.0f, 10.0f)];
	[self setMaxSize:NSMakeSize(10000.0f, 10000.0f)];
	[super toggleFullScreen:Sender];
}

- (void)keyDown:(NSEvent*)Event
{
	// @note Deliberately empty - we don't want OS X to handle keyboard input as it will recursively re-add events we aren't handling
}

- (void)keyUp:(NSEvent*)Event
{
	// @note Deliberately empty - we don't want OS X to handle keyboard input as it will recursively re-add events we aren't handling
}

- (NSSize)window:(NSWindow *)window willUseFullScreenContentSize:(NSSize)proposedSize
{
	// Without this delegate method we seem to get different behavour in rare edge cases when changing to windowed fullscreen mode
	return proposedSize;
}

- (void)windowWillEnterFullScreen:(NSNotification*)Notification
{
	FMacCursor* MacCursor = MacApplication ? (FMacCursor*)MacApplication->Cursor.Get() : nullptr;
	if (MacCursor)
	{
		MacCursor->SetShouldIgnoreLocking(true);
	}

	// Handle clicking on the titlebar fullscreen item
	if (self.TargetWindowMode == EWindowMode::Windowed)
	{
		// Use the current default fullscreen mode when switching via the OS button
		static auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.FullScreenMode"));
		check(CVar);
		self.TargetWindowMode = CVar->GetValueOnAnyThread() == 0 ? EWindowMode::Fullscreen : EWindowMode::WindowedFullscreen;
		
#if WITH_EDITORONLY_DATA // Always use WindowedFullscreen for the Editor or bad things happen
		if (GIsEditor)
		{
			self.TargetWindowMode = EWindowMode::WindowedFullscreen;
		}
#endif
	}
}

- (void)windowDidEnterFullScreen:(NSNotification*)Notification
{
	WindowMode = self.TargetWindowMode;

	if (MacApplication)
	{
		MacApplication->DeferEvent(Notification);
	}
}

- (void)windowWillExitFullScreen:(NSNotification *)Notification
{
	FMacCursor* MacCursor = MacApplication ? (FMacCursor*)MacApplication->Cursor.Get() : nullptr;
	if (MacCursor)
	{
		MacCursor->SetShouldIgnoreLocking(true);
	}

	if (self.TargetWindowMode != EWindowMode::Windowed)
	{
		self.TargetWindowMode = EWindowMode::Windowed;
	}
}

- (void)windowDidExitFullScreen:(NSNotification*)Notification
{
	WindowMode = EWindowMode::Windowed;
	self.TargetWindowMode = EWindowMode::Windowed;

	if (MacApplication)
	{
		MacApplication->DeferEvent(Notification);
	}
}

- (void)windowDidBecomeKey:(NSNotification *)notification
{
	bKeyWindow = true;
}

- (void)windowDidResignKey:(NSNotification *)notification
{
	bKeyWindow = false;
}

- (void)windowDidBecomeMain:(NSNotification*)Notification
{
	SCOPED_AUTORELEASE_POOL;
	if ([NSApp isHidden] == NO)
	{
		[self orderFrontAndMakeMain:false andKey:false];
	}

	if (MacApplication)
	{
		const bool bWaitForThread = false;
		
		GameThreadCall(^{
			if (MacApplication) // Another check because game thread may destroy MacApplication before it gets here
			{
				TSharedPtr<FMacWindow> Window = MacApplication->FindWindowByNSWindow(self);
				if (Window.IsValid())
				{
					MacApplication->OnWindowActivationChanged(Window.ToSharedRef(), EWindowActivation::Activate);
				}
			}
		}, bWaitForThread, @[ NSDefaultRunLoopMode, UnrealResizeEventMode, UnrealShowEventMode, UnrealFullscreenEventMode, UnrealCloseEventMode ]);
	}
}

- (void)windowDidResignMain:(NSNotification*)Notification
{
	SCOPED_AUTORELEASE_POOL;
	[self setMovable: YES];
	[self setMovableByWindowBackground: NO];

	if (MacApplication)
	{
		const bool bWaitForThread = false;
		GameThreadCall(^{
			if (MacApplication) // Another check because game thread may destroy MacApplication before it gets here
			{
				TSharedPtr<FMacWindow> Window = MacApplication->FindWindowByNSWindow(self);
				if (Window.IsValid())
				{
					MacApplication->OnWindowActivationChanged(Window.ToSharedRef(), EWindowActivation::Deactivate);
				}
			}
		}, bWaitForThread, @[ NSDefaultRunLoopMode, UnrealResizeEventMode, UnrealShowEventMode, UnrealFullscreenEventMode, UnrealCloseEventMode ]);
	}
}

- (void)windowWillMove:(NSNotification*)Notification
{
	if (MacApplication)
	{
		MacApplication->DeferEvent(Notification);
	}
}

- (void)windowDidMove:(NSNotification*)Notification
{
	SCOPED_AUTORELEASE_POOL;
	bZoomed = [self isZoomed];
	
	NSView* OpenGLView = [self openGLView];
	[[NSNotificationCenter defaultCenter] postNotificationName:NSViewFrameDidChangeNotification object:OpenGLView];
	[[NSNotificationCenter defaultCenter] postNotificationName:NSViewBoundsDidChangeNotification object:OpenGLView];

	if (MacApplication)
	{
		MacApplication->DeferEvent(Notification);
	}
}

- (NSRect)constrainFrameRect:(NSRect)FrameRect toScreen:(NSScreen*)Screen
{
	NSRect ConstrainedRect = [super constrainFrameRect:FrameRect toScreen:Screen];

	if (self.TargetWindowMode == EWindowMode::Windowed)
	{
		// In windowed mode do not limit the window size to screen size
		ConstrainedRect.origin.y -= FrameRect.size.height - ConstrainedRect.size.height;
		ConstrainedRect.size = FrameRect.size;
	}

	return ConstrainedRect;
}

- (void)windowDidChangeScreen:(NSNotification*)Notification
{
	CachedScreen = [self screen];
	// The windowdidChangeScreen notification only comes after you finish dragging.
	// It does however, work fine for handling display arrangement changes that cause a window to go offscreen.
	if (MacApplication && MacApplication->IsDisplayReconfiguring())
	{
		SCOPED_AUTORELEASE_POOL;
		NSScreen* Screen = [self screen];
		NSRect Frame = [self frame];
		NSRect VisibleFrame = [Screen visibleFrame];
		if (NSContainsRect(VisibleFrame, Frame) == NO)
		{
			// May need to scale the window to fit if it is larger than the new display.
			if (Frame.size.width > VisibleFrame.size.width || Frame.size.height > VisibleFrame.size.height)
			{
				NSRect NewFrame;
				NewFrame.size.width = Frame.size.width > VisibleFrame.size.width ? VisibleFrame.size.width : Frame.size.width;
				NewFrame.size.height = Frame.size.height > VisibleFrame.size.height ? VisibleFrame.size.height : Frame.size.height;
				NewFrame.origin = VisibleFrame.origin;
				
				[self setFrame:NewFrame display:NO];
			}
			else
			{
				NSRect Intersection = NSIntersectionRect(VisibleFrame, Frame);
				NSPoint Origin = Frame.origin;
				
				// If there's at least something on screen, try shifting it entirely on screen.
				if (Intersection.size.width > 0 && Intersection.size.height > 0)
				{
					CGFloat X = Frame.size.width - Intersection.size.width;
					CGFloat Y = Frame.size.height - Intersection.size.height;
					
					if (Intersection.size.width+Intersection.origin.x >= VisibleFrame.size.width+VisibleFrame.origin.x)
					{
						Origin.x -= X;
					}
					else if (Origin.x < VisibleFrame.origin.x)
					{
						Origin.x += X;
					}
					
					if (Intersection.size.height+Intersection.origin.y >= VisibleFrame.size.height+VisibleFrame.origin.y)
					{
						Origin.y -= Y;
					}
					else if (Origin.y < VisibleFrame.origin.y)
					{
						Origin.y += Y;
					}
				}
				else
				{
					Origin = VisibleFrame.origin;
				}
				
				[self setFrameOrigin:Origin];
			}
		}
	}
	else
	{
		GameThreadCall(^{
			if (MacApplication)
			{
				TSharedPtr<FMacWindow> Window = MacApplication->FindWindowByNSWindow(self);
				if (Window.IsValid())
				{
					MacApplication->OnWindowDidResize(Window.ToSharedRef(), false);
				}
			}
		}, false);
	}

	GameThreadCall(^{
		if (MacApplication)
		{
			TSharedPtr<FMacWindow> Window = MacApplication->FindWindowByNSWindow(self);
			if (Window.IsValid())
			{
				MacApplication->OnWindowChangedScreen(Window.ToSharedRef());
			}
		}
	}, false);
}

- (void)windowDidChangeBackingProperties:(NSNotification *)notification
{
	CachedBackingScaleFactor = [self backingScaleFactor];
}

- (void)windowWillStartLiveResize:(NSNotification*)Notification
{
	SCOPED_AUTORELEASE_POOL;
	if (MacApplication)
	{
		MacApplication->DismissAllMenusOnGameThread();
		MacApplication->DeferEvent(Notification);
	}
}

- (void)windowDidEndLiveResize:(NSNotification*)Notification
{
	SCOPED_AUTORELEASE_POOL;
	if (MacApplication)
	{
		MacApplication->DeferEvent(Notification);
	}
}

- (NSSize)windowWillResize:(NSWindow *)sender toSize:(NSSize)frameSize
{
	SCOPED_AUTORELEASE_POOL;
	if (MacApplication && sender == self && !bIsBeingOrderedFront) // Skip informing Slate if we're simply changing the z order of windows
	{
		const bool bWaitForThread = true;
		
		GameThreadCall(^{
			if (MacApplication) // Another check because game thread may destroy MacApplication before it gets here
			{
				TSharedPtr<FMacWindow> Window = MacApplication->FindWindowByNSWindow((FCocoaWindow*)sender);
				if (Window.IsValid())
				{
					MacApplication->OnWindowWillResize(Window.ToSharedRef());
				}
			}
		}, bWaitForThread, @[ NSDefaultRunLoopMode, UnrealResizeEventMode, UnrealShowEventMode, UnrealFullscreenEventMode ]);
	}
	return frameSize;
}

- (void)windowDidResize:(NSNotification*)Notification
{
	SCOPED_AUTORELEASE_POOL;
	bZoomed = [self isZoomed];
	if (MacApplication)
	{
		bIsBeingResized = true;
		MacApplication->DeferEvent(Notification);
		bIsBeingResized = false;
	}
}

- (void)windowWillClose:(NSNotification*)Notification
{
	SCOPED_AUTORELEASE_POOL;

	[self setDelegate:nil];
}

- (NSDragOperation)draggingEntered:(id <NSDraggingInfo>)Sender
{
	return NSDragOperationGeneric;
}

- (void)draggingExited:(id <NSDraggingInfo>)Sender
{
	SCOPED_AUTORELEASE_POOL;
	NSNotification* Notification = [NSNotification notificationWithName:NSDraggingExited object:Sender];
	if (MacApplication)
	{
		MacApplication->DeferEvent(Notification);
	}
}

- (NSDragOperation)draggingUpdated:(id <NSDraggingInfo>)Sender
{
	SCOPED_AUTORELEASE_POOL;
	NSNotification* Notification = [NSNotification notificationWithName:NSDraggingUpdated object:Sender];
	if (MacApplication)
	{
		MacApplication->DeferEvent(Notification);
	}
	return NSDragOperationGeneric;
}

- (BOOL)prepareForDragOperation:(id <NSDraggingInfo>)Sender
{
	SCOPED_AUTORELEASE_POOL;
	NSNotification* Notification = [NSNotification notificationWithName:NSPrepareForDragOperation object:Sender];
	if (MacApplication)
	{
		MacApplication->DeferEvent(Notification);
	}
	return YES;
}

- (BOOL)performDragOperation:(id <NSDraggingInfo>)Sender
{
	SCOPED_AUTORELEASE_POOL;
	NSNotification* Notification = [NSNotification notificationWithName:NSPerformDragOperation object:Sender];
	if (MacApplication)
	{
		MacApplication->DeferEvent(Notification);
	}
	return YES;
}

#if WITH_ACCESSIBILITY

- (void)UpdateAccessibilityView:(AccessibleWidgetId) InAccessibilityWindowId
{
	checkf([NSThread isMainThread], TEXT("Updating accessibility view in FCocoaWindow from wrong Thread! Accessibility can only be done on Main Thread!"));
	NSView* OpenGLView = [self openGLView];
	// FCocoaAccessibilityView is the base class all custom NSViews must inherit from to support accessibility
	if([OpenGLView isKindOfClass:[FCocoaAccessibilityView class]])
	{
		FCocoaAccessibilityView* CurrentAccessibilityView = (FCocoaAccessibilityView*) OpenGLView;
		[CurrentAccessibilityView SetAccessibilityWindowAsAccessibilityChild: InAccessibilityWindowId];
	}
}

- (void)ClearAccessibilityView
{
	checkf([NSThread isMainThread], TEXT("Updating accessibility view in FCocoaWindow from wrong Thread! Accessibility can only be done on Main Thread!"));
	NSView* OpenGLView = [self openGLView];
	// FCocoaAccessibilityView is the base class all custom NSViews must inherit from to support accessibility
	if([OpenGLView isKindOfClass:[FCocoaAccessibilityView class]])
	{
		FCocoaAccessibilityView* CurrentAccessibilityView = (FCocoaAccessibilityView*) OpenGLView;
		[CurrentAccessibilityView RemoveAccessibilityWindow];
	}
}
#endif

- (BOOL)AllowMainWindow
{
	switch (self.Type)
	{
		case EWindowType::Menu:
		case EWindowType::ToolTip:
		case EWindowType::Notification:
		case EWindowType::CursorDecorator:
			return false;
		default:
			return true;
	}
}

- (void)windowDidMiniaturize:(NSNotification *)notification
{
	bMinimized = true;
}

- (void)windowDidDeminiaturize:(NSNotification *)notification
{
	bMinimized = false;
	if (MacApplication)
	{
		MacApplication->ReassociateMouseInput();
	}
}

@end
