// Copyright Epic Games, Inc. All Rights Reserved.

#include "HIDInputInterface.h"
#include "HAL/PlatformTime.h"
#include "Misc/CallbackDevice.h"
#include "Misc/ConfigCacheIni.h"
#include "Templates/SharedPointer.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "GenericPlatform/InputDeviceRegistry.h"

static int32 GetDevicePropertyAsInt32(IOHIDDeviceRef DeviceRef, CFStringRef Property)
{
	CFTypeRef Ref = IOHIDDeviceGetProperty(DeviceRef, Property);
	if (Ref && CFGetTypeID(Ref) == CFNumberGetTypeID())
	{
		int32 Value = 0;
		CFNumberGetValue((CFNumberRef)Ref, kCFNumberSInt32Type, &Value);
		return Value;
	}
	return 0;
}

TSharedRef<HIDInputInterface> HIDInputInterface::Create(const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler)
{
	return MakeShareable(new HIDInputInterface(InMessageHandler));
}

HIDInputInterface::HIDInputInterface(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
:	MessageHandler(InMessageHandler)
{
	for (int32 ControllerIndex=0; ControllerIndex < MAX_NUM_HIDINPUT_CONTROLLERS; ++ControllerIndex)
	{
		FControllerState& ControllerState = ControllerStates[ControllerIndex];
		FMemory::Memzero(&ControllerState, sizeof(FControllerState));

		ControllerState.ControllerId = ControllerIndex;
	}

	bIsGamepadAttached = false;
	InitialButtonRepeatDelay = 0.2f;
	ButtonRepeatDelay = 0.1f;

	GConfig->GetFloat(TEXT("/Script/Engine.InputSettings"), TEXT("InitialButtonRepeatDelay"), InitialButtonRepeatDelay, GInputIni);
	GConfig->GetFloat(TEXT("/Script/Engine.InputSettings"), TEXT("ButtonRepeatDelay"), ButtonRepeatDelay, GInputIni);

	Buttons[0] = FGamepadKeyNames::FaceButtonBottom;
	Buttons[1] = FGamepadKeyNames::FaceButtonRight;
	Buttons[2] = FGamepadKeyNames::FaceButtonLeft;
	Buttons[3] = FGamepadKeyNames::FaceButtonTop;
	Buttons[4] = FGamepadKeyNames::LeftShoulder;
	Buttons[5] = FGamepadKeyNames::RightShoulder;
	Buttons[6] = FGamepadKeyNames::SpecialRight;
	Buttons[7] = FGamepadKeyNames::SpecialLeft;
	Buttons[8] = FGamepadKeyNames::LeftThumb;
	Buttons[9] = FGamepadKeyNames::RightThumb;
	Buttons[10] = FGamepadKeyNames::LeftTriggerThreshold;
	Buttons[11] = FGamepadKeyNames::RightTriggerThreshold;
	Buttons[12] = FGamepadKeyNames::DPadUp;
	Buttons[13] = FGamepadKeyNames::DPadDown;
	Buttons[14] = FGamepadKeyNames::DPadLeft;
	Buttons[15] = FGamepadKeyNames::DPadRight;
	Buttons[16] = FGamepadKeyNames::LeftStickUp;
	Buttons[17] = FGamepadKeyNames::LeftStickDown;
	Buttons[18] = FGamepadKeyNames::LeftStickLeft;
	Buttons[19] = FGamepadKeyNames::LeftStickRight;
	Buttons[20] = FGamepadKeyNames::RightStickUp;
	Buttons[21] = FGamepadKeyNames::RightStickDown;
	Buttons[22] = FGamepadKeyNames::RightStickLeft;
	Buttons[23] = FGamepadKeyNames::RightStickRight;

	// Init HID Manager
	HIDManager = IOHIDManagerCreate(kCFAllocatorDefault, 0L);
	if (!HIDManager)
	{
		return;	// This will cause all subsequent calls to return information that nothing's connected
	}

	IOReturn Result = IOHIDManagerOpen(HIDManager, kIOHIDOptionsTypeNone);
	if (Result != kIOReturnSuccess)
	{
		CFRelease(HIDManager);
		HIDManager = NULL;
		return;
	}

	// Set HID Manager to detect devices of two distinct types: Gamepads and joysticks
	CFMutableArrayRef MatchingArray = CFArrayCreateMutable(kCFAllocatorDefault, 0, &kCFTypeArrayCallBacks);
	if (!MatchingArray)
	{
		CFRelease(HIDManager);
		HIDManager = NULL;
		return;
	}

	CFDictionaryRef MatchingJoysticks = CreateDeviceMatchingDictionary(kHIDPage_GenericDesktop, kHIDUsage_GD_Joystick);
	if (!MatchingJoysticks)
	{
		CFRelease(MatchingArray);
		CFRelease(HIDManager);
		HIDManager = 0;
		return;
	}

	CFArrayAppendValue(MatchingArray, MatchingJoysticks);
	CFRelease(MatchingJoysticks);

	CFDictionaryRef MatchingGamepads = CreateDeviceMatchingDictionary(kHIDPage_GenericDesktop, kHIDUsage_GD_GamePad);
	if (!MatchingGamepads)
	{
		CFRelease(MatchingArray);
		CFRelease(HIDManager);
		HIDManager = 0;
		return;
	}

	CFArrayAppendValue(MatchingArray, MatchingGamepads);
	CFRelease(MatchingGamepads);

	IOHIDManagerSetDeviceMatchingMultiple(HIDManager, MatchingArray);
	CFRelease(MatchingArray);

	// Setup HID Manager's add/remove devices callbacks
	IOHIDManagerRegisterDeviceMatchingCallback(HIDManager, HIDDeviceMatchingCallback, this);
	IOHIDManagerRegisterDeviceRemovalCallback(HIDManager, HIDDeviceRemovalCallback, this);

	// Add HID Manager to run loop
	IOHIDManagerScheduleWithRunLoop(HIDManager, CFRunLoopGetCurrent(), kCFRunLoopDefaultMode);
}

void HIDInputInterface::FHIDDeviceInfo::SetupMappings()
{
	FMemory::Memset(ButtonsMapping, -1, sizeof(ButtonsMapping));

	const int32 VendorID = GetDevicePropertyAsInt32(DeviceRef, CFSTR(kIOHIDVendorIDKey));
	const int32 ProductID = GetDevicePropertyAsInt32(DeviceRef, CFSTR(kIOHIDProductIDKey));

	UE_LOGF(LogMac, Log, "Controller attached: VendorID 0x%x, ProductID 0x%x", VendorID, ProductID);

	if (VendorID == 0x54c && ProductID == 0x268)
	{
		// PlayStation 3 Controller
		ButtonsMapping[1]	= 7;	// Select		->	Back
		ButtonsMapping[2]	= 8;	// L3			->	Left Thumbstick
		ButtonsMapping[3]	= 9;	// R3			->	Right Thumbstick
		ButtonsMapping[4]	= 6;	// Start		->	Start
		ButtonsMapping[5]	= 12;	// DPad Up		->	DPad Up
		ButtonsMapping[6]	= 15;	// DPad Right	->	DPad Right
		ButtonsMapping[7]	= 13;	// DPad Down	->	DPad Down
		ButtonsMapping[8]	= 14;	// DPad Left	->	DPad Left
		ButtonsMapping[9]	= 10;	// L2			->	Left Trigger
		ButtonsMapping[10]	= 11;	// R2			->	Right Trigger
		ButtonsMapping[11]	= 4;	// L1			->	Left Shoulder
		ButtonsMapping[12]	= 5;	// R1			->	Right Shoulder
		ButtonsMapping[13]	= 3;	// Triangle		->	Y
		ButtonsMapping[14]	= 1;	// Circle		->	B
		ButtonsMapping[15]	= 0;	// Cross		->	A
		ButtonsMapping[16]	= 2;	// Square		->	X

		LeftAnalogXMapping = kHIDUsage_GD_X;
		LeftAnalogYMapping = kHIDUsage_GD_Y;
		LeftTriggerAnalogMapping = kHIDUsage_GD_Rx;
		RightAnalogXMapping = kHIDUsage_GD_Z;
		RightAnalogYMapping = kHIDUsage_GD_Rz;
		RightTriggerAnalogMapping = kHIDUsage_GD_Ry;
	}
	else if (VendorID == 0x54c && (ProductID == 0x5c4 || ProductID == 0x9cc))
	{
		// PlayStation 4 Controller
		ButtonsMapping[1]	= 2;	// Square		->	X
		ButtonsMapping[2]	= 0;	// Cross		->	A
		ButtonsMapping[3]	= 1;	// Circle		->	B
		ButtonsMapping[4]	= 3;	// Triangle		->	Y
		ButtonsMapping[5]	= 4;	// L1			->	Left Shoulder
		ButtonsMapping[6]	= 5;	// R1			->	Right Shoulder
		ButtonsMapping[7]	= 10;	// L2			->	Left Trigger
		ButtonsMapping[8]	= 11;	// R2			->	Right Trigger
		ButtonsMapping[10]	= 6;	// Options		->	Start
		ButtonsMapping[11]	= 8;	// L3			->	Left Thumbstick
		ButtonsMapping[12]	= 9;	// R3			->	Right Thumbstick
		ButtonsMapping[14]	= 7;	// Touchpad		->	Back

		LeftAnalogXMapping = kHIDUsage_GD_X;
		LeftAnalogYMapping = kHIDUsage_GD_Y;
		LeftTriggerAnalogMapping = kHIDUsage_GD_Rx;
		RightAnalogXMapping = kHIDUsage_GD_Z;
		RightAnalogYMapping = kHIDUsage_GD_Rz;
		RightTriggerAnalogMapping = kHIDUsage_GD_Ry;
	}
	else if (VendorID == 0x45e && (ProductID == 0x2d1 || ProductID == 0x2ea))
	{
		// Xbox One Controller
		ButtonsMapping[1]	= 0;	// A
		ButtonsMapping[2]	= 1;	// B
		ButtonsMapping[3]	= 2;	// X
		ButtonsMapping[4]	= 3;	// Y
		ButtonsMapping[5]	= 4;	// Left Shoulder
		ButtonsMapping[6]	= 5;	// Right Shoulder
		ButtonsMapping[7]	= 8;	// Left Thumbstick
		ButtonsMapping[8]	= 9;	// Right Thumbstick
		ButtonsMapping[9]	= 6;	// Menu
		ButtonsMapping[10]	= 7;	// View
		ButtonsMapping[12]	= 12;	// DPad Up
		ButtonsMapping[13]	= 13;	// DPad Down
		ButtonsMapping[14]	= 14;	// DPad Left
		ButtonsMapping[15]	= 15;	// DPad Right

		LeftAnalogXMapping = kHIDUsage_GD_X;
		LeftAnalogYMapping = kHIDUsage_GD_Y;
		LeftTriggerAnalogMapping = kHIDUsage_GD_Z;
		RightAnalogXMapping = kHIDUsage_GD_Rx;
		RightAnalogYMapping = kHIDUsage_GD_Ry;
		RightTriggerAnalogMapping = kHIDUsage_GD_Rz;
	}
	else if (VendorID == 0x45e && (ProductID == 0x2fd || ProductID == 0xb13))
	{
		// IOHIDDeviceGetProperty(DeviceRef, CFSTR(kIOHIDTransportKey));
		// Should return bluetooth for this.  It could be that any Xbox One pad on bluetooth
		// should in here as they will different to the 3rd party driver that is assumed above
		
		//ButtonsMapping[Element.Usage] = FGamepadKeyNames index from order specified in HIDInputInterface constructor
		ButtonsMapping[1]	= 0;	// A
		ButtonsMapping[2]	= 1;	// B
		ButtonsMapping[4]	= 2;	// X
		ButtonsMapping[5]	= 3;	// Y
		ButtonsMapping[7]	= 4;	// Left Shoulder
		ButtonsMapping[8]	= 5;	// Right Shoulder
		ButtonsMapping[12]	= 6;	// Menu
		ButtonsMapping[14] =  8; 	// Left Thumb stick
		ButtonsMapping[15] =  9; 	// Right Thumb stick
		
		LeftAnalogXMapping = kHIDUsage_GD_X;
		LeftAnalogYMapping = kHIDUsage_GD_Y;
		LeftTriggerAnalogMapping = 0xc5;
		RightAnalogXMapping = kHIDUsage_GD_Z;
		RightAnalogYMapping = kHIDUsage_GD_Rz;
		RightTriggerAnalogMapping = 0xc4;
	}
	else if (VendorID == 0x57e && ProductID == 0x2009)
	{
		// Nintendo Switch Pro Controller
		ButtonsMapping[1]	= 0;	// B
		ButtonsMapping[2]	= 1;	// A
		ButtonsMapping[3]	= 2;	// Y
		ButtonsMapping[4]	= 3;	// X
		ButtonsMapping[5]	= 4;	// L
		ButtonsMapping[6]	= 5;	// R
		ButtonsMapping[7]	= 10;	// ZL
		ButtonsMapping[8]	= 11;	// ZR
		ButtonsMapping[9]	= 7;	// -
		ButtonsMapping[10]	= 6;	// +
		ButtonsMapping[11]	= 8;	// Left Thumbstick
		ButtonsMapping[12]	= 9;	// Right Thumbstick

		LeftAnalogXMapping = kHIDUsage_GD_X;
		LeftAnalogYMapping = kHIDUsage_GD_Y;
		RightAnalogXMapping = kHIDUsage_GD_Rx;
		RightAnalogYMapping = kHIDUsage_GD_Ry;
	}
	else
	{
		// Array generated from https://github.com/d235j/360Controller-n/blob/c83340e55be337e28a149a2930657006d3a3fef8/360Controller/Info.plist (with additions)
		static const FXBox360ControllerID XBox360Controllers[] =
		{
			{ 0x0E6F, 0x0113 }, 	// AfterglowGamepadforXbox360
			{ 0x0E6F, 0x0213 }, 	// AfterglowGamepadforXbox360
			{ 0x1BAD, 0xF900 }, 	// AfterglowGamepadforXbox360
			{ 0x0738, 0xCB29 }, 	// AviatorforXbox360PC
			{ 0x15E4, 0x3F10 }, 	// BatarangwiredcontrollerXBOX
			{ 0x146B, 0x0601 }, 	// BigbenControllerBB7201
			{ 0x0738, 0xF401 }, 	// Controller
			{ 0x0E6F, 0xF501 }, 	// Controller
			{ 0x1430, 0xF801 }, 	// Controller
			{ 0x1BAD, 0x028E }, 	// Controller
			{ 0x1BAD, 0xFA01 }, 	// Controller
			{ 0x12AB, 0x0004 }, 	// DDRUniverse2Mat
			{ 0x24C6, 0x5B00 }, 	// Ferrari458Racingwheel
			{ 0x1430, 0x4734 }, 	// GH4Guitar
			{ 0x046D, 0xC21D }, 	// GamepadF310
			{ 0x0E6F, 0x0301 }, 	// GamepadforXbox360
			{ 0x0E6F, 0x0401 }, 	// GamepadforXbox360Z
			{ 0x12AB, 0x0302 }, 	// GamepadforXbox360ZZ
			{ 0x1BAD, 0xF902 }, 	// GamepadforXbox360ZZZ
			{ 0x1BAD, 0xF901 }, 	// GamestopXbox360Controller
			{ 0x1430, 0x474C }, 	// GuitarHeroforPCMAC
			{ 0x1BAD, 0xF501 }, 	// HORIPADEX2TURBO
			{ 0x1BAD, 0x0003 }, 	// HarmonixDrumKitforXbox360
			{ 0x1BAD, 0x0002 }, 	// HarmonixGuitarforXbox360
			{ 0x0F0D, 0x000A }, 	// HoriCoDOA4FightStick
			{ 0x0F0D, 0x000D }, 	// HoriFightingStickEx2
			{ 0x0F0D, 0x0016 }, 	// HoriRealArcadeProEx
			{ 0x24C6, 0x5501 }, 	// HoriRealArcadeProVXSA
			{ 0x24C6, 0x5506 }, 	// HoriSOULCALIBURVStick
			{ 0x1BAD, 0xF02D }, 	// JoytechNeoSe
			{ 0x162E, 0xBEEF }, 	// JoytechNeoSeTake2
			{ 0x046D, 0xC242 }, 	// LogitechChillStream
			{ 0x046D, 0xC21E }, 	// LogitechF510
			{ 0x1BAD, 0xFD01 }, 	// MadCatz360
			{ 0x0738, 0x4740 }, 	// MadCatzBeatPad
			{ 0x1BAD, 0xF025 }, 	// MadCatzCallofDutyGamePad
			{ 0x1BAD, 0xF027 }, 	// MadCatzFPSProGamePad
			{ 0x1BAD, 0xF021 }, 	// MadCatzGhostReconFSGamePad
			{ 0x0738, 0x4736 }, 	// MadCatzMicroConGamePadPro
			{ 0x1BAD, 0xF036 }, 	// MadCatzMicroConGamePadProZ
			{ 0x0738, 0x9871 }, 	// MadCatzPortableDrumKit
			{ 0x0738, 0x4728 }, 	// MadCatzStreetFighterIVFightPad
			{ 0x0738, 0x4718 }, 	// MadCatzStreetFighterIVFightStickSE
			{ 0x0738, 0x4716 }, 	// MadCatzXbox360Controller
			{ 0x0738, 0x4726 }, 	// MadCatzXbox360Controller
			{ 0x0738, 0xBEEF }, 	// MadCatzXbox360Controller
			{ 0x1BAD, 0xF016 }, 	// MadCatzXbox360Controller
			{ 0x0738, 0xB726 }, 	// MadCatzXboxcontrollerMW2
			{ 0x045E, 0x028E }, 	// MicrosoftXbox360Controller
			{ 0x045E, 0x0719 }, 	// MicrosoftXbox360Controller
			{ 0x12AB, 0x0301 }, 	// PDPAFTERGLOWAX1
			{ 0x0E6F, 0x0105 }, 	// PDPDancePad
			{ 0x0E6F, 0x0201 }, 	// PelicanTSZ360Pad
			{ 0x15E4, 0x3F00 }, 	// PowerAMiniProElite
			{ 0x24C6, 0x5300 }, 	// PowerAMiniProEliteGlow
			{ 0x1BAD, 0xF504 }, 	// REALARCADEPROEX
			{ 0x1BAD, 0xF502 }, 	// REALARCADEProVX
			{ 0x1689, 0xFD00 }, 	// RazerOnza
			{ 0x1689, 0xFD01 }, 	// RazerOnzaTournamentEdition
			{ 0x1430, 0x4748 }, 	// RedOctaneGuitarHeroXplorer
			{ 0x0E6F, 0x011F }, 	// RockCandyGamepadforXbox360
			{ 0x12AB, 0x0006 }, 	// RockRevolutionforXbox360
			{ 0x0738, 0xCB02 }, 	// SaitekCyborgRumblePadPCXbox360
			{ 0x0738, 0xCB03 }, 	// SaitekP3200RumblePadPCXbox360
			{ 0x1BAD, 0xF028 }, 	// StreetFighterIVFightPad
			{ 0x0738, 0x4738 }, 	// StreetFighterIVFightStickTE
			{ 0x0738, 0xF738 }, 	// SuperSFIVFightStickTES
			{ 0x1BAD, 0xF903 }, 	// TronXbox360controller
			{ 0x1BAD, 0x5500 }, 	// USBGamepad
			{ 0x1BAD, 0xF906 }, 	// XB360MortalKombatFightStick
			{ 0x15E4, 0x3F0A }, 	// XboxAirflowiredcontroller
			{ 0x0E6F, 0x0401 },		// GameStop XBox 360 Controller
		};

		bool bIsXbox360Controller = false;

		for (int i = 0; i < UE_ARRAY_COUNT(XBox360Controllers); ++i)
		{
			const FXBox360ControllerID ControllerID = XBox360Controllers[i];
			if (ControllerID.VendorID == VendorID && ControllerID.ProductID == ProductID)
			{
				bIsXbox360Controller = true;
				break;
			}
		}

		if (bIsXbox360Controller)
		{
			// Xbox 360 Controller
			ButtonsMapping[1]	= 0;	// A
			ButtonsMapping[2]	= 1;	// B
			ButtonsMapping[3]	= 2;	// X
			ButtonsMapping[4]	= 3;	// Y
			ButtonsMapping[5]	= 4;	// Left Shoulder
			ButtonsMapping[6]	= 5;	// Right Shoulder
			ButtonsMapping[7]	= 8;	// Left Thumbstick
			ButtonsMapping[8]	= 9;	// Right Thumbstick
			ButtonsMapping[9]	= 6;	// Start
			ButtonsMapping[10]	= 7;	// Back
			ButtonsMapping[12]	= 12;	// DPad Up
			ButtonsMapping[13]	= 13;	// DPad Down
			ButtonsMapping[14]	= 14;	// DPad Left
			ButtonsMapping[15]	= 15;	// DPad Right

			LeftAnalogXMapping = kHIDUsage_GD_X;
			LeftAnalogYMapping = kHIDUsage_GD_Y;
			LeftTriggerAnalogMapping = kHIDUsage_GD_Z;
			RightAnalogXMapping = kHIDUsage_GD_Rx;
			RightAnalogYMapping = kHIDUsage_GD_Ry;
			RightTriggerAnalogMapping = kHIDUsage_GD_Rz;
		}
		else
		{
			// Generic (based on Logitech RumblePad 2)
			ButtonsMapping[1]	= 2;	// X
			ButtonsMapping[2]	= 0;	// A
			ButtonsMapping[3]	= 1;	// B
			ButtonsMapping[4]	= 3;	// Y
			ButtonsMapping[5]	= 4;	// Left Shoulder
			ButtonsMapping[6]	= 5;	// Right Shoulder
			ButtonsMapping[7]	= 10;	// Left Trigger
			ButtonsMapping[8]	= 11;	// Right Trigger
			ButtonsMapping[9]	= 7;	// Back
			ButtonsMapping[10]	= 6;	// Start
			ButtonsMapping[11]	= 8;	// Left Thumbstick
			ButtonsMapping[12]	= 9;	// Right Thumbstick

			LeftAnalogXMapping = kHIDUsage_GD_X;
			LeftAnalogYMapping = kHIDUsage_GD_Y;
			LeftTriggerAnalogMapping = kHIDUsage_GD_Rx;
			RightAnalogXMapping = kHIDUsage_GD_Z;
			RightAnalogYMapping = kHIDUsage_GD_Rz;
			RightTriggerAnalogMapping = kHIDUsage_GD_Ry;
		}
	}
}

void HIDInputInterface::OnNewHIDController(IOReturn Result, IOHIDDeviceRef DeviceRef)
{
	int32 ControllerIndex = -1;

	for (int32 Index = 0; Index < MAX_NUM_HIDINPUT_CONTROLLERS; ++Index)
	{
		if (ControllerStates[Index].Device.DeviceRef == NULL)
		{
			ControllerIndex = Index;
			break;
		}
	}

	if (ControllerIndex != -1 && (Result = IOHIDDeviceOpen(DeviceRef, kIOHIDOptionsTypeNone)) == kIOReturnSuccess)
	{
		CFArrayRef ElementsArray = IOHIDDeviceCopyMatchingElements(DeviceRef, NULL, 0);
		if (ElementsArray)
		{
			FHIDDeviceInfo& DeviceInfo = ControllerStates[ControllerIndex].Device;
			DeviceInfo.DeviceRef = DeviceRef;
			DeviceInfo.Elements.Empty();
			DeviceInfo.SetupMappings();

			CFIndex ElementsCount = CFArrayGetCount(ElementsArray);
			IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
			for (CFIndex ElementIndex = 0; ElementIndex < ElementsCount; ++ElementIndex)
			{
				FHIDElementInfo Element;
				Element.ElementRef = (IOHIDElementRef)CFArrayGetValueAtIndex(ElementsArray, ElementIndex);
				Element.Type = IOHIDElementGetType(Element.ElementRef);
				Element.UsagePage = IOHIDElementGetUsagePage(Element.ElementRef);
				Element.Usage = IOHIDElementGetUsage(Element.ElementRef);
				Element.MinValue = IOHIDElementGetLogicalMin(Element.ElementRef);
				Element.MaxValue = IOHIDElementGetLogicalMax(Element.ElementRef);

				if ((Element.Type == kIOHIDElementTypeInput_Button && Element.UsagePage == kHIDPage_Button && Element.Usage < MAX_NUM_CONTROLLER_BUTTONS) ||
					((Element.Type == kIOHIDElementTypeInput_Misc || Element.Type == kIOHIDElementTypeInput_Axis) && (Element.UsagePage == kHIDPage_GenericDesktop || Element.UsagePage == kHIDPage_Simulation)))
				{
					DeviceInfo.Elements.Add(Element);
				}
			}

			CFRelease(ElementsArray);

			FPlatformUserId PlatformUserId = PLATFORMUSERID_NONE;
			FInputDeviceId DeviceId = INPUTDEVICEID_NONE;

			DeviceMapper.RemapControllerIdToPlatformUserAndDevice(ControllerIndex, OUT PlatformUserId, OUT DeviceId);
			{
				static const FName HIDInputDeviceName(TEXT("HIDInputInterface"));
				FInputDeviceDescriptor Desc;
				Desc.HardwareDeviceHandle     = DeviceId;
				Desc.InputDeviceName          = HIDInputDeviceName;
				Desc.HardwareDeviceIdentifier = FName(TEXT("Mac_HIDController"));
				FInputDeviceRegistry::RegisterDevice(Desc);
			}
			DeviceMapper.Internal_MapInputDeviceToUser(DeviceId, PlatformUserId, EInputDeviceConnectionState::Connected);

			bIsGamepadAttached = true;
		}
		else
		{
			IOHIDDeviceClose(DeviceRef, kIOHIDOptionsTypeNone);
		}
	}
}

void HIDInputInterface::SendControllerEvents()
{
	for (int32 ControllerIndex = 0; ControllerIndex < MAX_NUM_HIDINPUT_CONTROLLERS; ++ControllerIndex)
	{
		FControllerState& ControllerState = ControllerStates[ControllerIndex];

		IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
		FPlatformUserId UserId = FGenericPlatformMisc::GetPlatformUserForUserIndex(ControllerState.ControllerId);
        FInputDeviceId DeviceId = DeviceMapper.GetPrimaryInputDeviceForUser(UserId);

		if( ControllerState.Device.DeviceRef )
		{
			bool CurrentButtonStates[MAX_NUM_CONTROLLER_BUTTONS] = {0};

			for (int Index = 0; Index < ControllerState.Device.Elements.Num(); ++Index)
			{
				FHIDElementInfo& Element = ControllerState.Device.Elements[Index];

				IOHIDValueRef ValueRef;
				IOReturn Result = IOHIDDeviceGetValue(ControllerState.Device.DeviceRef, Element.ElementRef, &ValueRef);
				if (Result == kIOReturnSuccess && IOHIDValueGetLength(ValueRef) <= sizeof(int32))
				{
					const int32 NewValue = IOHIDValueGetIntegerValue(ValueRef);
					if (Element.UsagePage == kHIDPage_Button)
					{
						const int8 MappedIndex = ControllerState.Device.ButtonsMapping[Element.Usage];
						if (MappedIndex != -1)
						{
							CurrentButtonStates[MappedIndex] = NewValue > 0;
						}
					}
					else
					{
						const bool bIsTrigger = Element.Usage == ControllerState.Device.LeftTriggerAnalogMapping || Element.Usage == ControllerState.Device.RightTriggerAnalogMapping;
						const float Percentage = FMath::GetRangePct((float)Element.MinValue, (float)Element.MaxValue, (float)NewValue);
						const float FloatValue = bIsTrigger ? Percentage : Percentage * 2.0f - 1.0f;

						if (Element.Usage == ControllerState.Device.LeftAnalogXMapping && ControllerState.LeftAnalogX != NewValue)
						{
							MessageHandler->OnControllerAnalog(FGamepadKeyNames::LeftAnalogX, UserId, DeviceId, FloatValue);
							CurrentButtonStates[18] = FloatValue < -0.2f;	// FGamepadKeyNames::LeftStickLeft
							CurrentButtonStates[19] = FloatValue > 0.2f;	// FGamepadKeyNames::LeftStickRight
							ControllerState.LeftAnalogX = NewValue;
						}
						else if (Element.Usage == ControllerState.Device.LeftAnalogYMapping && ControllerState.LeftAnalogY != NewValue)
						{
							MessageHandler->OnControllerAnalog(FGamepadKeyNames::LeftAnalogY, UserId, DeviceId, -FloatValue);
							CurrentButtonStates[16] = FloatValue < -0.2f;	// FGamepadKeyNames::LeftStickUp
							CurrentButtonStates[17] = FloatValue > 0.2f;	// FGamepadKeyNames::LeftStickDown
							ControllerState.LeftAnalogY = NewValue;
						}
						else if (Element.Usage == ControllerState.Device.RightAnalogXMapping && ControllerState.RightAnalogX != NewValue)
						{
							MessageHandler->OnControllerAnalog(FGamepadKeyNames::RightAnalogX, UserId, DeviceId, FloatValue);
							CurrentButtonStates[22] = FloatValue < -0.2f;	// FGamepadKeyNames::RightStickLeft
							CurrentButtonStates[23] = FloatValue > 0.2f;	// FGamepadKeyNames::RightStickDown
							ControllerState.RightAnalogX = NewValue;
						}
						else if (Element.Usage == ControllerState.Device.RightAnalogYMapping && ControllerState.RightAnalogY != NewValue)
						{
							MessageHandler->OnControllerAnalog(FGamepadKeyNames::RightAnalogY, UserId, DeviceId, -FloatValue);
							CurrentButtonStates[20] = FloatValue < -0.2f;	// FGamepadKeyNames::RightStickUp
							CurrentButtonStates[21] = FloatValue > 0.2f;	// FGamepadKeyNames::RightStickRight
							ControllerState.RightAnalogY = NewValue;
						}
						else if (Element.Usage == ControllerState.Device.LeftTriggerAnalogMapping)
						{
							if (ControllerState.LeftTriggerAnalog != NewValue)
							{
								MessageHandler->OnControllerAnalog(FGamepadKeyNames::LeftTriggerAnalog, UserId, DeviceId, FloatValue);
								ControllerState.LeftTriggerAnalog = NewValue;
							}
							CurrentButtonStates[10] = FloatValue > 0.01f;
						}
						else if (Element.Usage == ControllerState.Device.RightTriggerAnalogMapping)
						{
							if (ControllerState.RightTriggerAnalog != NewValue)
							{
								MessageHandler->OnControllerAnalog(FGamepadKeyNames::RightTriggerAnalog, UserId, DeviceId, FloatValue);
								ControllerState.RightTriggerAnalog = NewValue;
							}
							CurrentButtonStates[11] = FloatValue > 0.01f;
						}
						else if (Element.Usage == kHIDUsage_GD_Hatswitch)
						{
							// Need to account for Min and Max Value, this assumes 0-7, macOS reports 1-8 for bluetooth xbox one controller
							switch (NewValue - Element.MinValue)
							{
								case 0: CurrentButtonStates[12] = true;									break; // Up
								case 1: CurrentButtonStates[12] = true;	CurrentButtonStates[15] = true;	break; // Up + right
								case 2: CurrentButtonStates[15] = true;									break; // Right
								case 3: CurrentButtonStates[15] = true;	CurrentButtonStates[13] = true;	break; // Right + down
								case 4: CurrentButtonStates[13] = true;									break; // Down
								case 5: CurrentButtonStates[13] = true;	CurrentButtonStates[14] = true;	break; // Down + left
								case 6: CurrentButtonStates[14] = true;									break; // Left
								case 7: CurrentButtonStates[14] = true;	CurrentButtonStates[12] = true;	break; // Left + up
							}
						}
					}
				}
			}

			const double CurrentTime = FPlatformTime::Seconds();

			// For each button check against the previous state and send the correct message if any
			for (int32 ButtonIndex = 0; ButtonIndex < MAX_NUM_CONTROLLER_BUTTONS; ++ButtonIndex)
			{
				if (CurrentButtonStates[ButtonIndex] != ControllerState.ButtonStates[ButtonIndex])
				{
					if (CurrentButtonStates[ButtonIndex])
					{
						MessageHandler->OnControllerButtonPressed(Buttons[ButtonIndex], UserId, DeviceId, false);
					}
					else
					{
						MessageHandler->OnControllerButtonReleased(Buttons[ButtonIndex], UserId, DeviceId, false);
					}

					if (CurrentButtonStates[ButtonIndex] != 0)
					{
						// this button was pressed - set the button's NextRepeatTime to the InitialButtonRepeatDelay
						ControllerState.NextRepeatTime[ButtonIndex] = CurrentTime + InitialButtonRepeatDelay;
					}
				}
				else if (CurrentButtonStates[ButtonIndex] != 0 && ControllerState.NextRepeatTime[ButtonIndex] <= CurrentTime)
				{
					MessageHandler->OnControllerButtonPressed(Buttons[ButtonIndex], UserId, DeviceId, true);

					// set the button's NextRepeatTime to the ButtonRepeatDelay
					ControllerState.NextRepeatTime[ButtonIndex] = CurrentTime + ButtonRepeatDelay;
				}

				// Update the state for next time
				ControllerState.ButtonStates[ButtonIndex] = CurrentButtonStates[ButtonIndex];
			}	
		}
	}
}

CFMutableDictionaryRef HIDInputInterface::CreateDeviceMatchingDictionary(UInt32 UsagePage, UInt32 Usage)
{
	// Create a dictionary to add usage page/usages to
	CFMutableDictionaryRef Dict = CFDictionaryCreateMutable(kCFAllocatorDefault, 0, &kCFTypeDictionaryKeyCallBacks, &kCFTypeDictionaryValueCallBacks);
	if (Dict)
	{
		// Add key for device type to refine the matching dictionary.
		CFNumberRef PageCFNumberRef = CFNumberCreate(kCFAllocatorDefault, kCFNumberIntType, &UsagePage);
		if (PageCFNumberRef)
		{
			CFDictionarySetValue(Dict, CFSTR(kIOHIDDeviceUsagePageKey), PageCFNumberRef);
			CFRelease(PageCFNumberRef);

			// Note: the usage is only valid if the usage page is also defined
			CFNumberRef UsageCFNumberRef = CFNumberCreate(kCFAllocatorDefault, kCFNumberIntType, &Usage);
			if (UsageCFNumberRef)
			{
				CFDictionarySetValue(Dict, CFSTR(kIOHIDDeviceUsageKey), UsageCFNumberRef);
				CFRelease(UsageCFNumberRef);

				return Dict;
			}
		}
	}

	return NULL;
}

void HIDInputInterface::HIDDeviceMatchingCallback(void* Context, IOReturn Result, void* Sender, IOHIDDeviceRef DeviceRef)
{
	HIDInputInterface* HIDInput = (HIDInputInterface*)Context;
	HIDInput->OnNewHIDController(Result, DeviceRef);
}

void HIDInputInterface::HIDDeviceRemovalCallback(void* Context, IOReturn Result, void* Sender, IOHIDDeviceRef DeviceRef)
{
	HIDInputInterface* HIDInput = (HIDInputInterface*)Context;
	IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();

	for (int32 Index = 0; Index < MAX_NUM_HIDINPUT_CONTROLLERS; ++Index)
	{
		if(HIDInput->ControllerStates[Index].Device.DeviceRef == DeviceRef)
		{
			FPlatformUserId PlatformUserId = PLATFORMUSERID_NONE;
			FInputDeviceId DeviceId = INPUTDEVICEID_NONE;

			DeviceMapper.RemapControllerIdToPlatformUserAndDevice(Index, OUT PlatformUserId, OUT DeviceId);
			DeviceMapper.Internal_MapInputDeviceToUser(DeviceId, PlatformUserId, EInputDeviceConnectionState::Disconnected);

			HIDInput->ControllerStates[Index].Device.DeviceRef = NULL;
			break;
		}
	}

	HIDInput->bIsGamepadAttached = false;
	for (int32 Index = 0; Index < MAX_NUM_HIDINPUT_CONTROLLERS; ++Index)
	{
		if (HIDInput->ControllerStates[Index].Device.DeviceRef)
		{
			HIDInput->bIsGamepadAttached = true;
			break;
		}
	}
}
