// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <IOKit/hid/IOHIDLib.h>
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "Templates/SharedPointer.h"

#define MAX_NUM_HIDINPUT_CONTROLLERS 4

/** Max number of controller buttons.  Must be < 256*/
#define MAX_NUM_CONTROLLER_BUTTONS 24

/**
 * Interface class for HID Input devices
 */
class HIDInputInterface
{
public:

	static TSharedRef<HIDInputInterface> Create(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler);

	void SendControllerEvents();

	void SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
	{
		MessageHandler = InMessageHandler;
	}

	bool IsGamepadAttached() const { return bIsGamepadAttached; }

private:

	HIDInputInterface(const TSharedRef<FGenericApplicationMessageHandler>& MessageHandler);

	void OnNewHIDController(IOReturn Result, IOHIDDeviceRef DeviceRef);

	static CFMutableDictionaryRef CreateDeviceMatchingDictionary(UInt32 UsagePage, UInt32 Usage);
	static void HIDDeviceMatchingCallback(void* Context, IOReturn Result, void* Sender, IOHIDDeviceRef DeviceRef);
	static void HIDDeviceRemovalCallback(void* Context, IOReturn Result, void* Sender, IOHIDDeviceRef DeviceRef);

private:

	struct FHIDElementInfo
	{
		IOHIDElementRef ElementRef;
		IOHIDElementType Type;
		uint32 UsagePage;
		uint32 Usage;
		int32 MinValue;
		int32 MaxValue;
	};

	struct FHIDDeviceInfo
	{
		IOHIDDeviceRef DeviceRef;
		TArray<FHIDElementInfo> Elements;
		int8 ButtonsMapping[MAX_NUM_CONTROLLER_BUTTONS];
		uint16 LeftAnalogXMapping;
		uint16 LeftAnalogYMapping;
		uint16 LeftTriggerAnalogMapping;
		uint16 RightAnalogXMapping;
		uint16 RightAnalogYMapping;
		uint16 RightTriggerAnalogMapping;

		void SetupMappings();
	};

	struct FXBox360ControllerID
	{
		const int32 VendorID;
		const int32 ProductID;
	};

	struct FControllerState
	{
		/** Last frame's button states, so we only send events on edges */
		bool ButtonStates[MAX_NUM_CONTROLLER_BUTTONS];

		/** Next time a repeat event should be generated for each button */
		double NextRepeatTime[MAX_NUM_CONTROLLER_BUTTONS];

		/** Raw Left thumb x analog value */
		int32 LeftAnalogX;
		
		/** Raw left thumb y analog value */
		int32 LeftAnalogY;

		/** Raw Right thumb x analog value */
		int32 RightAnalogX;

		/** Raw Right thumb x analog value */
		int32 RightAnalogY;

		/** Left Trigger analog value */
		int32 LeftTriggerAnalog;

		/** Right trigger analog value */
		int32 RightTriggerAnalog;

		/** Id of the controller */
		int32 ControllerId;

		FHIDDeviceInfo Device;
	};

private:

	/** Names of all the buttons */
	FGamepadKeyNames::Type Buttons[MAX_NUM_CONTROLLER_BUTTONS];

	/** Controller states */
	FControllerState ControllerStates[MAX_NUM_HIDINPUT_CONTROLLERS];

	/** Delay before sending a repeat message after a button was first pressed */
	float InitialButtonRepeatDelay;

	/** Delay before sendign a repeat message after a button has been pressed for a while */
	float ButtonRepeatDelay;

	bool bIsGamepadAttached;

	IOHIDManagerRef HIDManager;

	TSharedRef<FGenericApplicationMessageHandler> MessageHandler;
};
