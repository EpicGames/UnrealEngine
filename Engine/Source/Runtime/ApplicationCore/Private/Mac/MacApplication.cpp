// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/MacApplication.h"
#include "Mac/MacWindow.h"
#include "Mac/MacCursor.h"
#include "Mac/CocoaMenu.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "IInputDeviceModule.h"
#include "IInputDevice.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "Mac/CocoaThread.h"
#include "Modules/ModuleManager.h"
#include "Mac/CocoaTextView.h"
#include "Misc/ScopeLock.h"
#include "Misc/App.h"
#include "Mac/MacPlatformApplicationMisc.h"
#include "HAL/ThreadHeartBeat.h"
#include "IHapticDevice.h"
#include "Apple/ApplePlatformCrashContext.h"
#if WITH_ACCESSIBILITY
#include "Mac/Accessibility/MacAccessibilityManager.h"
#include "Mac/Accessibility/MacAccessibilityElement.h"
#endif


#include <IOKit/IOKitLib.h>
#include <IOKit/graphics/IOGraphicsLib.h>
#include "Misc/CoreDelegates.h"

#include "Apple/ScopeAutoreleasePool.h"
#include "Apple/AppleStringUtils.h"
#include "Apple/AppleControllerInterface.h"
#include "Apple/AppleMouseController.h"
#include "HIDInputInterface.h"

FMacApplication* MacApplication = nullptr;

static FCriticalSection GAllScreensMutex;
static TArray<FMacScreenRef> GAllScreens;

const uint32 RESET_EVENT_SUBTYPE = 0x0f00;

#if WITH_EDITOR
typedef int32 (*MTContactCallbackFunction)(void*, void*, int32, double, int32);
extern "C" CFMutableArrayRef MTDeviceCreateList(void);
extern "C" void MTRegisterContactFrameCallback(void*, MTContactCallbackFunction);
extern "C" void MTDeviceStart(void*, int);
extern "C" bool MTDeviceIsBuiltIn(void*);
#endif

namespace MacApplicationCVars
{
	static bool bUseNewMouseControllerForMovement = true;
	static FAutoConsoleVariableRef CVarUseNewMouseControllerMovement(
		TEXT("Slate.MacUseNewMouseControllerMovement"),
		bUseNewMouseControllerForMovement,
		TEXT("false: AppKit handles mouse movement in all modes (legacy). true: FAppleMouseController handles movement during high-precision mode; AppKit handles normal mode."),
		ECVF_ReadOnly);
}

// Simple wrapper for Legacy HID Implementation and AppleControllerInterface GCController implementation selection
// Remove this wrapper and HIDInputInterface once 10.14.0 is no longer supported
static TAutoConsoleVariable<int32> CVarMacControllerPreferGCImpl(
	TEXT("Slate.MacControllerPreferGCImpl"),
	1,
	TEXT("Prefer Selection of Mac Controller implementation:\n")
	TEXT("\t0: Use legacy HID System.\n")
	TEXT("\t1: Use GCController implementation.\n")
	TEXT("Default is 1."),
	ECVF_ReadOnly);

class FMacControllerInterface
{
public:
	void SetMessageHandler(const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler)
	{
		if(AppleControllerInterface.IsValid()) 		AppleControllerInterface->SetMessageHandler(InMessageHandler);
		else if(HIDControllerInterface.IsValid()) 	HIDControllerInterface->SetMessageHandler(InMessageHandler);
	}
	void Tick(float DeltaTime)
	{
		if(AppleControllerInterface.IsValid()) 		AppleControllerInterface->Tick(DeltaTime);
		// No tick in HIDInputInterface
	}
	void SendControllerEvents()
	{
		if(AppleControllerInterface.IsValid()) 		AppleControllerInterface->SendControllerEvents();
		else if(HIDControllerInterface.IsValid()) 	HIDControllerInterface->SendControllerEvents();
	}
	bool IsGamepadAttached() const
	{
		if(AppleControllerInterface.IsValid()) 		return AppleControllerInterface->IsGamepadAttached();
		else if(HIDControllerInterface.IsValid()) 	return HIDControllerInterface->IsGamepadAttached();
		return false;
	}

	// IInputInterface forwarders. The legacy HIDInputInterface has no haptics or lightbar, so these are
	// only meaningful when the GameController-based path is selected
	void SetForceFeedbackChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value)
	{
		if (AppleControllerInterface.IsValid()) AppleControllerInterface->SetForceFeedbackChannelValue(ControllerId, ChannelType, Value);
	}
	void SetForceFeedbackChannelValues(int32 ControllerId, const FForceFeedbackValues& Values)
	{
		if (AppleControllerInterface.IsValid()) AppleControllerInterface->SetForceFeedbackChannelValues(ControllerId, Values);
	}
	void SetLightColor(int32 ControllerId, FColor Color)
	{
		if (AppleControllerInterface.IsValid()) AppleControllerInterface->SetLightColor(ControllerId, Color);
	}
	void ResetLightColor(int32 ControllerId)
	{
		if (AppleControllerInterface.IsValid()) AppleControllerInterface->ResetLightColor(ControllerId);
	}
	void SetDeviceProperty(int32 ControllerId, const FInputDeviceProperty* Property)
	{
		if (AppleControllerInterface.IsValid()) AppleControllerInterface->SetDeviceProperty(ControllerId, Property);
	}

	FMacControllerInterface(const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler)
	{
		if(CVarMacControllerPreferGCImpl.GetValueOnAnyThread() > 0)
		{
			AppleControllerInterface = FAppleControllerInterface::Create(InMessageHandler);
		}
		else
		{
			HIDControllerInterface = HIDInputInterface::Create(InMessageHandler);
		}
	}
private:
	TSharedPtr<FAppleControllerInterface> AppleControllerInterface;
	TSharedPtr<HIDInputInterface> HIDControllerInterface;
};

FMacApplication::MenuBarShutdownFuncPtr FMacApplication::MenuBarShutdownFunc = nullptr;

FMacApplication* FMacApplication::CreateMacApplication()
{
	MacApplication = new FMacApplication();
	return MacApplication;
}

FMacApplication::FMacApplication()
:	GenericApplication(MakeShareable(new FMacCursor()))
,	bUsingHighPrecisionMouseInput(false)
,	bUsingTrackpad(false)
,	LastPressedMouseButton(EMouseButtons::Invalid)
,	bIsProcessingDeferredEvents(false)
, 	HIDInput(MakeShareable(new FMacControllerInterface(MessageHandler)))
,   bHasLoadedInputPlugins(false)
,	DraggedWindow(nullptr)
,	WindowUnderCursor(nullptr)
,   SystemResolutionX(0)
,   SystemResolutionY(0)
,	bSystemModalMode(false)
,	ModifierKeysFlags(0)
,	CurrentModifierFlags(0)
,	bIsRightClickEmulationEnabled(true)
,	bEmulatingRightClick(false)
,	bIgnoreMouseMoveDelta(0)
,	bMouseInputNeedsReassociation(false)
,	bIsWorkspaceSessionActive(true)
,	KeyBoardLayoutData(nil)
#if WITH_ACCESSIBILITY
,	AccessibilityCacheTimer(Nil)
,	AccessibilityAnnouncementDelayTimer(Nil)
#endif
,	LastNativeEvent(nil)
,	bDisplayReconfiguring(false)
{
	TextInputMethodSystem = MakeShareable(new FMacTextInputMethodSystem);
	if (!TextInputMethodSystem->Initialize())
	{
		TextInputMethodSystem.Reset();
	}

	MainThreadCall(^{
		AppActivationObserver = [[NSNotificationCenter defaultCenter] addObserverForName:NSApplicationDidBecomeActiveNotification
																				  object:[NSApplication sharedApplication]
																				   queue:[NSOperationQueue mainQueue]
																			  usingBlock:^(NSNotification* Notification) { OnApplicationDidBecomeActive(); }];

		AppDeactivationObserver = [[NSNotificationCenter defaultCenter] addObserverForName:NSApplicationWillResignActiveNotification
																					object:[NSApplication sharedApplication]
																					 queue:[NSOperationQueue mainQueue]
																				usingBlock:^(NSNotification* Notification) { OnApplicationWillResignActive(); }];

		WorkspaceActivationObserver = [[[NSWorkspace sharedWorkspace] notificationCenter] addObserverForName:NSWorkspaceSessionDidBecomeActiveNotification
																									  object:[NSWorkspace sharedWorkspace]
																									   queue:[NSOperationQueue mainQueue]
																								  usingBlock:^(NSNotification* Notification){ bIsWorkspaceSessionActive = true; }];

		WorkspaceDeactivationObserver = [[[NSWorkspace sharedWorkspace] notificationCenter] addObserverForName:NSWorkspaceSessionDidResignActiveNotification
																										object:[NSWorkspace sharedWorkspace]
																										 queue:[NSOperationQueue mainQueue]
																									usingBlock:^(NSNotification* Notification){ bIsWorkspaceSessionActive = false; }];

		WorkspaceActiveSpaceChangeObserver = [[[NSWorkspace sharedWorkspace] notificationCenter] addObserverForName:NSWorkspaceActiveSpaceDidChangeNotification
																											 object:[NSWorkspace sharedWorkspace]
																											  queue:[NSOperationQueue mainQueue]
																										 usingBlock:^(NSNotification* Notification){ OnActiveSpaceDidChange(); }];

		ScreensDidWakeObserver = [[[NSWorkspace sharedWorkspace] notificationCenter] addObserverForName:NSWorkspaceScreensDidWakeNotification
																								object:[NSWorkspace sharedWorkspace]
																								 queue:[NSOperationQueue mainQueue]
																							usingBlock:^(NSNotification* Notification){ ReassociateMouseInput(); }];

		SystemDidWakeObserver = [[[NSWorkspace sharedWorkspace] notificationCenter] addObserverForName:NSWorkspaceDidWakeNotification
																							   object:[NSWorkspace sharedWorkspace]
																								queue:[NSOperationQueue mainQueue]
																						   usingBlock:^(NSNotification* Notification){ ReassociateMouseInput(); }];

		MouseMovedEventMonitor = [NSEvent addGlobalMonitorForEventsMatchingMask:NSEventMaskMouseMoved handler:^(NSEvent* Event) { DeferEvent(Event); }];
		EventMonitor = [NSEvent addLocalMonitorForEventsMatchingMask:NSEventMaskAny handler:^(NSEvent* Event) { return HandleNSEvent(Event); }];

		CGDisplayRegisterReconfigurationCallback(FMacApplication::OnDisplayReconfiguration, this);
		
		CacheKeyboardInputSource();

		WindowUnderCursor = [FindSlateWindowUnderCursor() retain];
	});

#if WITH_EDITOR
	NSMutableArray* MultiTouchDevices = (__bridge NSMutableArray*)MTDeviceCreateList();
	for (id Device in MultiTouchDevices)
	{
		MTRegisterContactFrameCallback((void*)Device, FMacApplication::MTContactCallback);
		MTDeviceStart((void*)Device, 0);
	}
	[MultiTouchDevices release];

	FMemory::Memzero(GestureUsage);
	LastGestureUsed = EGestureEvent::None;

	FCoreDelegates::PreSlateModalWithContext.AddRaw(this, &FMacApplication::StartScopedModalEvent);
    FCoreDelegates::PostSlateModalWithContext.AddRaw(this, &FMacApplication::EndScopedModalEvent);
#endif

	FMacApplication::OnDisplayReconfiguration(kCGNullDirectDisplay, kCGDisplayDesktopShapeChangedFlag, this);
    
    // Listen for res changes to update targeted fullscreen res
    FCoreDelegates::OnSystemResolutionChanged.AddLambda([this](uint32 ResolutionX, uint32 ResolutionY) {
        SystemResolutionX = ResolutionX;
        SystemResolutionY = ResolutionY;
    });

	if (MacApplicationCVars::bUseNewMouseControllerForMovement)
	{
		// Only use the movement capabilities from the controller. For button input handling we need the context about on which window the event is happening
		// but that is not supported by GC Mouse yet. In this setup we use Appkit to handle Mouse button and non precision movement modes, and the new Mouse controller
		// (GC Mouse based) for high precision movement
		MouseController = MakeShared<FAppleMouseController>(MessageHandler, EMouseControllerCapabilities::Movement);
		MouseController->OnMovementDispatched().AddRaw(this, &FMacApplication::HandleMouseControllerMovement);
	}
}

FMacApplication::~FMacApplication()
{
	IModularFeatures::Get().OnModularFeatureRegistered().RemoveAll(this);

	if (MenuBarShutdownFunc)
	{
		MenuBarShutdownFunc();
	}

	MainThreadCall(^{
#if WITH_ACCESSIBILITY
		if(GetAccessibleMessageHandler()->IsActive())
		{
			OnVoiceoverDisabled();
		}
		if(AccessibilityCacheTimer != Nil)
		{
			[AccessibilityCacheTimer release];
		}
		
		if(AccessibilityAnnouncementDelayTimer)
		{
			[AccessibilityAnnouncementDelayTimer release];
		}
		[[FMacAccessibilityManager AccessibilityManager] TearDown];
#endif

		if (MouseMovedEventMonitor)
		{
			[NSEvent removeMonitor:MouseMovedEventMonitor];
		}
		if (EventMonitor)
		{
			[NSEvent removeMonitor:EventMonitor];
		}
		if (AppActivationObserver)
		{
			[[NSNotificationCenter defaultCenter] removeObserver:AppActivationObserver];
		}
		if (AppDeactivationObserver)
		{
			[[NSNotificationCenter defaultCenter] removeObserver:AppDeactivationObserver];
		}
		if (WorkspaceActivationObserver)
		{
			[[NSNotificationCenter defaultCenter] removeObserver:WorkspaceActivationObserver];
		}
		if (WorkspaceDeactivationObserver)
		{
			[[NSNotificationCenter defaultCenter] removeObserver:WorkspaceDeactivationObserver];
		}
		if (WorkspaceActiveSpaceChangeObserver)
		{
			[[NSNotificationCenter defaultCenter] removeObserver:WorkspaceActiveSpaceChangeObserver];
		}
		if (ScreensDidWakeObserver)
		{
			[[[NSWorkspace sharedWorkspace] notificationCenter] removeObserver:ScreensDidWakeObserver];
		}
		if (SystemDidWakeObserver)
		{
			[[[NSWorkspace sharedWorkspace] notificationCenter] removeObserver:SystemDidWakeObserver];
		}

		CGDisplayRemoveReconfigurationCallback(FMacApplication::OnDisplayReconfiguration, this);
		
		if(KeyBoardLayoutData != nil)
		{
			[KeyBoardLayoutData release];
			KeyBoardLayoutData = nil;
		}
		
		[WindowUnderCursor release];
	});

	if (TextInputMethodSystem.IsValid())
	{
		TextInputMethodSystem->Terminate();
	}
#if WITH_EDITOR
    FCoreDelegates::PreModal.RemoveAll(this);
    FCoreDelegates::PostModal.RemoveAll(this);
#endif

	if (MouseController)
	{
		MouseController->OnMovementDispatched().RemoveAll(this);
		MouseController.Reset();
	}

	MacApplication = nullptr;
}

void FMacApplication::SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
{
	GenericApplication::SetMessageHandler(InMessageHandler);
	HIDInput->SetMessageHandler(InMessageHandler);

	if (MouseController)
	{
		MouseController->SetMessageHandler(InMessageHandler);
	}
}

#if WITH_ACCESSIBILITY
void FMacApplication::SetAccessibleMessageHandler(const TSharedRef<FGenericAccessibleMessageHandler>& InAccessibleMessageHandler)
{
	GenericApplication::SetAccessibleMessageHandler(InAccessibleMessageHandler);
	// We register the primary user (keyboard).
	// This user is what Mac Voiceover will interact with 
	FGenericAccessibleUserRegistry& UserRegistry = AccessibleMessageHandler->GetAccessibleUserRegistry();
	// We failed to register the primary user, this should only happen if another user with the 0th index has already been registered.
	ensure(UserRegistry.RegisterUser(MakeShared<FGenericAccessibleUser>(FGenericAccessibleUserRegistry::GetPrimaryUserIndex())));
	InAccessibleMessageHandler->SetAccessibleEventDelegate(FGenericAccessibleMessageHandler::FAccessibleEvent::CreateRaw(this, &FMacApplication::OnAccessibleEventRaised));
	
	MainThreadCall(^{
		// Initializing FMacAccessibilityManager singleton to enable KVO registration for Voiceover changes
		[FMacAccessibilityManager AccessibilityManager].MacApplication = this;
		const bool bVoiceoverEnabled = [NSWorkspace sharedWorkspace].isVoiceOverEnabled;
		if(bVoiceoverEnabled)
		{
			OnVoiceoverEnabled();
		}
	}, false);
}
#endif

void FMacApplication::PollGameDeviceState(const float TimeDelta)
{
	if (!bHasLoadedInputPlugins)
	{
		// Create all externally-implemented input devices from loaded modules.
		TArray<IInputDeviceModule*> PluginImplementations = IModularFeatures::Get().GetModularFeatureImplementations<IInputDeviceModule>( IInputDeviceModule::GetModularFeatureName() );
		for( auto InputPluginIt = PluginImplementations.CreateIterator(); InputPluginIt; ++InputPluginIt )
		{
			TSharedPtr<IInputDevice> Device = (*InputPluginIt)->CreateInputDevice(MessageHandler);
			if (Device.IsValid())
			{
				UE_LOGF(LogInit, Log, "Adding external input plugin.");
				ExternalInputDevices.Add(MoveTemp(Device));
			}
		}

		// Modules loaded after this point are handled by this callback
		IModularFeatures::Get().OnModularFeatureRegistered().AddRaw(this, &FMacApplication::OnInputDeviceModuleRegistered);

		bHasLoadedInputPlugins = true;
	}

	// Poll game device state and send new events
	HIDInput->Tick( TimeDelta );
	HIDInput->SendControllerEvents();

	if (MouseController)
	{
		MouseController->SendControllerEvents();
	}

	// Poll externally-implemented devices
	for( auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt )
	{
		(*DeviceIt)->Tick( TimeDelta );
		(*DeviceIt)->SendControllerEvents();
	}
}

void FMacApplication::PumpMessages(const float TimeDelta)
{
	FPlatformApplicationMisc::PumpMessages(true);
}

void FMacApplication::DismissAllMenusOnGameThread()
{
	GameThreadCall(^{
		MessageHandler->DismissAllMenus();
	}, /*bWait=*/false, @[ NSDefaultRunLoopMode, UnrealResizeEventMode ]);
}

void FMacApplication::ProcessDeferredEvents(const float TimeDelta)
{
	TArray<FDeferredMacEvent> EventsToProcess;

	EventsMutex.Lock();
	EventsToProcess.Append(DeferredEvents);
	DeferredEvents.Empty();
	EventsMutex.Unlock();

	const bool bAlreadyProcessingDeferredEvents = bIsProcessingDeferredEvents;
	bIsProcessingDeferredEvents = true;

	for (int32 Index = 0; Index < EventsToProcess.Num(); ++Index)
	{
		ProcessEvent(EventsToProcess[Index]);
	}

	bIsProcessingDeferredEvents = bAlreadyProcessingDeferredEvents;

	InvalidateTextLayouts();
	CloseQueuedWindows();
}

TSharedRef<FGenericWindow> FMacApplication::MakeWindow()
{
	return FMacWindow::Make();
}

void FMacApplication::InitializeWindow(const TSharedRef<FGenericWindow>& InWindow, const TSharedRef<FGenericWindowDefinition>& InDefinition, const TSharedPtr<FGenericWindow>& InParent, const bool bShowImmediately)
{
	const TSharedRef<FMacWindow> Window = StaticCastSharedRef<FMacWindow >(InWindow);
	const TSharedPtr<FMacWindow> ParentWindow = StaticCastSharedPtr<FMacWindow>(InParent);

	{
		FScopeLock Lock(&WindowsMutex);
		Windows.Add(Window);
	}

	Window->Initialize(this, InDefinition, ParentWindow, bShowImmediately);
}

FModifierKeysState FMacApplication::GetModifierKeys() const
{
	uint32 CurrentFlags = ModifierKeysFlags;

	const bool bIsLeftShiftDown			= (CurrentFlags & (1 << 0)) != 0;
	const bool bIsRightShiftDown		= (CurrentFlags & (1 << 1)) != 0;
	const bool bIsLeftControlDown		= (CurrentFlags & (1 << 6)) != 0; // Mac pretends the Command key is Control
	const bool bIsRightControlDown		= (CurrentFlags & (1 << 7)) != 0; // Mac pretends the Command key is Control
	const bool bIsLeftAltDown			= (CurrentFlags & (1 << 4)) != 0;
	const bool bIsRightAltDown			= (CurrentFlags & (1 << 5)) != 0;
	const bool bIsLeftCommandDown		= (CurrentFlags & (1 << 2)) != 0; // Mac pretends the Control key is Command
	const bool bIsRightCommandDown		= (CurrentFlags & (1 << 3)) != 0; // Mac pretends the Control key is Command
	const bool bAreCapsLocked			= (CurrentFlags & (1 << 8)) != 0;

	return FModifierKeysState(bIsLeftShiftDown, bIsRightShiftDown, bIsLeftControlDown, bIsRightControlDown, bIsLeftAltDown, bIsRightAltDown, bIsLeftCommandDown, bIsRightCommandDown, bAreCapsLocked);
}

bool FMacApplication::IsCursorDirectlyOverSlateWindow() const
{
	return WindowUnderCursor && WindowUnderCursor != DraggedWindow;
}

TSharedPtr<FGenericWindow> FMacApplication::GetWindowUnderCursor()
{
	if (WindowUnderCursor && WindowUnderCursor->bAcceptsInput)
	{
		return StaticCastSharedPtr<FGenericWindow>(FindWindowByNSWindow(WindowUnderCursor));
	}
	return TSharedPtr<FMacWindow>(nullptr);
}

void FMacApplication::SetHighPrecisionMouseMode(const bool Enable, const TSharedPtr<FGenericWindow>& InWindow)
{
	UE_LOGF(LogAppleController, Verbose, "[%s] HighPrecisionMouseMode %ls", __func__, Enable ? TEXT("Enabled") : TEXT("Disabled"));

	bUsingHighPrecisionMouseInput = Enable;
	((FMacCursor*)Cursor.Get())->SetHighPrecisionMouseMode(Enable);

	if (MouseController && MacApplicationCVars::bUseNewMouseControllerForMovement)
	{
		if (Enable && bMouseInputNeedsReassociation.load(std::memory_order_acquire))
		{
			MouseController->RebindMouseHandlers();
		}
		MouseController->SetEnabled(Enable);
	}
}

void FMacApplication::ReassociateMouseInput()
{
	CGAssociateMouseAndMouseCursorPosition(true);

	if (MouseController && MacApplicationCVars::bUseNewMouseControllerForMovement)
	{
		MouseController->SuspendMovementHandler();
	}

	bMouseInputNeedsReassociation.store(true, std::memory_order_release);
}

bool FMacApplication::IsGamepadAttached() const
{
	return HIDInput->IsGamepadAttached();
}

FPlatformRect FMacApplication::GetWorkArea(const FPlatformRect& CurrentWindow) const
{
	SCOPED_AUTORELEASE_POOL;

	FMacScreenRef Screen = FindScreenBySlatePosition(CurrentWindow.Left, CurrentWindow.Top);

	const NSRect VisibleFrame = Screen->VisibleFramePixels;

	FPlatformRect WorkArea;
	WorkArea.Left = FMath::TruncToInt(VisibleFrame.origin.x);
	WorkArea.Top = FMath::TruncToInt(VisibleFrame.origin.y);
	WorkArea.Right = WorkArea.Left + FMath::TruncToInt(VisibleFrame.size.width);
	WorkArea.Bottom = WorkArea.Top + FMath::TruncToInt(VisibleFrame.size.height);

	return WorkArea;
}

#if WITH_EDITOR
void FMacApplication::SendAnalytics(IAnalyticsProvider* Provider)
{
	static_assert(((uint32)EGestureEvent::Count) == 9, "If the number of gestures changes you need to add more entries below!");

	TArray<FAnalyticsEventAttribute> GestureAttributes;
	GestureAttributes.Add(FAnalyticsEventAttribute(FString("Scroll"),	GestureUsage[(int32)EGestureEvent::Scroll]));
	GestureAttributes.Add(FAnalyticsEventAttribute(FString("Magnify"),	GestureUsage[(int32)EGestureEvent::Magnify]));
	GestureAttributes.Add(FAnalyticsEventAttribute(FString("Swipe"),	GestureUsage[(int32)EGestureEvent::Swipe]));
	GestureAttributes.Add(FAnalyticsEventAttribute(FString("Rotate"),	GestureUsage[(int32)EGestureEvent::Rotate]));
	GestureAttributes.Add(FAnalyticsEventAttribute(FString("Pan"),	GestureUsage[(int32)EGestureEvent::Pan]));
	GestureAttributes.Add(FAnalyticsEventAttribute(FString("Tap"),	GestureUsage[(int32)EGestureEvent::Tap]));
	GestureAttributes.Add(FAnalyticsEventAttribute(FString("LongPress"),	GestureUsage[(int32)EGestureEvent::LongPress]));
	GestureAttributes.Add(FAnalyticsEventAttribute(FString("Flick"),	GestureUsage[(int32)EGestureEvent::Flick]));

	Provider->RecordEvent(FString("Mac.Gesture.Usage"), GestureAttributes);

	FMemory::Memzero(GestureUsage);
	LastGestureUsed = EGestureEvent::None;
}
void FMacApplication::StartScopedModalEvent(const FCoreDelegates::FModalWindowContext& Context)
{
    FPlatformApplicationMisc::bMacApplicationModalMode = true;
    FPlatformApplicationMisc::bChachedMacMenuStateNeedsUpdate = true;
}

void FMacApplication::EndScopedModalEvent(const FCoreDelegates::FModalWindowContext& Context)
{
    FPlatformApplicationMisc::bMacApplicationModalMode = false;
    FPlatformApplicationMisc::bChachedMacMenuStateNeedsUpdate = true;
}
#endif

void FMacApplication::CloseWindow(TSharedRef<FMacWindow> Window)
{
	FScopeLock Lock(&WindowsToCloseMutex);
	if (!SlateWindowsToClose.Contains(Window))
	{
		SlateWindowsToClose.Add(Window);
	}
}

void FMacApplication::DeferEvent(NSObject* Object)
{
	FDeferredMacEvent DeferredEvent;

	FCocoaWindow* CursorWindow = FindSlateWindowUnderCursor();
	if (WindowUnderCursor != CursorWindow)
	{
		[WindowUnderCursor release];
		WindowUnderCursor = [CursorWindow retain];
	}

	if (Object && [Object isKindOfClass:[NSEvent class]])
	{
		NSEvent* Event = (NSEvent*)Object;
		DeferredEvent.Window = FindEventWindow(Event);
		DeferredEvent.Event = [Event retain];
		DeferredEvent.Type = [Event type];
		DeferredEvent.LocationInWindow = FVector2D([Event locationInWindow].x, [Event locationInWindow].y);
		DeferredEvent.ModifierFlags = [Event modifierFlags];
		DeferredEvent.Timestamp = [Event timestamp];
		DeferredEvent.WindowNumber = [Event windowNumber];
		DeferredEvent.Context = nil;

		if (DeferredEvent.Type == NSEventTypeKeyDown)
		{
			// In Unreal the main window rather than key window is the current active window in Slate, so the main window may be the one we want to send immKeyDown to,
			// for example in case of search text edit fields in context menus.
			NSWindow* MainWindow = [NSApp mainWindow];
			FCocoaWindow* IMMWindow = [MainWindow isKindOfClass:[FCocoaWindow class]] ? (FCocoaWindow*)MainWindow : DeferredEvent.Window;
			if (IMMWindow && [IMMWindow openGLView])
			{
				FCocoaTextView* View = (FCocoaTextView*)[IMMWindow openGLView];
				if (View && [View imkKeyDown:Event])
				{
					return;
				}
			}
		}

		switch (DeferredEvent.Type)
		{
			case NSEventTypeMouseMoved:
			case NSEventTypeLeftMouseDragged:
			case NSEventTypeRightMouseDragged:
			case NSEventTypeOtherMouseDragged:
			case NSEventTypeSwipe:
				DeferredEvent.Delta = (bIgnoreMouseMoveDelta != 0) ? FVector2D::ZeroVector : FVector2D([Event deltaX], [Event deltaY]);
				break;

			case NSEventTypeLeftMouseDown:
			case NSEventTypeRightMouseDown:
			case NSEventTypeOtherMouseDown:
			case NSEventTypeLeftMouseUp:
			case NSEventTypeRightMouseUp:
			case NSEventTypeOtherMouseUp:
				DeferredEvent.ButtonNumber = [Event buttonNumber];
				DeferredEvent.ClickCount = [Event clickCount];
				if (bIsRightClickEmulationEnabled && DeferredEvent.Type == NSEventTypeLeftMouseDown && (DeferredEvent.ModifierFlags & NSEventModifierFlagControl))
				{
					bEmulatingRightClick = true;
					DeferredEvent.Type = NSEventTypeRightMouseDown;
					DeferredEvent.ButtonNumber = 2;
				}
				else if (DeferredEvent.Type == NSEventTypeLeftMouseUp && bEmulatingRightClick)
				{
					bEmulatingRightClick = false;
					DeferredEvent.Type = NSEventTypeRightMouseUp;
					DeferredEvent.ButtonNumber = 2;
				}
				break;

			case NSEventTypeScrollWheel:
				DeferredEvent.Delta = FVector2D([Event deltaX], [Event deltaY]);
				DeferredEvent.ScrollingDelta = FVector2D([Event scrollingDeltaX], [Event scrollingDeltaY]);
				DeferredEvent.Phase = [Event phase];
				DeferredEvent.MomentumPhase = [Event momentumPhase];
				DeferredEvent.IsDirectionInvertedFromDevice = [Event isDirectionInvertedFromDevice];
				break;

			case NSEventTypeMagnify:
				DeferredEvent.Delta = FVector2D([Event magnification], [Event magnification]);
				break;

			case NSEventTypeRotate:
				DeferredEvent.Delta = FVector2D([Event rotation], [Event rotation]);
				break;

			case NSEventTypeKeyDown:
			case NSEventTypeKeyUp:
			{
				if ([[Event characters] length] > 0)
				{
					DeferredEvent.KeyCode = [Event keyCode];
					DeferredEvent.Character = ConvertChar([[Event characters] characterAtIndex:0]);
					DeferredEvent.CharCode = TranslateCharCode([[Event charactersIgnoringModifiers] characterAtIndex:0], DeferredEvent.KeyCode);
					DeferredEvent.IsRepeat = [Event isARepeat];
					DeferredEvent.IsPrintable = IsPrintableKey(DeferredEvent.Character);
				}
				else
				{
					return;
				}
				break;
			}
		}
	}
	else if (Object && [Object isKindOfClass:[NSNotification class]])
	{
		NSNotification* Notification = (NSNotification*)Object;
		DeferredEvent.NotificationName = [[Notification name] retain];
		if ([[Notification object] isKindOfClass:[FCocoaWindow class]])
		{
			DeferredEvent.Window = (FCocoaWindow*)[Notification object];

			if (DeferredEvent.NotificationName == NSWindowDidResizeNotification)
			{
				if (DeferredEvent.Window)
				{
					const bool bWaitForThread = false;
					GameThreadCall(^{
						TSharedPtr<FMacWindow> Window = FindWindowByNSWindow(DeferredEvent.Window);
						if (Window.IsValid())
						{
							OnWindowDidResize(Window.ToSharedRef());
						}
					}, bWaitForThread, @[ NSDefaultRunLoopMode, UnrealResizeEventMode, UnrealShowEventMode, UnrealFullscreenEventMode ]);
				}
				return;
			}
		}
		else if ([[Notification object] conformsToProtocol:@protocol(NSDraggingInfo)])
		{
			NSWindow* NotificationWindow = [(id<NSDraggingInfo>)[Notification object] draggingDestinationWindow];

			if (NotificationWindow && [NotificationWindow isKindOfClass:[FCocoaWindow class]])
			{
				DeferredEvent.Window = (FCocoaWindow*)NotificationWindow;
			}

			if (DeferredEvent.NotificationName == NSPrepareForDragOperation)
			{
				DeferredEvent.DraggingPasteboard = [[(id<NSDraggingInfo>)[Notification object] draggingPasteboard] retain];
			}
		}
	}

	FScopeLock Lock(&EventsMutex);
	DeferredEvents.Add(DeferredEvent);
}

TSharedPtr<FMacWindow> FMacApplication::FindWindowByNSWindow(FCocoaWindow* WindowHandle)
{
	FScopeLock Lock(&WindowsMutex);

	for (int32 WindowIndex=0; WindowIndex < Windows.Num(); ++WindowIndex)
	{
		TSharedRef<FMacWindow> Window = Windows[WindowIndex];
		if (Window->GetWindowHandle() == WindowHandle)
		{
			return Window;
		}
	}

	return TSharedPtr<FMacWindow>(nullptr);
}

void FMacApplication::InvalidateTextLayout(FCocoaWindow* Window)
{
	WindowsRequiringTextInvalidation.AddUnique(Window);
}

NSEvent* FMacApplication::HandleNSEvent(NSEvent* Event)
{
	NSEvent* ReturnEvent = Event;

	if (MacApplication && !MacApplication->bSystemModalMode)
	{
		const bool bIsResentEvent = [Event type] == NSEventTypeApplicationDefined && [Event subtype] == (NSEventSubtype)RESET_EVENT_SUBTYPE;

		if (bIsResentEvent)
		{
			ReturnEvent = (NSEvent*)[Event data1];
		}
		else
		{
			MacApplication->DeferEvent(Event);

			bool bShouldStopEventDispatch = false;
			switch ([Event type])
			{
				case NSEventTypeKeyDown:
				case NSEventTypeKeyUp:
					bShouldStopEventDispatch = true;
					break;

				case NSEventTypeLeftMouseDown:
				case NSEventTypeRightMouseDown:
				case NSEventTypeOtherMouseDown:
				case NSEventTypeLeftMouseUp:
				case NSEventTypeRightMouseUp:
				case NSEventTypeOtherMouseUp:
					// Make sure we don't stop receiving mouse dragged events if the cursor is over the edge of the screen and the cursor is locked
					bShouldStopEventDispatch = ((FMacCursor*)MacApplication->Cursor.Get())->IsLocked();
					break;
			}

			if (bShouldStopEventDispatch)
			{
				ReturnEvent = nil;
			}
		}
	}

	return ReturnEvent;
}

void FMacApplication::OnDisplayReconfiguration(CGDirectDisplayID Display, CGDisplayChangeSummaryFlags Flags, void* UserInfo)
{
	FMacApplication* App = (FMacApplication*)UserInfo;

	
	if(Flags & kCGDisplayBeginConfigurationFlag)
	{
		App->bDisplayReconfiguring = true;
	}
	else if (Flags & kCGDisplayDesktopShapeChangedFlag)
	{
		App->bDisplayReconfiguring = false;

		App->UpdateScreensArray();

		// Slate needs to know when desktop size changes.
		FDisplayMetrics DisplayMetrics;
		FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);
		App->BroadcastDisplayMetricsChanged(DisplayMetrics);
	}

	MainThreadCall(^{
		FScopeLock Lock(&App->WindowsMutex);
		for (int32 WindowIndex=0; WindowIndex < App->Windows.Num(); ++WindowIndex)
		{
			TSharedRef<FMacWindow> WindowRef = App->Windows[WindowIndex];
			WindowRef->OnDisplayReconfiguration(Display, Flags);
		}
	});
}

#if WITH_EDITOR
int32 FMacApplication::MTContactCallback(void* Device, void* Data, int32 NumFingers, double TimeStamp, int32 Frame)
{
	if (MacApplication)
	{
		const bool bIsTrackpad = MTDeviceIsBuiltIn(Device);
		MacApplication->bUsingTrackpad = NumFingers > (bIsTrackpad ? 1 : 0);
	}
	return 1;
}
#endif

void FMacApplication::ProcessEvent(const FDeferredMacEvent& Event)
{
	TSharedPtr<FMacWindow> EventWindow = FindWindowByNSWindow(Event.Window);
	if (Event.Type)
	{
		if(LastNativeEvent != nil)
		{
			[LastNativeEvent release];
			LastNativeEvent = nil;
		}
		
		switch (Event.Type)
		{
			case NSEventTypeMouseMoved:
			case NSEventTypeLeftMouseDragged:
			case NSEventTypeRightMouseDragged:
			case NSEventTypeOtherMouseDragged:
				ConditionallyUpdateModifierKeys(Event);
				ProcessMouseMovedEvent(Event, EventWindow);
				FPlatformAtomics::InterlockedExchange(&bIgnoreMouseMoveDelta, 0);
				break;

			case NSEventTypeLeftMouseDown:
			case NSEventTypeRightMouseDown:
			case NSEventTypeOtherMouseDown:
				ConditionallyUpdateModifierKeys(Event);
				ProcessMouseDownEvent(Event, EventWindow);
				break;

			case NSEventTypeLeftMouseUp:
			case NSEventTypeRightMouseUp:
			case NSEventTypeOtherMouseUp:
				ConditionallyUpdateModifierKeys(Event);
				ProcessMouseUpEvent(Event, EventWindow);
				break;

			case NSEventTypeScrollWheel:
				ConditionallyUpdateModifierKeys(Event);
				ProcessScrollWheelEvent(Event, EventWindow);
				break;

			case NSEventTypeMagnify:
			case NSEventTypeSwipe:
			case NSEventTypeRotate:
			case NSEventTypeBeginGesture:
			case NSEventTypeEndGesture:
				ConditionallyUpdateModifierKeys(Event);
				ProcessGestureEvent(Event);
				break;

			case NSEventTypeKeyDown:
				// retain LastNativeEvent in the event of a ResendLastEvent
				if(Event.Event != nil)
				{
					[Event.Event retain];
				}
				LastNativeEvent = Event.Event;
				ConditionallyUpdateModifierKeys(Event);
				ProcessKeyDownEvent(Event, EventWindow);
				break;

			case NSEventTypeKeyUp:
				// retain LastNativeEvent in the event of a ResendLastEvent
				if(Event.Event != nil)
				{
					[Event.Event retain];
				}
				LastNativeEvent = Event.Event;
				ConditionallyUpdateModifierKeys(Event);
				ProcessKeyUpEvent(Event);
				break;
				
			case NSEventTypeFlagsChanged:
				ConditionallyUpdateModifierKeys(Event);
				break;
				
			case NSEventTypeMouseEntered:
			case NSEventTypeMouseExited:
				ConditionallyUpdateModifierKeys(Event);
				break;
		}
	}
	else if (EventWindow.IsValid())
	{
		if (Event.NotificationName == NSWindowWillStartLiveResizeNotification)
		{
			MessageHandler->BeginReshapingWindow(EventWindow.ToSharedRef());
		}
		else if (Event.NotificationName == NSWindowDidEndLiveResizeNotification)
		{
			MessageHandler->FinishedReshapingWindow(EventWindow.ToSharedRef());
		}
		else if (Event.NotificationName == NSWindowDidEnterFullScreenNotification)
		{
			OnWindowDidResize(EventWindow.ToSharedRef(), true);
		}
		else if (Event.NotificationName == NSWindowDidExitFullScreenNotification)
		{
			OnWindowDidResize(EventWindow.ToSharedRef(), true);
		}
		else if (Event.NotificationName == NSWindowWillMoveNotification)
		{
			DraggedWindow = EventWindow->GetWindowHandle();
		}
		else if (Event.NotificationName == NSWindowDidMoveNotification)
		{
			OnWindowDidMove(EventWindow.ToSharedRef());
		}
		else if (Event.NotificationName == NSDraggingExited)
		{
			MessageHandler->OnDragLeave(EventWindow.ToSharedRef());
		}
		else if (Event.NotificationName == NSDraggingUpdated)
		{
			// MouseMoved events are suspended during drag and drop operations, so we need to update the cursor position here
			NSPoint CursorPos = [NSEvent mouseLocation];
			FVector2D FloatPosition = ConvertCocoaPositionToSlate(CursorPos.x, CursorPos.y);
			FIntVector2 NewPosition(FMath::TruncToInt(FloatPosition.X), FMath::TruncToInt(FloatPosition.Y));
			FMacCursor* MacCursor = (FMacCursor*)Cursor.Get();
			if (MacCursor->UpdateCursorClipping(NewPosition))
			{
				MacCursor->SetPosition(NewPosition.X, NewPosition.Y);
			}
			else
			{
				MacCursor->UpdateCurrentPosition(NewPosition);
			}

			MessageHandler->OnDragOver(EventWindow.ToSharedRef());
		}
		else if (Event.NotificationName == NSPrepareForDragOperation)
		{
			SCOPED_AUTORELEASE_POOL;

			// Decipher the pasteboard data
			const bool bHaveText = [[Event.DraggingPasteboard types] containsObject:NSPasteboardTypeString];
			const bool bHaveFiles = [[Event.DraggingPasteboard types] containsObject:NSPasteboardTypeFileURL];

			if (bHaveFiles && bHaveText)
			{
				TArray<FString> FileList;

				NSArray *Files = [Event.DraggingPasteboard readObjectsForClasses:@[[NSURL class]] options:nil];
				for (int32 Index = 0; Index < [Files count]; Index++)
				{
					NSString* FilePath = [[Files objectAtIndex: Index] path];
					const FString ListElement = UTF8_TO_TCHAR([FilePath fileSystemRepresentation]);
					FileList.Add(ListElement);
				}

				NSString* Text = [Event.DraggingPasteboard stringForType:NSPasteboardTypeString];

				MessageHandler->OnDragEnterExternal(EventWindow.ToSharedRef(), FString(Text), FileList);
			}
			else if (bHaveFiles)
			{
				TArray<FString> FileList;

				NSArray *Files = [Event.DraggingPasteboard readObjectsForClasses:@[[NSURL class]] options:nil];
				for (int32 Index = 0; Index < [Files count]; Index++)
				{
					NSString* FilePath = [[Files objectAtIndex: Index] path];
					const FString ListElement = UTF8_TO_TCHAR([FilePath fileSystemRepresentation]);
					FileList.Add(ListElement);
				}

				MessageHandler->OnDragEnterFiles(EventWindow.ToSharedRef(), FileList);
			}
			else if (bHaveText)
			{
				NSString* Text = [Event.DraggingPasteboard stringForType:NSPasteboardTypeString];
				MessageHandler->OnDragEnterText(EventWindow.ToSharedRef(), FString(Text));
			}
		}
		else if (Event.NotificationName == NSPerformDragOperation)
		{
			MessageHandler->OnDragDrop(EventWindow.ToSharedRef());
		}
	}
}

void FMacApplication::ResendLastEvent() 
{
	if(LastNativeEvent != nil) 
	{
		ResendEvent(LastNativeEvent);
	}
}

void FMacApplication::ResendEvent(NSEvent* Event)
{
	MainThreadCall(^{
		NSEvent* Wrapper = [NSEvent otherEventWithType:NSEventTypeApplicationDefined location:[Event locationInWindow] modifierFlags:[Event modifierFlags] timestamp:[Event timestamp] windowNumber:[Event windowNumber] context:nil subtype:RESET_EVENT_SUBTYPE data1:(NSInteger)Event data2:0];
		[NSApp sendEvent:Wrapper];
	});
}

void FMacApplication::HandleMouseControllerMovement(FIntVector2 Delta)
{
	if (Delta.X != 0 || Delta.Y != 0)
	{
		bMouseInputNeedsReassociation.store(false, std::memory_order_release);
	}

	// GCMouse reports raw hardware deltas — cursor warps (CGWarpMouseCursorPosition) do not
	// generate GCMouse events, so no warp compensation is needed (unlike the NSEvent path).
	if (bUsingHighPrecisionMouseInput)
	{
		FMacCursor* MacCursor = static_cast<FMacCursor*>(Cursor.Get());
		FIntVector2 HighPrecisionMousePos = MacCursor->GetIntPosition();

		HighPrecisionMousePos.X += Delta.X;
		HighPrecisionMousePos.Y += Delta.Y;

		MacCursor->UpdateCursorClipping(HighPrecisionMousePos);

		FMacScreenRef Screen = FindScreenBySlatePosition(HighPrecisionMousePos.X, HighPrecisionMousePos.Y);
		NSRect VisibleFrame = Screen->VisibleFramePixels;

		// Avoid the menu bar & dock disclosure borders at the top & bottom of fullscreen windows
		if (ActiveWindow.IsValid() && ActiveWindow->GetWindowMode() != EWindowMode::Windowed)
		{
			VisibleFrame.origin.y += 5;
			VisibleFrame.size.height -= 10;
		}

		const int32 ClampedPosX = FMath::Clamp(HighPrecisionMousePos.X, static_cast<int32>(VisibleFrame.origin.x), static_cast<int32>(VisibleFrame.origin.x + VisibleFrame.size.width) - 1);
		const int32 ClampedPosY = FMath::Clamp(HighPrecisionMousePos.Y, static_cast<int32>(VisibleFrame.origin.y), static_cast<int32>(VisibleFrame.origin.y + VisibleFrame.size.height) - 1);
		MacCursor->SetPosition(ClampedPosX, ClampedPosY);
	}
}

void FMacApplication::ProcessMouseMovedEvent(const FDeferredMacEvent& Event, TSharedPtr<FMacWindow> EventWindow)
{
	if (EventWindow.IsValid() && EventWindow->IsRegularWindow())
	{
		const EWindowZone::Type Zone = GetCurrentWindowZone(EventWindow.ToSharedRef());
		bool IsMouseOverTitleBar = (Zone == EWindowZone::TitleBar);
		const bool IsMovable = IsMouseOverTitleBar || IsEdgeZone(Zone);

		FCocoaWindow* WindowHandle = EventWindow->GetWindowHandle();
		MainThreadCall(^{
			[WindowHandle setMovable:IsMovable];
			[WindowHandle setMovableByWindowBackground:IsMouseOverTitleBar];
		}, false);
	}

	FMacCursor* MacCursor = (FMacCursor*)Cursor.Get();

	if (bUsingHighPrecisionMouseInput)
	{
		if (!MacApplicationCVars::bUseNewMouseControllerForMovement || bMouseInputNeedsReassociation.load(std::memory_order_acquire))
		{
			// Get the mouse position
			FIntVector2 HighPrecisionMousePos = MacCursor->GetIntPosition();

			// Find the visible frame of the screen the cursor is currently on.
			FMacScreenRef Screen = FindScreenBySlatePosition(HighPrecisionMousePos.X, HighPrecisionMousePos.Y);
			NSRect VisibleFrame = Screen->VisibleFramePixels;

			// Under OS X we disassociate the cursor and mouse position during hi-precision mouse input.
			// The game snaps the mouse cursor back to the starting point when this is disabled, which
			// accumulates mouse delta that we want to ignore.
			const FIntVector2 AccumDelta = MacCursor->GetMouseWarpDelta();

			// Account for warping delta's
			FIntVector2 Delta(FMath::TruncToInt(Event.Delta.X), FMath::TruncToInt(Event.Delta.Y));
			const FIntVector2 WarpDelta(FMath::Abs(AccumDelta.X)<FMath::Abs(Delta.X) ? AccumDelta.X : Delta.X, FMath::Abs(AccumDelta.Y)<FMath::Abs(Delta.Y) ? AccumDelta.Y : Delta.Y);
			Delta.X -= WarpDelta.X;
			Delta.Y -= WarpDelta.Y;

			// Update to latest position
			HighPrecisionMousePos.X += Delta.X;
			HighPrecisionMousePos.Y += Delta.Y;

			// Clip to lock rect
			MacCursor->UpdateCursorClipping(HighPrecisionMousePos);

			// Clamp to the current screen and avoid the menu bar and dock to prevent popups and other
			// assorted potential for mouse abuse.
			if (bUsingHighPrecisionMouseInput)
			{
				// Avoid the menu bar & dock disclosure borders at the top & bottom of fullscreen windows
				if (EventWindow.IsValid() && EventWindow->GetWindowMode() != EWindowMode::Windowed)
				{
					VisibleFrame.origin.y += 5;
					VisibleFrame.size.height -= 10;
				}
				int32 ClampedPosX = FMath::Clamp(HighPrecisionMousePos.X, (int32)VisibleFrame.origin.x, (int32)(VisibleFrame.origin.x + VisibleFrame.size.width)-1);
				int32 ClampedPosY = FMath::Clamp(HighPrecisionMousePos.Y, (int32)VisibleFrame.origin.y, (int32)(VisibleFrame.origin.y + VisibleFrame.size.height)-1);
				MacCursor->SetPosition(ClampedPosX, ClampedPosY);
			}
			else
			{
				MacCursor->SetPosition(HighPrecisionMousePos.X, HighPrecisionMousePos.Y);
			}

			// Forward the delta on to Slate
			MessageHandler->OnRawMouseMove(Delta.X, Delta.Y);
		}
	}
	else
	{
		NSPoint CursorPos = [NSEvent mouseLocation];
		FVector2D FloatPosition = ConvertCocoaPositionToSlate(CursorPos.x, CursorPos.y);
		FIntVector2 NewPosition(FMath::TruncToInt(FloatPosition.X), FMath::TruncToInt(FloatPosition.Y));
		const FIntVector2 OldPosition = MacCursor->GetIntPosition();
		const FIntVector2 MouseDelta(NewPosition.X - OldPosition.X, NewPosition.Y - OldPosition.Y);
		if (MacCursor->UpdateCursorClipping(NewPosition))
		{
			MacCursor->SetPosition(NewPosition.X, NewPosition.Y);
		}
		else
		{
			MacCursor->UpdateCurrentPosition(NewPosition);
		}

		if (EventWindow.IsValid())
		{
			// Cocoa does not update NSWindow's frame until user stops dragging the window, so while window is being dragged, we calculate
			// its position based on mouse move delta
			if (DraggedWindow && DraggedWindow == EventWindow->GetWindowHandle())
			{
				const int32 X = EventWindow->PositionX + MouseDelta.X;
				const int32 Y = EventWindow->PositionY + MouseDelta.Y;
				MessageHandler->OnMovedWindow(EventWindow.ToSharedRef(), X, Y);
				EventWindow->PositionX = X;
				EventWindow->PositionY = Y;
			}

			MessageHandler->OnMouseMove();
		}
	}

	if (EventWindow.IsValid() && EventWindow->GetWindowHandle() && !DraggedWindow && !GetCapture())
	{
		MessageHandler->OnCursorSet();
	}
}

void FMacApplication::ProcessMouseDownEvent(const FDeferredMacEvent& Event, TSharedPtr<FMacWindow> EventWindow)
{
	EMouseButtons::Type Button = Event.Type == NSEventTypeLeftMouseDown ? EMouseButtons::Left : EMouseButtons::Right;
	if (Event.Type == NSEventTypeOtherMouseDown)
	{
		switch (Event.ButtonNumber)
		{
			case 2:
				Button = EMouseButtons::Middle;
				break;

			case 3:
				Button = EMouseButtons::Thumb01;
				break;

			case 4:
				Button = EMouseButtons::Thumb02;
				break;
		}
	}

	if (EventWindow.IsValid())
	{
		const EWindowZone::Type Zone = GetCurrentWindowZone(EventWindow.ToSharedRef());
		
		bool const bResizable = !bUsingHighPrecisionMouseInput && EventWindow->IsRegularWindow() && (EventWindow->GetDefinition().SupportsMaximize || EventWindow->GetDefinition().HasSizingFrame);
		
		if (Button == LastPressedMouseButton && (Event.ClickCount % 2) == 0)
		{
			if (Zone == EWindowZone::TitleBar)
			{
				const bool bShouldMinimize = [[NSUserDefaults standardUserDefaults] boolForKey:@"AppleMiniaturizeOnDoubleClick"];
				FCocoaWindow* WindowHandle = EventWindow->GetWindowHandle();
				if (bShouldMinimize)
				{
					MainThreadCall(^{ [WindowHandle performMiniaturize:nil]; });
				}
				else
				{
					MainThreadCall(^{ [WindowHandle zoom:nil]; });
				}
			}
			else
			{
				MessageHandler->OnMouseDoubleClick(EventWindow, Button);
			}
		}
		// Only forward left mouse button down events if it's not inside the resize edge zone of a normal resizable window.
		else if (!bResizable || Button != EMouseButtons::Left || !IsEdgeZone(Zone))
		{
			MessageHandler->OnMouseDown(EventWindow, Button);
		}

		if (EventWindow.IsValid() && EventWindow->GetWindowHandle() && !DraggedWindow && !GetCapture())
		{
			MessageHandler->OnCursorSet();
		}
	}

	LastPressedMouseButton = Button;
}

void FMacApplication::ProcessMouseUpEvent(const FDeferredMacEvent& Event, TSharedPtr<FMacWindow> EventWindow)
{
	EMouseButtons::Type Button = Event.Type == NSEventTypeLeftMouseUp ? EMouseButtons::Left : EMouseButtons::Right;
	if (Event.Type == NSEventTypeOtherMouseUp)
	{
		switch (Event.ButtonNumber)
		{
			case 2:
				Button = EMouseButtons::Middle;
				break;

			case 3:
				Button = EMouseButtons::Thumb01;
				break;

			case 4:
				Button = EMouseButtons::Thumb02;
				break;
		}
	}

	MessageHandler->OnMouseUp(Button);

	if (EventWindow.IsValid())
	{
		// 10.12.6 Fix for window position after dragging window to desktop selector in mission control
		// 10.13.0 Doesn't need this as it always fires the window move event after desktop drag operation
		if (DraggedWindow != nullptr && EventWindow->GetWindowHandle() == DraggedWindow)
		{
			OnWindowDidMove(EventWindow.ToSharedRef());
		}

		if (EventWindow->GetWindowHandle() && !DraggedWindow && !GetCapture())
		{
			MessageHandler->OnCursorSet();
		}
	}

	FPlatformApplicationMisc::bChachedMacMenuStateNeedsUpdate = true;
	FPlatformAtomics::InterlockedExchangePtr((void**)&DraggedWindow, nullptr);
}

void FMacApplication::ProcessScrollWheelEvent(const FDeferredMacEvent& Event, TSharedPtr<FMacWindow> EventWindow)
{
	const float DeltaY = (Event.ModifierFlags & NSEventModifierFlagShift) ? (float)Event.Delta.X : (float)Event.Delta.Y;

	NSEventPhase Phase = Event.Phase;

	if (Event.MomentumPhase != NSEventPhaseNone || Event.Phase != NSEventPhaseNone)
	{
		const FVector2D ScrollDelta(Event.ScrollingDelta.X, Event.ScrollingDelta.Y);

		// This is actually a scroll gesture from trackpad
		MessageHandler->OnTouchGesture(EGestureEvent::Scroll, ScrollDelta, DeltaY, Event.IsDirectionInvertedFromDevice);
		RecordUsage(EGestureEvent::Scroll);
	}
	else
	{
		MessageHandler->OnMouseWheel(DeltaY);
	}

	if (EventWindow.IsValid() && EventWindow->GetWindowHandle() && !DraggedWindow && !GetCapture())
	{
		MessageHandler->OnCursorSet();
	}
}

void FMacApplication::ProcessGestureEvent(const FDeferredMacEvent& Event)
{
	if (Event.Type == NSEventTypeBeginGesture)
	{
		MessageHandler->OnBeginGesture();
	}
	else if (Event.Type == NSEventTypeEndGesture)
	{
		MessageHandler->OnEndGesture();
#if WITH_EDITOR
		LastGestureUsed = EGestureEvent::None;
#endif
	}
	else
	{
		const EGestureEvent GestureType = Event.Type == NSEventTypeMagnify ? EGestureEvent::Magnify : (Event.Type == NSEventTypeSwipe ? EGestureEvent::Swipe : EGestureEvent::Rotate);
		MessageHandler->OnTouchGesture(GestureType, Event.Delta, 0, Event.IsDirectionInvertedFromDevice);
		RecordUsage(GestureType);
	}
}

void FMacApplication::ProcessKeyDownEvent(const FDeferredMacEvent& Event, TSharedPtr<FMacWindow> EventWindow)
{
	bool bHandled = false;
	if (!bSystemModalMode && EventWindow.IsValid())
	{
		bHandled = MessageHandler->OnKeyDown(Event.KeyCode, Event.CharCode, Event.IsRepeat);

		// First KeyDown, then KeyChar. This is important, as in-game console ignores first character otherwise
		bool bCmdKeyPressed = Event.ModifierFlags & 0x18;
		if (!bCmdKeyPressed && Event.IsPrintable)
		{
			MessageHandler->OnKeyChar(Event.Character, Event.IsRepeat);
		}
	}
	if (bHandled)
	{
		FCocoaMenu* MainMenu = [[NSApp mainMenu] isKindOfClass:[FCocoaMenu class]] ? (FCocoaMenu*)[NSApp mainMenu]: nil;
		if (MainMenu)
		{
			NSEvent* NativeEvent = Event.Event;
			MainThreadCall(^{ [MainMenu highlightKeyEquivalent:NativeEvent]; }, false);
		}
	}
	else
	{
		ResendEvent(Event.Event);
	}
}

void FMacApplication::ProcessKeyUpEvent(const FDeferredMacEvent& Event)
{
	bool bHandled = false;
	if (!bSystemModalMode)
	{
		bHandled = MessageHandler->OnKeyUp(Event.KeyCode, Event.CharCode, Event.IsRepeat);
	}
	if (!bHandled)
	{
		ResendEvent(Event.Event);
	}
}

void FMacApplication::OnWindowDidMove(TSharedRef<FMacWindow> Window)
{
	SCOPED_AUTORELEASE_POOL;
    if ([Window->GetWindowHandle() windowMode] == EWindowMode::Fullscreen)
    {
        // Fullscreen mode always moves to 0,0
        MessageHandler->OnMovedWindow(Window, 0, 0);
        Window->PositionX = 0;
        Window->PositionY = 0;
    }
    else
    {
		__block NSRect WindowFrame, OpenGLFrame;
		MainThreadCall(^{
			// Make sure to take notch and padding into account
			WindowFrame = [Window->GetWindowHandle() frame];
			OpenGLFrame = [Window->GetWindowHandle() openGLFrame];
		});

        const double X = WindowFrame.origin.x;
        const double Y = WindowFrame.origin.y + OpenGLFrame.size.height;

		
		// Update ScreensArray when displays are reconfiguring to have up to date informations
		// This is to prevent UE-176002 that happened because UpdateScreensArray is called once
		// the display reconfiguring is done but OnWindowDidResize is call during the reconfiguring
		if (bDisplayReconfiguring)
		{
			UpdateScreensArray();
		}
        FVector2D SlatePosition = ConvertCocoaPositionToSlate(X, Y);
        
        MessageHandler->OnMovedWindow(Window, FMath::TruncToInt(SlatePosition.X), FMath::TruncToInt(SlatePosition.Y));
        Window->PositionX = FMath::TruncToInt(SlatePosition.X);
        Window->PositionY = FMath::TruncToInt(SlatePosition.Y);
    }
}

void FMacApplication::OnWindowWillResize(TSharedRef<FMacWindow> Window)
{
	SCOPED_AUTORELEASE_POOL;
	
	// OnResizingWindow flushes the renderer commands which is needed before we start resizing, but also right after that
	// because window view's drawRect: can be called before Slate has a chance to flush them.
	MessageHandler->OnResizingWindow(Window);
}

void FMacApplication::OnWindowDidResize(TSharedRef<FMacWindow> Window, bool bRestoreMouseCursorLocking)
{
	SCOPED_AUTORELEASE_POOL;

	OnWindowDidMove(Window);

	const FCocoaWindow* CocoaWindow = Window->GetWindowHandle();

	const uint32 WindowWidth  = FMath::TruncToInt([CocoaWindow openGLFrame].size.width * Window->GetDPIScaleFactor());
	const uint32 WindowHeight = FMath::TruncToInt([CocoaWindow openGLFrame].size.height * Window->GetDPIScaleFactor());

	if (bRestoreMouseCursorLocking)
	{
		FMacCursor* MacCursor = (FMacCursor*)MacApplication->Cursor.Get();
		if (MacCursor)
		{
			MacCursor->SetShouldIgnoreLocking(false);
		}
	}
    
	if (Window->GetWindowHandle().TargetWindowMode == EWindowMode::Fullscreen)
	{
        if (SystemResolutionX != 0 && SystemResolutionY != 0)
        {
            // If resolution setting has changed recently, use the target resolution, window size is no longer accurate
            MessageHandler->OnSizeChanged(Window, SystemResolutionX, SystemResolutionY);
        }
        else
        {
            MessageHandler->OnSizeChanged(Window, WindowWidth, WindowHeight);
        }
	}
	else
    {
        MessageHandler->OnSizeChanged(Window, WindowWidth, WindowHeight);
	}
	MessageHandler->OnResizingWindow(Window);
}


void FMacApplication::OnWindowChangedScreen(TSharedRef<FMacWindow> Window)
{
	SCOPED_AUTORELEASE_POOL;
	MessageHandler->HandleDPIScaleChanged(Window);
}


bool FMacApplication::OnWindowDestroyed(TSharedRef<FMacWindow> DestroyedWindow)
{
	SCOPED_AUTORELEASE_POOL;

	FCocoaWindow* WindowHandle = DestroyedWindow->GetWindowHandle();
	const bool bDestroyingMainWindow = DestroyedWindow == ActiveWindow;

	const bool bAllowMainWindow = WindowHandle.AllowMainWindow;
	const EWindowType WindowType =WindowHandle.Type;
    
	if (bDestroyingMainWindow)
	{
		OnWindowActivationChanged(DestroyedWindow, EWindowActivation::Deactivate);
	}
	
#if WITH_ACCESSIBILITY
	MainThreadCall(^{
		//This is to clear the FMacAccessibilityElement* that represents the cocoa window in the FCocoaAccessibilityView.
		// Before the FCocoaAccessibilityView is deallocated, it could be a danling pointer and accessibility calls from VoiceOVer will cause a crash
		[WindowHandle ClearAccessibilityView];
	}, false);
#endif
	
	{
		FScopeLock Lock(&WindowsMutex);
		Windows.Remove(DestroyedWindow);
	}
	
	{
		FScopeLock Lock(&WindowsToCloseMutex);
		if (!CocoaWindowsToClose.Contains(WindowHandle))
		{
			CocoaWindowsToClose.Add(WindowHandle);
		}
	}

	TSharedPtr<FMacWindow> WindowToActivate;

	if (bDestroyingMainWindow || DestroyedWindow->GetWindowHandle().Type == EWindowType::Menu)
	{
		FScopeLock Lock(&WindowsMutex);
		// Figure out which window will now become active and let Slate know without waiting for Cocoa events.
		// Ignore notification windows as Slate keeps bringing them to front and while they technically can be main windows,
		// trying to activate them would result in Slate dismissing menus.
		for (int32 Index = 0; Index < Windows.Num(); ++Index)
		{
			TSharedRef<FMacWindow> WindowRef = Windows[Index];
			if (!CocoaWindowsToClose.Contains(WindowRef->GetWindowHandle()) && [WindowRef->GetWindowHandle() canBecomeMainWindow] && WindowRef->GetDefinition().Type != EWindowType::Notification)
			{
				WindowToActivate = WindowRef;
				break;
			}
		}
	}

	if (WindowToActivate.IsValid())
	{
		if (bAllowMainWindow)
		{
			WindowToActivate->SetWindowFocus();
		}
		
		FCocoaWindow* ActivateWindowHandle = WindowToActivate->GetWindowHandle();
		bool bActivateAllowMainWindow = ActivateWindowHandle.AllowMainWindow;
				
		if (WindowType == EWindowType::Menu && bActivateAllowMainWindow)
		{
			// For some reason when submenus are getting closed a main window is getting activated rather then the previous menu.  I think a
			// better solution would be to investigate the ordering of Windows[], or possibly just enumerating the NSWindows via z-order.
			// I'm worried about the consequences of that kind of change so I'm opting for this since it seems less risky.  We count all
			// the open menus and only activate a normal window if the menu closing is the last one.

			int32 NumMenus=0;
			for (int32 Index = 0; Index < Windows.Num(); ++Index)
			{
				TSharedPtr<FMacWindow> IsMenuWindowRef = Windows[Index];
				if (IsMenuWindowRef.IsValid())
				{
					if (IsMenuWindowRef->GetWindowHandle().Type == EWindowType::Menu)
					{
						++NumMenus;
					}
				}
			}
			
			if (NumMenus<=0)
			{
				const bool bWaitForThread = false;
				GameThreadCall(^{
					OnWindowActivationChanged(WindowToActivate.ToSharedRef(), EWindowActivation::Activate);
				}, bWaitForThread, @[ NSDefaultRunLoopMode, UnrealResizeEventMode, UnrealShowEventMode, UnrealFullscreenEventMode, UnrealCloseEventMode ]);
			}
		}
	}

	MessageHandler->OnCursorSet();

	return true;
}

void FMacApplication::OnWindowActivated(TSharedRef<FMacWindow> Window)
{
	if (ActiveWindow.IsValid())
	{
		OnWindowActivationChanged(ActiveWindow.ToSharedRef(), EWindowActivation::Deactivate);
	}
	OnWindowActivationChanged(Window, EWindowActivation::Activate);
}

void FMacApplication::OnWindowOrderedFront(TSharedRef<FMacWindow> Window)
{
	// Sort Windows array so that the order is the same as on screen
	TArray<TSharedRef<FMacWindow>> NewWindowsArray;
	NewWindowsArray.Add(Window);

	FScopeLock Lock(&WindowsMutex);
	for (int32 WindowIndex=0; WindowIndex < Windows.Num(); ++WindowIndex)
	{
		TSharedRef<FMacWindow> WindowRef = Windows[WindowIndex];
		if (WindowRef != Window)
		{
			NewWindowsArray.Add(WindowRef);
		}
	}
	Windows = NewWindowsArray;
	
#if WITH_ACCESSIBILITY
	// The IMainFrameModule creates an SWindow and tries to show it.
	// When the SWindow calls ShowWindow(), the SlateRHIRenderer is initialized and creates a FMetalViewport.
	// THe FMetalViewport creates a custom FMetalView NSView and sets it as the content view of the window, overriding any accessibility data on the original content view.
	// THis happens AFTER we call OnVoiceoverEnabled() and set all the window IDs the first time
	//We set the accessible Window ID here again to update the FMetalView NSView with accessibility children information
	// @see FMetalViewport::FMetalViewport()
	//@see SWindow::ShowWindow()
	//@Review: Is there a better place for this?
	if(GetAccessibleMessageHandler()->IsActive())
	{
		const AccessibleWidgetId WindowId = GetAccessibleMessageHandler()->GetAccessibleWindowId(Window);
		MainThreadCall(^{
			FCocoaWindow* CocoaWindow = Window->GetWindowHandle();
			if(CocoaWindow)
			{
				[CocoaWindow UpdateAccessibilityView:WindowId];
			}
		}, false);
	}
#endif

}

void FMacApplication::OnWindowActivationChanged(const TSharedRef<FMacWindow>& Window, const EWindowActivation ActivationType)
{
	if (ActivationType == EWindowActivation::Deactivate)
	{
		if (Window == ActiveWindow)
		{
			MessageHandler->OnWindowActivationChanged(Window, ActivationType);
			ActiveWindow.Reset();
		}
	}
	else
	{
		ReassociateMouseInput();

		if (ActiveWindow != Window)
		{
			MessageHandler->OnWindowActivationChanged(Window, ActivationType);
			ActiveWindow = Window;
			OnWindowOrderedFront(Window);
		}
	}

	MessageHandler->OnCursorSet();
}

void FMacApplication::OnApplicationDidBecomeActive()
{
	ReassociateMouseInput();

	OnWindowsReordered();

	for (int32 Index = 0; Index < SavedWindowsOrder.Num(); Index++)
	{
		const FSavedWindowOrderInfo& Info = SavedWindowsOrder[Index];
		NSWindow* Window = [NSApp windowWithWindowNumber:Info.WindowNumber];
		if (Window)
		{
			[Window setLevel:Info.Level];
		}
	}

	if (SavedWindowsOrder.Num() > 0)
	{
		NSWindow* TopWindow = [NSApp windowWithWindowNumber:SavedWindowsOrder[0].WindowNumber];
		if ( TopWindow )
		{
			[TopWindow orderWindow:NSWindowAbove relativeTo:0];
		}
		for (int32 Index = 1; Index < SavedWindowsOrder.Num(); Index++)
		{
			const FSavedWindowOrderInfo& Info = SavedWindowsOrder[Index];
			NSWindow* Window = [NSApp windowWithWindowNumber:Info.WindowNumber];
			if (Window && TopWindow)
			{
				[Window orderWindow:NSWindowBelow relativeTo:[TopWindow windowNumber]];
				TopWindow = Window;
			}
		}
	}

	((FMacCursor*)Cursor.Get())->UpdateVisibility();

	// If editor thread doesn't have the focus, don't suck up too much CPU time.
	if (GIsEditor && !IsRunningCommandlet())
	{
		// Boost our priority back to normal.
		struct sched_param Sched;
		FMemory::Memzero(&Sched, sizeof(struct sched_param));
		Sched.sched_priority = 15;
		pthread_setschedparam(pthread_self(), SCHED_RR, &Sched);
	}

	// app is active, allow sound
	FApp::SetVolumeMultiplier(1.0f);

	GameThreadCall(^{
		if (MacApplication)
		{
			MessageHandler->OnApplicationActivationChanged(true);

			// Slate expects window activation call after the app activates, so we just call it for the top window again
			NSWindow* NativeWindow = [NSApp keyWindow];
			if (NativeWindow)
			{
				TSharedPtr<FMacWindow> TopWindow = FindWindowByNSWindow((FCocoaWindow*)NativeWindow);
				if (TopWindow.IsValid())
				{
					MessageHandler->OnWindowActivationChanged(TopWindow.ToSharedRef(), EWindowActivation::Activate);
				}
			}
		}

#if !UE_SERVER
		// For non-editor clients, record if the active window is in focus
		FGenericCrashContext::SetEngineData(TEXT("Platform.AppHasFocus"), TEXT("true"));
#endif
	}, false);
	
	// Call out to update isOnActiveSpace per FCocoaWindow when the app becomes active.  Works round unrespostive crash reporter on Catalina
	// First call to isOnActiveSpace in OnActiveSpaceDidChange() during crash reporter startup returns false on Catalina but true on BigSur.
	OnActiveSpaceDidChange();
}

void FMacApplication::OnApplicationWillResignActive()
{
	OnWindowsReordered();

	if (SavedWindowsOrder.Num() > 0)
	{
		NSWindow* TopWindow = [NSApp windowWithWindowNumber:SavedWindowsOrder[0].WindowNumber];
		if (TopWindow)
		{
			[TopWindow orderWindow:NSWindowAbove relativeTo:0];
		}
		for (int32 Index = 1; Index < SavedWindowsOrder.Num(); Index++)
		{
			const FSavedWindowOrderInfo& Info = SavedWindowsOrder[Index];
			NSWindow* Window = [NSApp windowWithWindowNumber:Info.WindowNumber];
			if (Window)
			{
				[Window orderWindow:NSWindowBelow relativeTo:[TopWindow windowNumber]];
				TopWindow = Window;
			}
		}
	}

	SetHighPrecisionMouseMode(false, nullptr);

	((FMacCursor*)Cursor.Get())->UpdateVisibility();

	// If editor thread doesn't have the focus, don't suck up too much CPU time.
	if (GIsEditor && !IsRunningCommandlet())
	{
		// Drop our priority to speed up whatever is in the foreground.
		struct sched_param Sched;
		FMemory::Memzero(&Sched, sizeof(struct sched_param));
		Sched.sched_priority = 5;
		pthread_setschedparam(pthread_self(), SCHED_RR, &Sched);

		// Sleep for a bit to not eat up all CPU time.
		FPlatformProcess::Sleep(0.005f);
	}

	// app is inactive, apply multiplier
	FApp::SetVolumeMultiplier(FApp::GetUnfocusedVolumeMultiplier());

	GameThreadCall(^{
		if (MacApplication)
		{
			MessageHandler->OnApplicationActivationChanged(false);
		}

#if !UE_SERVER
		// For non-editor clients, record if the active window is in focus
		FGenericCrashContext::SetEngineData(TEXT("Platform.AppHasFocus"), TEXT("false"));
#endif
	}, false);
}

void FMacApplication::OnWindowsReordered()
{
	TMap<int32, int32> Levels;

	for (int32 Index = 0; Index < SavedWindowsOrder.Num(); Index++)
	{
		const FSavedWindowOrderInfo& Info = SavedWindowsOrder[Index];
		Levels.Add(Info.WindowNumber, Info.Level);
	}

	SavedWindowsOrder.Empty();

	FScopeLock Lock(&WindowsMutex);

	int32 MinLevel = 0;
	int32 MaxLevel = 0;
	for (int32 WindowIndex=0; WindowIndex < Windows.Num(); ++WindowIndex)
	{
		FCocoaWindow* Window = Windows[WindowIndex]->GetWindowHandle();
		const int32 WindowLevel = Levels.Contains([Window windowNumber]) ? Levels[[Window windowNumber]] : [Window level];
		MinLevel = FMath::Min(MinLevel, WindowLevel);
		MaxLevel = FMath::Max(MaxLevel, WindowLevel);
	}

	for (int32 Level = MaxLevel; Level >= MinLevel; Level--)
	{
		for (int32 WindowIndex=0; WindowIndex < Windows.Num(); ++WindowIndex)
		{
			FCocoaWindow* Window = Windows[WindowIndex]->GetWindowHandle();
			const int32 WindowLevel = Levels.Contains([Window windowNumber]) ? Levels[[Window windowNumber]] : [Window level];
			if (Level == WindowLevel && [Window isKindOfClass:[FCocoaWindow class]] && [Window isVisible] && ![Window hidesOnDeactivate])
			{
				SavedWindowsOrder.Add(FSavedWindowOrderInfo([Window windowNumber], WindowLevel));
				[Window setLevel:NSNormalWindowLevel];
			}
		}
	}
}

void FMacApplication::OnActiveSpaceDidChange()
{
	FScopeLock Lock(&WindowsMutex);

	for (int32 WindowIndex=0; WindowIndex < Windows.Num(); ++WindowIndex)
	{
		TSharedRef<FMacWindow> WindowRef = Windows[WindowIndex];
		FCocoaWindow* WindowHandle = WindowRef->GetWindowHandle();
		if (WindowHandle)
		{
			WindowHandle->bIsOnActiveSpace = [WindowHandle isOnActiveSpace];
		}
	}
}

void FMacApplication::OnCursorLock()
{
	if (Cursor.IsValid())
	{
		SCOPED_AUTORELEASE_POOL;
		NSWindow* NativeWindow = [NSApp keyWindow];
		if (NativeWindow)
		{
			if (((FMacCursor*)Cursor.Get())->IsLocked())
			{
				MainThreadCall(^{
					SCOPED_AUTORELEASE_POOL;
					[NativeWindow setMinSize:NSMakeSize(NativeWindow.frame.size.width, NativeWindow.frame.size.height)];
					[NativeWindow setMaxSize:NSMakeSize(NativeWindow.frame.size.width, NativeWindow.frame.size.height)];
				}, false);
			}
			else
			{
				TSharedPtr<FMacWindow> Window = FindWindowByNSWindow((FCocoaWindow*)NativeWindow);
				if (Window.IsValid())
				{
					const FGenericWindowDefinition& Definition = Window->GetDefinition();
					const NSSize MinSize = NSMakeSize(Definition.SizeLimits.GetMinWidth().Get(10.0f), Definition.SizeLimits.GetMinHeight().Get(10.0f));
					const NSSize MaxSize = NSMakeSize(Definition.SizeLimits.GetMaxWidth().Get(10000.0f), Definition.SizeLimits.GetMaxHeight().Get(10000.0f));
					MainThreadCall(^{
						SCOPED_AUTORELEASE_POOL;
						[NativeWindow setMinSize:MinSize];
						[NativeWindow setMaxSize:MaxSize];
					}, false);
				}
			}
		}
	}
}

void FMacApplication::ConditionallyUpdateModifierKeys(const FDeferredMacEvent& Event)
{
	if (CurrentModifierFlags != Event.ModifierFlags)
	{
		NSUInteger ModifierFlags = Event.ModifierFlags;
		
		HandleModifierChange(ModifierFlags, (1<<4), 7, MMK_RightCommand);
		HandleModifierChange(ModifierFlags, (1<<3), 6, MMK_LeftCommand);
		HandleModifierChange(ModifierFlags, (1<<1), 0, MMK_LeftShift);
		HandleModifierChange(ModifierFlags, (1<<16), 8, MMK_CapsLock);
		HandleModifierChange(ModifierFlags, (1<<5), 4, MMK_LeftAlt);
		HandleModifierChange(ModifierFlags, (1<<0), 2, MMK_LeftControl);
		HandleModifierChange(ModifierFlags, (1<<2), 1, MMK_RightShift);
		HandleModifierChange(ModifierFlags, (1<<6), 5, MMK_RightAlt);
		HandleModifierChange(ModifierFlags, (1<<13), 3, MMK_RightControl);
		
		CurrentModifierFlags = ModifierFlags;
	}
}

void FMacApplication::HandleModifierChange(NSUInteger NewModifierFlags, NSUInteger FlagsShift, NSUInteger UEShift, EMacModifierKeys TranslatedCode)
{
	const bool CurrentPressed = (CurrentModifierFlags & FlagsShift) != 0;
	const bool NewPressed = (NewModifierFlags & FlagsShift) != 0;
	if (CurrentPressed != NewPressed)
	{
		if (NewPressed)
		{
			ModifierKeysFlags |= 1 << UEShift;
			MessageHandler->OnKeyDown(TranslatedCode, 0, false);
		}
		else
		{
			ModifierKeysFlags &= ~(1 << UEShift);
			MessageHandler->OnKeyUp(TranslatedCode, 0, false);
		}
	}
}

FCocoaWindow* FMacApplication::FindEventWindow(NSEvent* Event) const
{
	SCOPED_AUTORELEASE_POOL;

	FCocoaWindow* EventWindow = [[Event window] isKindOfClass:[FCocoaWindow class]] ? (FCocoaWindow*)[Event window] : nullptr;

	if ([Event type] != NSEventTypeKeyDown && [Event type] != NSEventTypeKeyUp)
	{
		if ([Event type] == NSEventTypeMouseMoved && WindowUnderCursor == nullptr)
		{
			// Ignore windows owned by other applications
			return nullptr;
		}
		else if (DraggedWindow)
		{
			EventWindow = DraggedWindow;
		}
		else if (WindowUnderCursor)
		{
			EventWindow = WindowUnderCursor;
		}
	}

	return EventWindow;
}

FCocoaWindow* FMacApplication::FindSlateWindowUnderCursor() const
{
	SCOPED_AUTORELEASE_POOL;
	const NSInteger WindowNumber = [NSWindow windowNumberAtPoint:[NSEvent mouseLocation] belowWindowWithWindowNumber:0];
	const NSWindow* Window = [NSApp windowWithWindowNumber:WindowNumber];
	return (Window && [Window isKindOfClass:[FCocoaWindow class]]) ? (FCocoaWindow*)Window : nil;
}

void FMacApplication::SetForceFeedbackChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value)
{
	if (FApp::UseVRFocus() && !FApp::HasVRFocus())
	{
		return; // do not proceed if the app uses VR focus but doesn't have it
	}

	for (const TSharedPtr<IInputDevice>& InputDevice : ExternalInputDevices)
	{
		if (InputDevice.IsValid())
		{
			InputDevice->SetChannelValue(ControllerId, ChannelType, Value);
		}
	}

	// Route to the GameController-backed implementation when it owns this controller id.
	HIDInput->SetForceFeedbackChannelValue(ControllerId, ChannelType, Value);
}

void FMacApplication::SetForceFeedbackChannelValues(int32 ControllerId, const FForceFeedbackValues &Values)
{
	if (FApp::UseVRFocus() && !FApp::HasVRFocus())
	{
		return; // do not proceed if the app uses VR focus but doesn't have it
	}

	for (const TSharedPtr<IInputDevice>& InputDevice : ExternalInputDevices)
	{
		if (InputDevice.IsValid())
		{
			// Mirrored from the Window's impl: "Ideally, we would want to use
			// GetHapticDevice instead but they're not implemented for SteamController"
			if (InputDevice->IsGamepadAttached())
			{
				InputDevice->SetChannelValues(ControllerId, Values);
			}
		}
	}

	HIDInput->SetForceFeedbackChannelValues(ControllerId, Values);
}

void FMacApplication::SetLightColor(int32 ControllerId, FColor Color)
{
	HIDInput->SetLightColor(ControllerId, Color);
}

void FMacApplication::ResetLightColor(int32 ControllerId)
{
	HIDInput->ResetLightColor(ControllerId);
}

void FMacApplication::SetDeviceProperty(int32 ControllerId, const FInputDeviceProperty* Property)
{
	HIDInput->SetDeviceProperty(ControllerId, Property);
}

void FMacApplication::SetHapticFeedbackValues(int32 ControllerId, int32 Hand, const FHapticFeedbackValues& Values)
{
	if (FApp::UseVRFocus() && !FApp::HasVRFocus())
	{
		return; // do not proceed if the app uses VR focus but doesn't have it
	}

	for (const TSharedPtr<IInputDevice>& InputDevice : ExternalInputDevices)
	{
		if (InputDevice.IsValid())
		{
			IHapticDevice* HapticDevice = InputDevice->GetHapticDevice();
			if (HapticDevice)
			{
				HapticDevice->SetHapticFeedbackValues(ControllerId, Hand, Values);
			}
		}
	}
}

void FMacApplication::UpdateScreensArray()
{
	MainThreadCall(^{
		SCOPED_AUTORELEASE_POOL;
		FScopeLock Lock(&GAllScreensMutex);
		GAllScreens.Empty();
		NSArray* Screens = [NSScreen screens];
		for (NSScreen* Screen in Screens)
		{
			GAllScreens.Add(MakeShareable(new FMacScreen(Screen)));
		}
	});

	FScopeLock Lock(&GAllScreensMutex);

	NSRect WholeWorkspace = {{0, 0}, {0, 0}};
	for (FMacScreenRef& CurScreen : GAllScreens)
	{
		WholeWorkspace = NSUnionRect(WholeWorkspace, CurScreen->Frame);
	}

	const bool bUseHighDPIMode = FPlatformApplicationMisc::IsHighDPIModeEnabled();

	TArray<FMacScreenRef> SortedScreens;
	for (FMacScreenRef& CurScreen : GAllScreens)
	{
		CurScreen->Frame.origin.y = CurScreen->FramePixels.origin.y = WholeWorkspace.origin.y + WholeWorkspace.size.height - CurScreen->Frame.size.height - CurScreen->Frame.origin.y;
		CurScreen->VisibleFrame.origin.y = CurScreen->VisibleFramePixels.origin.y = WholeWorkspace.origin.y + WholeWorkspace.size.height - CurScreen->VisibleFrame.size.height - CurScreen->VisibleFrame.origin.y;

		SortedScreens.Add(CurScreen);
	}

	SortedScreens.Sort([](const FMacScreenRef& A, const FMacScreenRef& B) -> bool { return A->Frame.origin.x < B->Frame.origin.x; });

	for (int32 Index = 0; Index < SortedScreens.Num(); ++Index)
	{
		FMacScreenRef& CurScreen = SortedScreens[Index];
		const double DPIScaleFactor = bUseHighDPIMode ? CurScreen->Screen.backingScaleFactor : 1.0;
		if (DPIScaleFactor != 1.0)
		{
			CurScreen->FramePixels.size.width = CurScreen->Frame.size.width * DPIScaleFactor;
			CurScreen->FramePixels.size.height = CurScreen->Frame.size.height * DPIScaleFactor;
			CurScreen->VisibleFramePixels.size.width = CurScreen->VisibleFrame.size.width * DPIScaleFactor;
			CurScreen->VisibleFramePixels.size.height = CurScreen->VisibleFrame.size.height * DPIScaleFactor;

			for (int32 OtherIndex = Index + 1; OtherIndex < SortedScreens.Num(); ++OtherIndex)
			{
				FMacScreenRef& OtherScreen = SortedScreens[OtherIndex];
				const double DiffFrame = (OtherScreen->Frame.origin.x - CurScreen->Frame.origin.x) * DPIScaleFactor;
				const double DiffVisibleFrame = (OtherScreen->VisibleFrame.origin.x - CurScreen->VisibleFrame.origin.x) * DPIScaleFactor;
				OtherScreen->FramePixels.origin.x = CurScreen->FramePixels.origin.x + DiffFrame;
				OtherScreen->VisibleFramePixels.origin.x = CurScreen->VisibleFramePixels.origin.x + DiffVisibleFrame;
			}
		}
	}

	SortedScreens.Sort([](const FMacScreenRef& A, const FMacScreenRef& B) -> bool { return A->Frame.origin.y < B->Frame.origin.y; });

	for (int32 Index = 0; Index < SortedScreens.Num(); ++Index)
	{
		FMacScreenRef& CurScreen = SortedScreens[Index];
		const double DPIScaleFactor = bUseHighDPIMode ? CurScreen->Screen.backingScaleFactor : 1.0;
		if (DPIScaleFactor != 1.0)
		{
			for (int32 OtherIndex = Index + 1; OtherIndex < SortedScreens.Num(); ++OtherIndex)
			{
				FMacScreenRef& OtherScreen = SortedScreens[OtherIndex];
				const double DiffFrame = (OtherScreen->Frame.origin.y - CurScreen->Frame.origin.y) * DPIScaleFactor;
				const double DiffVisibleFrame = (OtherScreen->VisibleFrame.origin.y - CurScreen->VisibleFrame.origin.y) * DPIScaleFactor;
				OtherScreen->FramePixels.origin.y = CurScreen->FramePixels.origin.y + DiffFrame;
				OtherScreen->VisibleFramePixels.origin.y = CurScreen->VisibleFramePixels.origin.y + DiffVisibleFrame;
			}
		}
	}

	// The primary screen needs to be at (0,0), so we need to offset all screen origins by its position
	FMacScreenRef& PrimaryScreen = GAllScreens[0];
	const FVector2D FrameOffset(PrimaryScreen->Frame.origin.x, PrimaryScreen->Frame.origin.y);
	const FVector2D FramePixelsOffset(PrimaryScreen->FramePixels.origin.x, PrimaryScreen->FramePixels.origin.y);
	for (FMacScreenRef& CurScreen : GAllScreens)
	{
		CurScreen->Frame.origin.x -= FrameOffset.X;
		CurScreen->Frame.origin.y -= FrameOffset.Y;
		CurScreen->VisibleFrame.origin.x -= FrameOffset.X;
		CurScreen->VisibleFrame.origin.y -= FrameOffset.Y;
		CurScreen->FramePixels.origin.x -= FramePixelsOffset.X;
		CurScreen->FramePixels.origin.y -= FramePixelsOffset.Y;
		CurScreen->VisibleFramePixels.origin.x -= FramePixelsOffset.X;
		CurScreen->VisibleFramePixels.origin.y -= FramePixelsOffset.Y;
	}
}

FVector2D FMacApplication::CalculateScreenOrigin(NSScreen* Screen)
{
	NSRect WholeWorkspace = {{0, 0}, {0, 0}};
	NSRect ScreenFrame = {{0, 0}, {0, 0}};
	FScopeLock Lock(&GAllScreensMutex);
	for (FMacScreenRef& CurScreen : GAllScreens)
	{
		WholeWorkspace = NSUnionRect(WholeWorkspace, CurScreen->FramePixels);
		if (Screen == CurScreen->Screen)
		{
			ScreenFrame = CurScreen->FramePixels;
		}
	}
	return FVector2D(ScreenFrame.origin.x, WholeWorkspace.size.height - ScreenFrame.size.height - ScreenFrame.origin.y);
}

float FMacApplication::GetPrimaryScreenBackingScaleFactor()
{
	FScopeLock Lock(&GAllScreensMutex);
	const bool bUseHighDPIMode = FPlatformApplicationMisc::IsHighDPIModeEnabled();
	return bUseHighDPIMode ? (float)GAllScreens[0]->Screen.backingScaleFactor : 1.0f;
}

FMacScreenRef FMacApplication::FindScreenBySlatePosition(double X, double Y)
{
	NSPoint Point = NSMakePoint(X, Y);

	FScopeLock Lock(&GAllScreensMutex);

	FMacScreenRef TargetScreen = GAllScreens[0];
	for (FMacScreenRef& Screen : GAllScreens)
	{
		if (NSPointInRect(Point, Screen->FramePixels))
		{
			TargetScreen = Screen;
			break;
		}
	}

	return TargetScreen;
}

FMacScreenRef FMacApplication::FindScreenByCocoaPosition(double X, double Y)
{
	NSPoint Point = NSMakePoint(X, Y);

	FScopeLock Lock(&GAllScreensMutex);

	FMacScreenRef TargetScreen = GAllScreens[0];
	for (FMacScreenRef& Screen : GAllScreens)
	{
		if (NSPointInRect(Point, Screen->Screen.frame))
		{
			TargetScreen = Screen;
			break;
		}
	}

	return TargetScreen;
}

FVector2D FMacApplication::ConvertSlatePositionToCocoa(double X, double Y)
{
	FMacScreenRef Screen = FindScreenBySlatePosition(X, Y);
	const bool bUseHighDPIMode = FPlatformApplicationMisc::IsHighDPIModeEnabled();
	const double DPIScaleFactor = bUseHighDPIMode ? Screen->Screen.backingScaleFactor : 1.0;
	const FVector2D OffsetOnScreen = FVector2D(X - Screen->FramePixels.origin.x, Screen->FramePixels.origin.y + Screen->FramePixels.size.height - Y) / DPIScaleFactor;
	return FVector2D(Screen->Screen.frame.origin.x + OffsetOnScreen.X, Screen->Screen.frame.origin.y + OffsetOnScreen.Y);
}

FVector2D FMacApplication::ConvertCocoaPositionToSlate(double X, double Y)
{
	FMacScreenRef Screen = FindScreenByCocoaPosition(X, Y);
	const bool bUseHighDPIMode = FPlatformApplicationMisc::IsHighDPIModeEnabled();
	const double DPIScaleFactor = bUseHighDPIMode ? Screen->Screen.backingScaleFactor : 1.0;
	const FVector2D OffsetOnScreen = FVector2D(X - Screen->Screen.frame.origin.x, Screen->Screen.frame.origin.y + Screen->Screen.frame.size.height - Y) * DPIScaleFactor;
	return FVector2D(Screen->FramePixels.origin.x + OffsetOnScreen.X, Screen->FramePixels.origin.y + OffsetOnScreen.Y);
}

CGPoint FMacApplication::ConvertSlatePositionToCGPoint(double X, double Y)
{
	FMacScreenRef Screen = FindScreenBySlatePosition(X, Y);
	const bool bUseHighDPIMode = FPlatformApplicationMisc::IsHighDPIModeEnabled();
	const double DPIScaleFactor = bUseHighDPIMode ? Screen->Screen.backingScaleFactor : 1.0;
	const FVector2D OffsetOnScreen = FVector2D(X - Screen->FramePixels.origin.x, Y - Screen->FramePixels.origin.y) / DPIScaleFactor;
	return CGPointMake(Screen->Frame.origin.x + OffsetOnScreen.X, Screen->Frame.origin.y + OffsetOnScreen.Y);
}

EWindowZone::Type FMacApplication::GetCurrentWindowZone(const TSharedRef<FMacWindow>& Window) const
{
	const FVector2D CursorPos = ((FMacCursor*)Cursor.Get())->GetPosition();
	const int32 LocalMouseX = FMath::TruncToInt(CursorPos.X - Window->PositionX);
	const int32 LocalMouseY = FMath::TruncToInt(CursorPos.Y - Window->PositionY);
	return MessageHandler->GetWindowZoneForPoint(Window, LocalMouseX, LocalMouseY);
}

bool FMacApplication::IsEdgeZone(EWindowZone::Type Zone) const
{
	return Zone == EWindowZone::NotInWindow || EWindowZone::IsBorder(Zone);
}

bool FMacApplication::IsPrintableKey(uint32 Character) const
{
	switch (Character)
	{
		case NSPauseFunctionKey:		// EKeys::Pause
		case 0x1b:						// EKeys::Escape
		case NSPageUpFunctionKey:		// EKeys::PageUp
		case NSPageDownFunctionKey:		// EKeys::PageDown
		case NSEndFunctionKey:			// EKeys::End
		case NSHomeFunctionKey:			// EKeys::Home
		case NSLeftArrowFunctionKey:	// EKeys::Left
		case NSUpArrowFunctionKey:		// EKeys::Up
		case NSRightArrowFunctionKey:	// EKeys::Right
		case NSDownArrowFunctionKey:	// EKeys::Down
		case NSInsertFunctionKey:		// EKeys::Insert
		case NSDeleteFunctionKey:		// EKeys::Delete
		case NSF1FunctionKey:			// EKeys::F1
		case NSF2FunctionKey:			// EKeys::F2
		case NSF3FunctionKey:			// EKeys::F3
		case NSF4FunctionKey:			// EKeys::F4
		case NSF5FunctionKey:			// EKeys::F5
		case NSF6FunctionKey:			// EKeys::F6
		case NSF7FunctionKey:			// EKeys::F7
		case NSF8FunctionKey:			// EKeys::F8
		case NSF9FunctionKey:			// EKeys::F9
		case NSF10FunctionKey:			// EKeys::F10
		case NSF11FunctionKey:			// EKeys::F11
		case NSF12FunctionKey:			// EKeys::F12
			return false;

		default:
			return true;
	}
}

TCHAR FMacApplication::ConvertChar(TCHAR Character) const
{
	switch (Character)
	{
		case NSDeleteCharacter:
			return '\b';
		default:
			return Character;
	}
}

void FMacApplication::CacheKeyboardInputSource()
{
	// Cocoa main thread only
	check([NSThread isMainThread]);
	
	if(KeyBoardLayoutData != nil)
	{
		[KeyBoardLayoutData release];
		KeyBoardLayoutData = nil;
	}
	
	TISInputSourceRef CurrentKeyboard = TISCopyCurrentKeyboardLayoutInputSource();
	if (CurrentKeyboard)
	{
		CFDataRef CurrentLayoutData = (CFDataRef)TISGetInputSourceProperty(CurrentKeyboard, kTISPropertyUnicodeKeyLayoutData);
		if(CurrentLayoutData)
		{
			const void* Bytes = CFDataGetBytePtr(CurrentLayoutData);
			size_t DataLength = CFDataGetLength(CurrentLayoutData);
			
			if(Bytes && DataLength > 0)
			{
				KeyBoardLayoutData = [[NSData alloc] initWithBytes:Bytes length:DataLength];
			}
		}
		
		CFRelease(CurrentKeyboard);
	}
}

unichar FMacApplication::TranslateKeyCodeToUniCode(uint32 KeyCode, uint32 Modifier)
{
	// Any thread allowed
	
	// Some just don't work as expected
	switch(KeyCode)
	{
		case kVK_PageUp:	return NSPageUpFunctionKey;
		case kVK_PageDown:	return NSPageDownFunctionKey;
		case kVK_End:		return NSEndFunctionKey;
		case kVK_Home:		return NSHomeFunctionKey;
		case kVK_F1: 		return NSF1FunctionKey;
		case kVK_F2: 		return NSF2FunctionKey;
		case kVK_F3: 		return NSF3FunctionKey;
		case kVK_F4: 		return NSF4FunctionKey;
		case kVK_F5: 		return NSF5FunctionKey;
		case kVK_F6: 		return NSF6FunctionKey;
		case kVK_F7: 		return NSF7FunctionKey;
		case kVK_F8: 		return NSF8FunctionKey;
		case kVK_F9: 		return NSF9FunctionKey;
		case kVK_F10:		return NSF10FunctionKey;
		case kVK_F11: 		return NSF11FunctionKey;
		case kVK_F12: 		return NSF12FunctionKey;
		default:
		{
			if (MacApplication && MacApplication->KeyBoardLayoutData != nil)
			{
				const UCKeyboardLayout *KeyboardLayout = (UCKeyboardLayout*)MacApplication->KeyBoardLayoutData.bytes;
				if (KeyboardLayout)
				{
					UniChar Buffer[256] = { 0 };
					UniCharCount BufferLength = 256;
					uint32 DeadKeyState = 0;

					OSStatus Status = UCKeyTranslate(KeyboardLayout, (uint16)KeyCode, kUCKeyActionDown, (uint16)(((Modifier) >> 8) & 0xFF), LMGetKbdType(), kUCKeyTranslateNoDeadKeysMask, &DeadKeyState, BufferLength, &BufferLength, Buffer);
					if (Status == noErr)
					{
						return Buffer[0];
					}
				}
			}
		}
	}
	
	return 0;
}

TCHAR FMacApplication::TranslateCharCode(TCHAR CharCode, uint32 KeyCode) const
{
	// Keys like F1-F12 or Enter do not need translation
	bool bNeedsTranslation = CharCode < NSOpenStepUnicodeReservedBase || CharCode > 0xF8FF;
	if (bNeedsTranslation)
	{
		// For non-numpad keys, the key code depends on the keyboard layout, so find out what was pressed by converting the key code to a Latin character
		TISInputSourceRef CurrentKeyboard = TISCopyCurrentKeyboardLayoutInputSource();
		if (CurrentKeyboard)
		{
			CFDataRef CurrentLayoutData = (CFDataRef)TISGetInputSourceProperty(CurrentKeyboard, kTISPropertyUnicodeKeyLayoutData);
			CFRelease(CurrentKeyboard);

			if (CurrentLayoutData)
			{
				const UCKeyboardLayout *KeyboardLayout = (UCKeyboardLayout*)CFDataGetBytePtr(CurrentLayoutData);
				if (KeyboardLayout)
				{
					UniChar Buffer[256] = { 0 };
					UniCharCount BufferLength = 256;
					uint32 DeadKeyState = 0;

					// To ensure we get a latin character, we pretend that command modifier key is pressed
					OSStatus Status = UCKeyTranslate(KeyboardLayout, (uint16)KeyCode, kUCKeyActionDown, cmdKey >> 8, LMGetKbdType(), kUCKeyTranslateNoDeadKeysMask, &DeadKeyState, BufferLength, &BufferLength, Buffer);
					if (Status == noErr)
					{
						CharCode = Buffer[0];
					}
				}
			}
		}
	}
	else
	{
		// Private use range should not be returned
		CharCode = 0;
	}

	return CharCode;
}

void FMacApplication::CloseQueuedWindows()
{
	// OnWindowClose may call PumpMessages, which would reenter this function, so make a local copy of SlateWindowsToClose array to avoid infinite recursive calls
	TArray<TSharedRef<FMacWindow>> LocalWindowsToClose;
	TArray<FCocoaWindow*> LocalCocoaWindowsToClose;

	{
		FScopeLock Lock(&WindowsToCloseMutex);
		LocalWindowsToClose = SlateWindowsToClose;
		LocalCocoaWindowsToClose = CocoaWindowsToClose;
		SlateWindowsToClose.Empty();
		CocoaWindowsToClose.Empty();
	}

	if (LocalWindowsToClose.Num() > 0)
	{
		for (TSharedRef<FMacWindow> Window : LocalWindowsToClose)
		{
			MessageHandler->OnWindowClose(Window);
		}
	}

	if (LocalCocoaWindowsToClose.Num() > 0)
	{
		MainThreadCall(^{
			SCOPED_AUTORELEASE_POOL;
			for (FCocoaWindow* Window : LocalCocoaWindowsToClose)
			{
				[Window close];
				[Window release];
			}
		}, true, UnrealCloseEventMode);
	}
}

void FMacApplication::InvalidateTextLayouts()
{
	if (WindowsRequiringTextInvalidation.Num() > 0)
	{
		MainThreadCall(^{
			SCOPED_AUTORELEASE_POOL;

			for (FCocoaWindow* CocoaWindow : WindowsRequiringTextInvalidation)
			{
				if (CocoaWindow && [CocoaWindow openGLView])
				{
					FCocoaTextView* TextView = (FCocoaTextView*)[CocoaWindow openGLView];
					[[TextView inputContext] invalidateCharacterCoordinates];
				}
			}

		}, true, UnrealIMEEventMode);

		WindowsRequiringTextInvalidation.Empty();
	}

}

#if WITH_EDITOR
void FMacApplication::RecordUsage(EGestureEvent Gesture)
{
	if (LastGestureUsed != Gesture)
	{
		LastGestureUsed = Gesture;
		GestureUsage[(int32)Gesture] += 1;
	}
}
#endif

void FDisplayMetrics::RebuildDisplayMetrics(FDisplayMetrics& OutDisplayMetrics)
{
	SCOPED_AUTORELEASE_POOL;

	FScopeLock Lock(&GAllScreensMutex);

	FMacScreenRef& PrimaryScreen = GAllScreens[0];

	const NSRect ScreenFrame = PrimaryScreen->FramePixels;
	const NSRect VisibleFrame = PrimaryScreen->VisibleFramePixels;

	// Total screen size of the primary monitor
	OutDisplayMetrics.PrimaryDisplayWidth = FMath::TruncToInt(ScreenFrame.size.width);
	OutDisplayMetrics.PrimaryDisplayHeight = FMath::TruncToInt(ScreenFrame.size.height);

	OutDisplayMetrics.MonitorInfo.Empty();

	NSRect WholeWorkspace = {{0,0},{0,0}};
	for (FMacScreenRef& Screen : GAllScreens)
	{
		WholeWorkspace = NSUnionRect(WholeWorkspace, Screen->FramePixels);

		NSDictionary* ScreenDesc = Screen->Screen.deviceDescription;
		const CGDirectDisplayID DisplayID = [[ScreenDesc objectForKey:@"NSScreenNumber"] unsignedIntegerValue];

		FMonitorInfo Info;
		Info.ID = FString::Printf(TEXT("%u"), DisplayID);

		CFArrayRef ArrDisplay = CGDisplayCopyAllDisplayModes(DisplayID, nullptr);
		if (ArrDisplay)
		{
			Info.NativeWidth = 0;
			Info.NativeHeight = 0;
			const CFIndex AppsCount = CFArrayGetCount(ArrDisplay);
			for (CFIndex i = 0; i < AppsCount; ++i)
			{
				const CGDisplayModeRef Mode = (const CGDisplayModeRef)CFArrayGetValueAtIndex(ArrDisplay, i);
				const int32 Width = (int32)CGDisplayModeGetWidth(Mode);
				const int32 Height = (int32)CGDisplayModeGetHeight(Mode);

				if (Width * Height > Info.NativeWidth * Info.NativeHeight)
				{
					Info.NativeWidth = Width;
					Info.NativeHeight = Height;
				}
			}

			if (!Info.NativeWidth || !Info.NativeHeight)
			{
				Info.NativeWidth = CGDisplayPixelsWide(DisplayID);
				Info.NativeHeight = CGDisplayPixelsHigh(DisplayID);
			}

			Info.MaxResolution = FIntPoint(Info.NativeWidth, Info.NativeHeight);
		
			CFRelease(ArrDisplay);

			Info.DisplayRect = FPlatformRect
			(
				FMath::TruncToInt(Screen->FramePixels.origin.x  + Screen->SafeAreaInsets.left),
				FMath::TruncToInt(Screen->FramePixels.origin.y + Screen->SafeAreaInsets.top),
				FMath::TruncToInt(Screen->FramePixels.size.width - Screen->SafeAreaInsets.right),
				FMath::TruncToInt(Screen->FramePixels.size.height - Screen->SafeAreaInsets.bottom)
			);
			Info.WorkArea = FPlatformRect
			(
				FMath::TruncToInt(Screen->VisibleFramePixels.origin.x),
				FMath::TruncToInt(Screen->VisibleFramePixels.origin.y),
				FMath::TruncToInt(Screen->VisibleFramePixels.size.width),
				FMath::TruncToInt(Screen->VisibleFramePixels.size.height)
			);

			Info.bIsPrimary = Screen->Screen == [NSScreen mainScreen];

			// dpi computations
			const CGSize DisplayPhysicalSize = CGDisplayScreenSize(DisplayID);
			const float MilimetreInch = 25.4f;
			float HorizontalDPI = MilimetreInch * (float)Info.NativeWidth / (float)DisplayPhysicalSize.width;
			float VerticalDPI = MilimetreInch * (float)Info.NativeHeight / (float)DisplayPhysicalSize.height;
			Info.DPI = FMath::CeilToInt((HorizontalDPI + VerticalDPI) / 2.0f);

			Info.Name = Screen->Screen.localizedName;

			OutDisplayMetrics.MonitorInfo.Add(Info);
		}
	}

	// Virtual desktop area
	OutDisplayMetrics.VirtualDisplayRect.Left = FMath::TruncToInt(WholeWorkspace.origin.x);
	OutDisplayMetrics.VirtualDisplayRect.Top = FMath::TruncToInt(FMath::Min(WholeWorkspace.origin.y, 0.0));
	OutDisplayMetrics.VirtualDisplayRect.Right = FMath::TruncToInt(WholeWorkspace.origin.x + WholeWorkspace.size.width);
	OutDisplayMetrics.VirtualDisplayRect.Bottom = FMath::TruncToInt(WholeWorkspace.size.height + OutDisplayMetrics.VirtualDisplayRect.Top);

	// Get the screen rect of the primary monitor, excluding taskbar etc.
	OutDisplayMetrics.PrimaryDisplayWorkAreaRect.Left = FMath::TruncToInt(VisibleFrame.origin.x);
	OutDisplayMetrics.PrimaryDisplayWorkAreaRect.Top = FMath::TruncToInt(VisibleFrame.origin.y);
	OutDisplayMetrics.PrimaryDisplayWorkAreaRect.Right = FMath::TruncToInt(VisibleFrame.size.width);
	OutDisplayMetrics.PrimaryDisplayWorkAreaRect.Bottom = FMath::TruncToInt(VisibleFrame.size.height + OutDisplayMetrics.PrimaryDisplayWorkAreaRect.Top);

	OutDisplayMetrics.TitleSafePaddingSize.X = PrimaryScreen->SafeAreaInsets.left;
    OutDisplayMetrics.TitleSafePaddingSize.Y = PrimaryScreen->SafeAreaInsets.top;
    OutDisplayMetrics.TitleSafePaddingSize.Z = PrimaryScreen->SafeAreaInsets.right;
    OutDisplayMetrics.TitleSafePaddingSize.W = PrimaryScreen->SafeAreaInsets.bottom;

	OutDisplayMetrics.TitleSafePaddingSize *= PrimaryScreen->Screen.backingScaleFactor;
    
    OutDisplayMetrics.ActionSafePaddingSize = OutDisplayMetrics.TitleSafePaddingSize;

	// Apply the debug safe zones
	OutDisplayMetrics.ApplyDefaultSafeZones();
}

void FMacApplication::OnInputDeviceModuleRegistered(const FName& Type, IModularFeature* ModularFeature)
{
	if (Type == IInputDeviceModule::GetModularFeatureName())
	{
		IInputDeviceModule* Module = static_cast<IInputDeviceModule*>(ModularFeature);

		TSharedPtr<IInputDevice> Device = Module->CreateInputDevice(MessageHandler);
		if (Device.IsValid())
		{
			UE_LOGF(LogInit, Log, "Adding external input plugin.");
			ExternalInputDevices.Add(MoveTemp(Device));
		}
	}
}

#if WITH_ACCESSIBILITY
float GMacAccessibleAnnouncementDelay = 0.1f;
FAutoConsoleVariableRef MacAccessibleAnnouncementDealyRef(
	TEXT("mac.AccessibleAnnouncementDelay"),
	GMacAccessibleAnnouncementDelay,
	TEXT("We need to introduce a small delay to avoid OSX system accessibility announcements from stomping on our requested user announcement. Delays <= 0.05f are too short and result in the announcement being dropped. Delays ~0.075f result in unstable delivery")
);

void FMacApplication::OnAccessibleEventRaised(const FAccessibleEventArgs& Args)
{
	// This should only be triggered by the accessible message handler which initiates from the Slate thread.
	check(IsInGameThread());
	
	const AccessibleWidgetId Id = Args.Widget->GetId();
	switch (Args.Event)
	{
		case EAccessibleEvent::FocusChange:
		{
			//@TODO: Posting accessibility focus notifications don't seem to do anything, investigate further
			break;
		}
		case EAccessibleEvent::ParentChanged:
		{
			FVariant NewValueCopy = Args.OldValue;
			MainThreadCall(^{
				FMacAccessibilityElement* Element = [[FMacAccessibilityManager AccessibilityManager] GetAccessibilityElement:Id];
				if(Element)
				{
					const AccessibleWidgetId NewParentId = NewValueCopy.GetValue<AccessibleWidgetId>();
					Element.ParentId = NewParentId;
					FMacAccessibilityElement* NewParent = [[FMacAccessibilityManager AccessibilityManager]GetAccessibilityElement:NewParentId];
					Element.accessibilityParent = NewParent;

					//if the element is orphaned by having a nill parent, we'll remove the entire subtree for now from the accessibility cache
					//@TODO: Widget switcher unparents its children, but this is never executed. Strange
					if(NewParent == Nil)
					{
						[[FMacAccessibilityManager AccessibilityManager] RemoveAccessibilitySubtree:Id];
					}
				}
				// LayoutChanged is to indicate things like "a widget became visible or hidden" while
				// ScreenChanged is for large-scale UI changes. It can potentially take an NSString to read
				// to the user when this happens, if we choose to support that.
				//@TODO: Posting a notification with user info to the NSApp doesn't seem to do anything. Find out wy.
			}, false);
			break;
		}
		case EAccessibleEvent::WidgetRemoved:
		{
			MainThreadCall(^{
				[[FMacAccessibilityManager AccessibilityManager] RemoveAccessibilityElement:Id];
			}, false);
			break;
		}
		case EAccessibleEvent::Notification:
		{
			NSString* Announcement = [NSString stringWithFString:Args.NewValue.GetValue<FString>()];
			NSDictionary* AnnouncementInfo = @{NSAccessibilityAnnouncementKey: Announcement, NSAccessibilityPriorityKey: @(NSAccessibilityPriorityHigh)};
			MainThreadCall(^{
				// If we don't wait for a small period of time, system announcements can stomp on our announcement
				AccessibilityAnnouncementDelayTimer = [NSTimer scheduledTimerWithTimeInterval:GMacAccessibleAnnouncementDelay repeats:NO block:^(NSTimer* _Nonnull timer){
					// The notification HAS to be posted to the main window, otherwise the announcement request
					// will never be received by OSX for whatever reason
					NSAccessibilityPostNotificationWithUserInfo(NSApp.mainWindow, NSAccessibilityAnnouncementRequestedNotification, AnnouncementInfo);
					AccessibilityAnnouncementDelayTimer = Nil;
				}];
			}, false);
			break;
		}
	}
}

void FMacApplication::OnVoiceoverEnabled()
{
	//Alll accessibility functions should originate from Main Thread
	check([NSThread isMainThread]);
	if(GetAccessibleMessageHandler()->IsActive())
	{
		return;
	}
	GetAccessibleMessageHandler()->SetActive(true);
	// Retrieving Slate accessibility data needs to be done on the game thread
	GameThreadCall(^{
		FScopeLock Lock(&WindowsMutex);
		TArray<AccessibleWidgetId> WindowIds;
		for(const TSharedRef<FMacWindow>& window : Windows)
		{
			AccessibleWidgetId Id = GetAccessibleMessageHandler()->GetAccessibleWindowId(window);
			WindowIds.Add(Id);
		}
		// All AppKit functions need to be called from Main Thread
		MainThreadCall(^{
			for(int WindowIndex = 0; WindowIndex < Windows.Num(); ++WindowIndex)
			{
				const TSharedRef<FMacWindow> CurrentWindow = Windows[WindowIndex];
				FCocoaWindow* CurrentCocoaWindow = CurrentWindow->GetWindowHandle();
				if(CurrentCocoaWindow)
				{
					[CurrentCocoaWindow UpdateAccessibilityView:WindowIds[WindowIndex]];
				}
			}// for all windows
			//Start caching Mac Accessibility data to be returned to Voiceover upon request
			// When Voiceover is enabled, the accessibility tree will take a while to build and
			//accessibility may not work properly till then.
			if (AccessibilityCacheTimer == nil)
			{
				AccessibilityCacheTimer = [NSTimer scheduledTimerWithTimeInterval:0.25f target:[FMacAccessibilityManager AccessibilityManager] selector:@selector(UpdateAllCachedProperties) userInfo:nil repeats:YES];
			}
		}, false);
	}, false);
}

void FMacApplication::OnVoiceoverDisabled()
{
	// Accessibility should originate from Main Thread
	check([NSThread isMainThread]);
	if(!GetAccessibleMessageHandler()->IsActive())
	{
		return;
	}
	GetAccessibleMessageHandler()->SetActive(false);
	MainThreadCall(^{
		{
			FScopeLock Lock(&WindowsMutex);
			for(const TSharedRef<FMacWindow>& CurrentWindow : Windows)
			{
				FCocoaWindow* CurrentCocoaWindow = CurrentWindow->GetWindowHandle();
				if(CurrentCocoaWindow)
				{
					[CurrentCocoaWindow ClearAccessibilityView];
				}
			}
		}
		[AccessibilityCacheTimer invalidate];
		AccessibilityCacheTimer = nil;
		[[FMacAccessibilityManager AccessibilityManager] Clear];
		//This releases the accessibility element referenced by the app, deallocating it
		NSApp.accessibilityApplicationFocusedUIElement = Nil;
	}, false);
}

#endif
