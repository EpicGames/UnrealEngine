// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/MacApplicationErrorOutputDevice.h"
#include "Mac/MacSystemIncludes.h"
#include "HAL/PlatformApplicationMisc.h"

void FMacApplicationErrorOutputDevice::HandleErrorRestoreUI()
{
	// Unhide the mouse.
	// @TODO: Remove usage of deprecated CGCursorIsVisible function
#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wdeprecated-declarations"
	while (!CGCursorIsVisible())
	{
		CGDisplayShowCursor(kCGDirectMainDisplay);
	}
#pragma clang diagnostic pop
	// Release capture and allow mouse to freely roam around.
	CGAssociateMouseAndMouseCursorPosition(true);

	FPlatformApplicationMisc::ClipboardCopy(GErrorHist);
}
