// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/MacConsoleOutputDevice.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ConfigCacheIni.h"
#include "Mac/MacApplication.h"
#include "Mac/MacPlatformApplicationMisc.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/ScopeLock.h"
#include "Mac/CocoaThread.h"

#include "HAL/PlatformApplicationMisc.h"
#include "CoreGlobals.h"

FMacConsoleOutputDevice::FMacConsoleOutputDevice()
	: ConsoleHandle(NULL)
	, TextView(NULL)
	, ScrollView(NULL)
	, TextViewTextColor(NULL)
	, OutstandingTasks(0)
{
	if (FParse::Param(FCommandLine::Get(), TEXT("SuppressConsoleOutputVerboseLogging")))
	{
		AllowedLogVerbosity = ELogVerbosity::Log;
	}
}

FMacConsoleOutputDevice::~FMacConsoleOutputDevice()
{
	DestroyConsole();
}

void FMacConsoleOutputDevice::SaveToINI()
{
	if (GConfig && IniFilename.Len())
	{
		NSRect Frame = [ConsoleHandle frame];

		GConfig->SetInt(TEXT("DebugMac"), TEXT("ConsoleWidth"), FMath::TruncToInt(Frame.size.width), IniFilename);
		GConfig->SetInt(TEXT("DebugMac"), TEXT("ConsoleHeight"), FMath::TruncToInt(Frame.size.height), IniFilename);
		GConfig->SetInt(TEXT("DebugMac"), TEXT("ConsoleX"), FMath::TruncToInt(Frame.origin.x), IniFilename);
		GConfig->SetInt(TEXT("DebugMac"), TEXT("ConsoleY"), FMath::TruncToInt(Frame.origin.y), IniFilename);
	}
}

void FMacConsoleOutputDevice::CreateConsole()
{
	if (ConsoleHandle || GIsBuildMachine)
	{
		return;
	}

	SCOPED_AUTORELEASE_POOL;

	int32 ConsoleWidth = 800;
	int32 ConsoleHeight = 600;
	int32 ConsolePosX = 0;
	int32 ConsolePosY = 0;
	bool bHasX = false;
	bool bHasY = false;

	if(GConfig)
	{
		GConfig->GetInt(TEXT("DebugMac"), TEXT("ConsoleWidth"), ConsoleWidth, GGameIni);
		GConfig->GetInt(TEXT("DebugMac"), TEXT("ConsoleHeight"), ConsoleHeight, GGameIni);
		bHasX = GConfig->GetInt(TEXT("DebugMac"), TEXT("ConsoleX"), ConsolePosX, GGameIni);
		bHasY = GConfig->GetInt(TEXT("DebugMac"), TEXT("ConsoleY"), ConsolePosY, GGameIni);
	}

	MainThreadCall(^{
		ConsoleHandle = [[FMacConsoleWindow alloc] initWithContentRect: NSMakeRect(ConsolePosX, ConsolePosY, ConsoleWidth, ConsoleHeight)
										styleMask: (NSWindowStyleMaskTitled | NSWindowStyleMaskClosable | NSWindowStyleMaskMiniaturizable | NSWindowStyleMaskResizable)
										  backing: NSBackingStoreBuffered
											defer: NO];
		[ConsoleHandle setDelegate:ConsoleHandle];

		ScrollView = [[NSScrollView alloc] initWithFrame:[[ConsoleHandle contentView] frame]];
		NSSize ContentSize = [ScrollView contentSize];
		
		[ScrollView setBorderType:NSNoBorder];
		[ScrollView setHasVerticalScroller:YES];
		[ScrollView setHasHorizontalScroller:NO];
		[ScrollView setAutoresizingMask:NSViewWidthSizable | NSViewHeightSizable];
		
		TextView = [[NSTextView alloc] initWithFrame:NSMakeRect( 0, 0, ContentSize.width, ContentSize.height )];
		[TextView setMinSize:NSMakeSize( 0.0, ContentSize.height ) ];
		[TextView setMaxSize:NSMakeSize( FLT_MAX, FLT_MAX )];
		[TextView setVerticallyResizable:YES];
		[TextView setHorizontallyResizable:NO];
		[TextView setAutoresizingMask:NSViewWidthSizable];
		[TextView setBackgroundColor: [NSColor blackColor]];
		
		[[TextView textContainer] setContainerSize:NSMakeSize( ContentSize.width, FLT_MAX )];
		[[TextView textContainer] setWidthTracksTextView:YES];
		
		[ScrollView setDocumentView:TextView];
		[ConsoleHandle setContentView:ScrollView];

		if (!bHasX || !bHasY)
		{
			[ConsoleHandle center];
		}

		[ConsoleHandle setOpaque:YES];
		[ConsoleHandle makeKeyAndOrderFront:nil];
		
		if(!MacApplication)
		{
			do
			{
				FMacPlatformApplicationMisc::PumpMessages( true );
			} while(ConsoleHandle && ![ConsoleHandle isVisible]);
		}
		
		SetDefaultTextColor();
	}, true, UnrealNilEventMode);
}

void FMacConsoleOutputDevice::DestroyConsole()
{
	if (ConsoleHandle)
	{
		SaveToINI();

		FMacConsoleWindow* ConsoleWindow = ConsoleHandle;
		ConsoleHandle = nullptr; // Stop further serialization as soon as possible

		if ([NSThread isGameThread])
		{
			do
			{
				FMacPlatformApplicationMisc::PumpMessages( true );
			} while(OutstandingTasks);
		}

		MainThreadCall(^{
			SCOPED_AUTORELEASE_POOL;
			if( TextViewTextColor )
				[TextViewTextColor release];
			
			[ConsoleWindow close];
			TextViewTextColor = NULL;
		}, true, UnrealNilEventMode);
	}
}

void FMacConsoleOutputDevice::Show( bool ShowWindow )
{
	if( ShowWindow )
	{
		CreateConsole();
	}
	else
	{
		DestroyConsole();
	}
}

bool FMacConsoleOutputDevice::IsShown()
{
	return ConsoleHandle != NULL;
}

void FMacConsoleOutputDevice::Serialize( const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category )
{
	if ((Verbosity & ELogVerbosity::VerbosityMask) > AllowedLogVerbosity)
		return;

	if( ConsoleHandle )
	{
		FScopeLock ScopeLock( &CriticalSection );

		static bool Entry=0;
		if( !GIsCriticalError || Entry )
		{
			// here we can change the color of the text to display, it's in the format:
			// ForegroundRed | ForegroundGreen | ForegroundBlue | ForegroundBright | BackgroundRed | BackgroundGreen | BackgroundBlue | BackgroundBright
			// where each value is either 0 or 1 (can leave off trailing 0's), so 
			// blue on bright yellow is "00101101" and red on black is "1"
			// An empty string reverts to the normal gray on black
			
			if (Verbosity == ELogVerbosity::SetColor)
			{
				if (FCString::Stricmp(Data, TEXT("")) == 0)
				{
					SetDefaultTextColor();
				}
				else
				{
					SCOPED_AUTORELEASE_POOL;
					
					// turn the string into a bunch of 0's and 1's
					TCHAR String[9];
					FMemory::Memset(String, 0, sizeof(TCHAR) * UE_ARRAY_COUNT(String));
					FCString::Strncpy(String, Data, UE_ARRAY_COUNT(String));
					for (TCHAR* S = String; *S; S++)
					{
						*S -= '0';
					}
					
					NSMutableArray* Colors = [[NSMutableArray alloc] init];
					NSMutableArray* AttributeKeys = [[NSMutableArray alloc] init];
					
					// Get FOREGROUND_INTENSITY and calculate final color
					CGFloat Intensity = String[3] ? 1.0 : 0.5;
					[Colors addObject:[NSColor colorWithSRGBRed:(String[0] ? 1.0 * Intensity : 0.0) green:(String[1] ? 1.0 * Intensity : 0.0) blue:(String[2] ? 1.0 * Intensity : 0.0) alpha:1.0]];
					
					// Get BACKGROUND_INTENSITY and calculate final color
					Intensity = String[7] ? 1.0 : 0.5;
					[Colors addObject:[NSColor colorWithSRGBRed:(String[4] ? 1.0 * Intensity : 0.0) green:(String[5] ? 1.0 * Intensity : 0.0) blue:(String[6] ? 1.0 * Intensity : 0.0) alpha:1.0]];
					
					[AttributeKeys addObject:NSForegroundColorAttributeName];
					[AttributeKeys addObject:NSBackgroundColorAttributeName];
					
					OutstandingTasks++;
					MainThreadCall(^{
						if( TextViewTextColor )
							[TextViewTextColor release];
						
						TextViewTextColor = [[NSDictionary alloc] initWithObjects:Colors forKeys:AttributeKeys];
						
						[Colors release];
						[AttributeKeys release];
						OutstandingTasks--;
					}, false);
				}
			}
			else
			{
				SCOPED_AUTORELEASE_POOL;
				
				FString OutputString = FString::Printf(TEXT("%s%s"), *FOutputDeviceHelper::FormatLogLine(Verbosity, Category, Data, GPrintLogTimes), LINE_TERMINATOR);

				OutstandingTasks++;
				MainThreadCall(^{
					NSAttributedString *AttributedString = [[NSAttributedString alloc] initWithString:OutputString.GetNSString() attributes:TextViewTextColor];
					[[TextView textStorage] appendAttributedString:AttributedString];
					[TextView scrollRangeToVisible:NSMakeRange([[TextView string] length], 0)];
					[AttributedString release];
					OutstandingTasks--;
				}, false);
				
				if(!MacApplication)
				{
					FMacPlatformApplicationMisc::PumpMessages( true );
				}
			}
		}
		else
		{
			Entry=1;
			try
			{
				// Ignore errors to prevent infinite-recursive exception reporting.
				Serialize( Data, Verbosity, Category );
			}
			catch( ... )
			{}
			Entry=0;
		}
	}
}

void FMacConsoleOutputDevice::SetDefaultTextColor()
{
	SCOPED_AUTORELEASE_POOL;
	FScopeLock ScopeLock( &CriticalSection );

	NSMutableArray* Colors = [[NSMutableArray alloc] init];
	NSMutableArray* AttributeKeys = [[NSMutableArray alloc] init];
	
	[Colors addObject:[NSColor grayColor]];
	[Colors addObject:[NSColor blackColor]];
	
	[AttributeKeys addObject:NSForegroundColorAttributeName];
	[AttributeKeys addObject:NSBackgroundColorAttributeName];

	OutstandingTasks++;
	MainThreadCall(^{
		if( TextViewTextColor )
			[TextViewTextColor release];
		
		TextViewTextColor = [[NSDictionary alloc] initWithObjects:Colors forKeys:AttributeKeys];
		
		[Colors release];
		[AttributeKeys release];
		OutstandingTasks--;
	}, false);
}

@implementation FMacConsoleWindow
- (void)windowWillClose:(NSNotification*)Notification
{
	// Only force exit when we are not already exiting
	if (!IsEngineExitRequested() && !MacApplication && [[NSApp orderedWindows] count] == 1)
	{
		UE_LOGF(LogCore, Warning, "*** INTERRUPTED *** : Console Window Closed, Shutting Down");
		if (GLog)
		{
			GLog->Panic();
		}
		_Exit(0);
	}
}
@end
