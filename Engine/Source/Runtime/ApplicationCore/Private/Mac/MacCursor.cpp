// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/MacCursor.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "Mac/MacWindow.h"
#include "Mac/MacApplication.h"
#include "Math/IntRect.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"
#include "Misc/CommandLine.h"

#import <IOKit/hid/IOHIDKeys.h>
#import <IOKit/hidsystem/IOHIDShared.h>

int32 GMacDisableMouseCoalescing = 1;
static FAutoConsoleVariableRef CVarMacDisableMouseCoalescing(
	TEXT("io.Mac.HighPrecisionDisablesMouseCoalescing"),
	GMacDisableMouseCoalescing,
	TEXT("If set to true then OS X mouse event coalescing will be disabled while using high-precision mouse mode, to send all mouse events to Unreal's event handling routines to reduce apparent mouse lag. (Default: True)"));

int32 GMacDisableMouseAcceleration = 0;
static FAutoConsoleVariableRef CVarMacDisableMouseAcceleration(
	TEXT("io.Mac.HighPrecisionDisablesMouseAcceleration"),
	GMacDisableMouseAcceleration,
	TEXT("If set to true then OS X's mouse acceleration curve will be disabled while using high-precision mouse mode (typically used when games capture the mouse) resulting in a linear relationship between mouse movement & on-screen cursor movement. For some pointing devices this will make the cursor very slow. (Default: False)"));

FMacCursor::FMacCursor()
:	bIsVisible(true)
,	bUseHighPrecisionMode(false)
,	CursorTypeOverride(-1)
,	CurrentPosition(0, 0)
,	MouseWarpDelta(0, 0)
,	bIsPositionInitialised(false)
,	bShouldIgnoreLocking(false)
,	HIDInterface(0)
,	SavedAcceleration(0)
{
	SCOPED_AUTORELEASE_POOL;

	// Load up cursors that we'll be using
	for (int32 CursorIndex = 0; CursorIndex < EMouseCursor::TotalCursorCount; ++CursorIndex)
	{
		CursorHandles[CursorIndex] = NULL;
		CursorOverrideHandles[CursorIndex] = NULL;

		NSCursor *CursorHandle = NULL;
		switch (CursorIndex)
		{
			case EMouseCursor::None:
			case EMouseCursor::Custom:
				break;

			case EMouseCursor::Default:
				CursorHandle = [NSCursor arrowCursor];
				break;

			case EMouseCursor::TextEditBeam:
				CursorHandle = [NSCursor IBeamCursor];
				break;

			case EMouseCursor::ResizeLeftRight:
				CursorHandle = [NSCursor resizeLeftRightCursor];
				break;

			case EMouseCursor::ResizeUpDown:
				CursorHandle = [NSCursor resizeUpDownCursor];
				break;

			case EMouseCursor::ResizeSouthEast:
			{
				FString Path = FString::Printf(TEXT("%s%sEditor/Slate/Cursor/SouthEastCursor.png"), FPlatformProcess::BaseDir(), *FPaths::EngineContentDir());
				NSImage* CursorImage = [[NSImage alloc] initWithContentsOfFile:Path.GetNSString()];
				CursorHandle = [[NSCursor alloc] initWithImage:CursorImage hotSpot:NSMakePoint(8, 8)];
				[CursorImage release];
				break;
			}

			case EMouseCursor::ResizeSouthWest:
			{
				FString Path = FString::Printf(TEXT("%s%sEditor/Slate/Cursor/SouthWestCursor.png"), FPlatformProcess::BaseDir(), *FPaths::EngineContentDir());
				NSImage* CursorImage = [[NSImage alloc] initWithContentsOfFile:Path.GetNSString()];
				CursorHandle = [[NSCursor alloc] initWithImage:CursorImage hotSpot:NSMakePoint(8, 8)];
				[CursorImage release];
				break;
			}

			case EMouseCursor::CardinalCross:
			{
				FString Path = FString::Printf(TEXT("%s%sEditor/Slate/Cursor/CardinalCrossCursor.png"), FPlatformProcess::BaseDir(), *FPaths::EngineContentDir());
				NSImage* CursorImage = [[NSImage alloc] initWithContentsOfFile:Path.GetNSString()];
				CursorHandle = [[NSCursor alloc] initWithImage:CursorImage hotSpot:NSMakePoint(8, 8)];
				[CursorImage release];
				break;
			}

			case EMouseCursor::Crosshairs:
				CursorHandle = [NSCursor crosshairCursor];
				break;

			case EMouseCursor::Hand:
				CursorHandle = [NSCursor pointingHandCursor];
				break;

			case EMouseCursor::GrabHand:
				CursorHandle = [NSCursor openHandCursor];
				break;

			case EMouseCursor::GrabHandClosed:
				CursorHandle = [NSCursor closedHandCursor];
				break;

			case EMouseCursor::SlashedCircle:
				CursorHandle = [NSCursor operationNotAllowedCursor];
				break;

			case EMouseCursor::EyeDropper:
			{
				FString Path = FString::Printf(TEXT("%s%sEditor/Slate/Cursor/EyeDropperCursor.png"), FPlatformProcess::BaseDir(), *FPaths::EngineContentDir());
				NSImage* CursorImage = [[NSImage alloc] initWithContentsOfFile:Path.GetNSString()];
				CursorHandle = [[NSCursor alloc] initWithImage:CursorImage hotSpot:NSMakePoint(1, 17)];
				[CursorImage release];
				break;
			}

			default:
				// Unrecognized cursor type!
				check(0);
				break;
		}

		CursorHandles[CursorIndex] = CursorHandle;
	}

#if !UE_BUILD_SHIPPING
	FParse::Value(FCommandLine::Get(), TEXT("MacCursorTypeOverride="), CursorTypeOverride);
	if (CursorTypeOverride < EMouseCursor::None || CursorTypeOverride >= EMouseCursor::TotalCursorCount)
	{
		CursorTypeOverride = -1;
	}
#endif

	// Set the default cursor
	SetType(EMouseCursor::Default);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Get the IOHIDSystem so we can disable mouse acceleration
	mach_port_t MainPort;
    kern_return_t KernResult;
    if (@available(macOS 12.0, iOS 15.0, *))
    {
        KernResult = IOMainPort(MACH_PORT_NULL, &MainPort);
    }
    else
    {
        KernResult = IOMasterPort(MACH_PORT_NULL, &MainPort);
    }
	if (KERN_SUCCESS == KernResult)
	{
		CFMutableDictionaryRef ClassesToMatch = IOServiceMatching("IOHIDSystem");
		if (ClassesToMatch)
		{
			io_iterator_t MatchingServices;
			KernResult = IOServiceGetMatchingServices(MainPort, ClassesToMatch, &MatchingServices);
			if (KERN_SUCCESS == KernResult)
			{
				io_object_t IntfService;
				if ((IntfService = IOIteratorNext(MatchingServices)))
				{
					KernResult = IOServiceOpen(IntfService, mach_task_self(), kIOHIDParamConnectType, &HIDInterface);
					if (KernResult == KERN_SUCCESS)
					{
						KernResult = IOHIDGetAccelerationWithKey(HIDInterface, CFSTR(kIOHIDMouseAccelerationType), &SavedAcceleration);
					}
					if (KERN_SUCCESS != KernResult)
					{
						HIDInterface = 0;
					}
				}
			}
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FMacCursor::~FMacCursor()
{
	SCOPED_AUTORELEASE_POOL;
	
	SetHighPrecisionMouseMode(false);
	
	// Release cursors
	// NOTE: Shared cursors will automatically be destroyed when the application is destroyed.
	//       For dynamically created cursors, use [CursorHandles[CursorIndex] release];
	for (int32 CursorIndex = 0; CursorIndex < EMouseCursor::TotalCursorCount; ++CursorIndex)
	{
		switch (CursorIndex)
		{
			case EMouseCursor::None:
			case EMouseCursor::Default:
			case EMouseCursor::TextEditBeam:
			case EMouseCursor::ResizeLeftRight:
			case EMouseCursor::ResizeUpDown:
			case EMouseCursor::Crosshairs:
			case EMouseCursor::Hand:
			case EMouseCursor::GrabHand:
			case EMouseCursor::GrabHandClosed:
			case EMouseCursor::SlashedCircle:
				// Standard shared cursors don't need to be destroyed
				break;

			case EMouseCursor::ResizeSouthEast:
			case EMouseCursor::ResizeSouthWest:
			case EMouseCursor::CardinalCross:
			case EMouseCursor::EyeDropper:
			case EMouseCursor::Custom:
				if (CursorHandles[CursorIndex] != NULL)
				{
					[CursorHandles[CursorIndex] release];
				}
				break;

			default:
				// Unrecognized cursor type!
				check(0);
				break;
		}
	}
}

void* FMacCursor::CreateCursorFromFile(const FString& InPathToCursorWithoutExtension, FVector2D InHotSpot)
{
	const FString TiffCursor = InPathToCursorWithoutExtension + TEXT(".tiff");
	const FString PngCursor = InPathToCursorWithoutExtension + TEXT(".png");

	// Try loading TIFF file
	NSImage* CursorImage = [[NSImage alloc] initWithContentsOfFile:TiffCursor.GetNSString()];
	if (!CursorImage)
	{
		// Try loading PNG file
		CursorImage = [[NSImage alloc] initWithContentsOfFile:PngCursor.GetNSString()];
	}

	if (!CursorImage)
	{
		return nullptr;
	}

	NSImageRep* CursorImageRep = [[CursorImage representations] objectAtIndex:0];

	const int32 PixelHotspotX = FMath::RoundToInt(InHotSpot.X * (CursorImageRep.size.width-1));
	const int32 PixelHotspotY = FMath::RoundToInt(InHotSpot.Y * (CursorImageRep.size.height-1));

	NSCursor* CursorHandle = [[NSCursor alloc] initWithImage:CursorImage hotSpot : NSMakePoint(PixelHotspotX, PixelHotspotY)];
	[CursorImage release];

	return CursorHandle;
}

void* FMacCursor::CreateCursorFromRGBABuffer(const FColor* Pixels, int32 Width, int32 Height, FVector2D InHotSpot)
{
	NSBitmapImageRep* CursorImageRep = [
		[NSBitmapImageRep alloc]
		initWithBitmapDataPlanes:NULL
			pixelsWide : Width
			pixelsHigh : Height
			bitsPerSample : 8
			samplesPerPixel : 4
			hasAlpha : YES
			isPlanar : NO
			colorSpaceName : NSCalibratedRGBColorSpace
			bitmapFormat : NSBitmapFormatAlphaFirst
			bytesPerRow : Width * 4
			bitsPerPixel : 32];

	uint8* CursorPixels = [CursorImageRep bitmapData];
	for (int32 X = 0; X < Width; X++)
	{
		for (int32 Y = 0; Y < Height; Y++)
		{
			uint8* TargetPixel = (uint8*) CursorPixels + Y * Width + X * 4;
			*(uint32*) TargetPixel = *(((uint32*) Pixels) + (Y * Width) + X);
		}
	}

	NSImage* CursorImage = [[NSImage alloc] initWithSize:NSMakeSize(Width, Height)];
	[CursorImage addRepresentation : CursorImageRep];

	const int32 PixelHotspotX = FMath::RoundToInt(InHotSpot.X * Width);
	const int32 PixelHotspotY = FMath::RoundToInt(InHotSpot.Y * Height);

	NSCursor* CursorHandle = [[NSCursor alloc] initWithImage:CursorImage hotSpot : NSMakePoint(PixelHotspotX, PixelHotspotY)];
	[CursorImage release];
	[CursorImageRep release];

	return CursorHandle;
}

FIntVector2 FMacCursor::GetIntPosition() const
{
	if (bIsPositionInitialised)
	{
		return CurrentPosition;
	}

	SCOPED_AUTORELEASE_POOL;
	NSPoint CursorPos = [NSEvent mouseLocation];
	FVector2D CurrentPos = FMacApplication::ConvertCocoaPositionToSlate(CursorPos.x, CursorPos.y);
	return FIntVector2(FMath::TruncToInt(CurrentPos.X), FMath::TruncToInt(CurrentPos.Y));
}

FVector2D FMacCursor::GetPosition() const
{
	FIntVector2 CurrentPos = GetIntPosition();
	return FVector2D(CurrentPos.X, CurrentPos.Y);
}

void FMacCursor::SetPosition(const int32 X, const int32 Y)
{
	FIntVector2 NewPos(X, Y);
	UpdateCursorClipping(NewPos);

	MouseWarpDelta.X += NewPos.X - CurrentPosition.X;
	MouseWarpDelta.Y += NewPos.Y - CurrentPosition.Y;

	if (!bIsPositionInitialised || NewPos != CurrentPosition)
	{
		if (!bUseHighPrecisionMode || (CurrentCursor && bIsVisible) || !bIsPositionInitialised)
		{
			WarpCursor(NewPos.X, NewPos.Y);
		}
		else
		{
			UpdateCurrentPosition(NewPos);
		}
	}
}

void FMacCursor::SetType(const EMouseCursor::Type InNewCursor)
{
	check(InNewCursor < EMouseCursor::TotalCursorCount);
	if (MacApplication && CurrentType == EMouseCursor::None && InNewCursor != EMouseCursor::None)
	{
		MacApplication->SetHighPrecisionMouseMode(false, nullptr);
	}
	
	CurrentType = CursorTypeOverride > -1 ? (EMouseCursor::Type)CursorTypeOverride : InNewCursor;
	CurrentCursor = CursorOverrideHandles[CurrentType] ? CursorOverrideHandles[CurrentType] : CursorHandles[CurrentType];

	if (CurrentCursor)
	{
		[CurrentCursor set];
	}

	UpdateVisibility();
}

void FMacCursor::GetSize(int32& Width, int32& Height) const
{
	// TODO: This isn't accurate
	Width = 16;
	Height = 16;
}

void FMacCursor::Show(bool bShow)
{
	bIsVisible = bShow;
	UpdateVisibility();
}

void FMacCursor::Lock(const RECT* const Bounds)
{
	SCOPED_AUTORELEASE_POOL;

	// Lock/Unlock the cursor
	if (Bounds == nullptr || bShouldIgnoreLocking)
	{
		CursorClipRect = FIntRect();
	}
	else
	{
		CursorClipRect.Min.X = Bounds->left;
		CursorClipRect.Min.Y = Bounds->top;
		CursorClipRect.Max.X = Bounds->right > Bounds->left ? Bounds->right - 1 : Bounds->left;
		CursorClipRect.Max.Y = Bounds->bottom > Bounds->top ? Bounds->bottom - 1 : Bounds->top;
	}

	MacApplication->OnCursorLock();

	if (!bUseHighPrecisionMode)
	{
		bIsPositionInitialised = false; // After exiting HighPrecisionMode, force GetIntPosition() to update its cached position in case the cursor was warped by another app while we were in high precision mode
	}
	FIntVector2 Position = GetIntPosition();
	if (UpdateCursorClipping(Position))
	{
		SetPosition(Position.X, Position.Y);
		WarpCursor(Position.X, Position.Y);
	}
}

bool FMacCursor::UpdateCursorClipping(FIntVector2& CursorPosition)
{
	bool bAdjusted = false;

	if (CursorClipRect.Area() > 0)
	{
		FIntVector2 PositionOnScreen(CursorPosition);
		FIntRect ClipRect(CursorClipRect);
		FIntVector2 ScreenOrigin(0, 0);

		if (PositionOnScreen.X < ClipRect.Min.X)
		{
			PositionOnScreen.X = ClipRect.Min.X;
			bAdjusted = true;
		}
		else if (PositionOnScreen.X > ClipRect.Max.X)
		{
			PositionOnScreen.X = ClipRect.Max.X;
			bAdjusted = true;
		}

		if (PositionOnScreen.Y < ClipRect.Min.Y)
		{
			PositionOnScreen.Y = ClipRect.Min.Y;
			bAdjusted = true;
		}
		else if (PositionOnScreen.Y > ClipRect.Max.Y)
		{
			PositionOnScreen.Y = ClipRect.Max.Y;
			bAdjusted = true;
		}

		if (bAdjusted)
		{
			CursorPosition.X = PositionOnScreen.X + ScreenOrigin.X;
			CursorPosition.Y = PositionOnScreen.Y + ScreenOrigin.Y;
		}
	}

	return bAdjusted;
}

void FMacCursor::UpdateVisibility()
{
	SCOPED_AUTORELEASE_POOL;
	// @TODO: Remove usage of deprecated functions
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if ([NSApp isActive])
	{
		if (CurrentCursor && bIsVisible)
		{
			// Enable the cursor.
			if (!CGCursorIsVisible())
			{
				CGDisplayShowCursor(kCGDirectMainDisplay);
			}
			if (GMacDisableMouseAcceleration && HIDInterface && bUseHighPrecisionMode)
			{
				IOHIDSetAccelerationWithKey(HIDInterface, CFSTR(kIOHIDMouseAccelerationType), SavedAcceleration);
			}
		}
		else
		{
			// Disable the cursor.
			if (CGCursorIsVisible())
			{
				CGDisplayHideCursor(kCGDirectMainDisplay);
			}
			if (GMacDisableMouseAcceleration && HIDInterface && bUseHighPrecisionMode && (!CurrentCursor || !bIsVisible))
			{
				IOHIDSetAccelerationWithKey(HIDInterface, CFSTR(kIOHIDMouseAccelerationType), -1);
			}
		}
	}
	else if (GMacDisableMouseAcceleration && HIDInterface && bUseHighPrecisionMode && (!CurrentCursor || !bIsVisible))
	{
		IOHIDSetAccelerationWithKey(HIDInterface, CFSTR(kIOHIDMouseAccelerationType), SavedAcceleration);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FMacCursor::UpdateCurrentPosition(const FIntVector2& Position)
{
	CurrentPosition = Position;
	bIsPositionInitialised = true;
}

void FMacCursor::WarpCursor(const int32 X, const int32 Y)
{
	// Apple suppress mouse events for 0.25 seconds after a call to Warp, unless we call CGAssociateMouseAndMouseCursorPosition.
	// Previously there was CGSetLocalEventsSuppressionInterval to explicitly control this behaviour but that is deprecated.
	// The replacement CGEventSourceSetLocalEventsSuppressionInterval isn't useful because it is unclear how to obtain the correct event source.
	// Instead, when we want the warp to be visible we need to disassociate mouse & cursor...
	if (!bUseHighPrecisionMode)
	{
		CGAssociateMouseAndMouseCursorPosition(false);
	}

	// Perform the warp as normal
	CGWarpMouseCursorPosition(FMacApplication::ConvertSlatePositionToCGPoint(X, Y));

	// And then reassociate the mouse cursor, which forces the mouse events to come through.
	if (!bUseHighPrecisionMode)
	{
		CGAssociateMouseAndMouseCursorPosition(true);
	}

	UpdateCurrentPosition(FIntVector2(X, Y));

	MacApplication->IgnoreMouseMoveDelta();
}

FIntVector2 FMacCursor::GetMouseWarpDelta()
{
	FIntVector2 Result = (!bUseHighPrecisionMode || (CurrentCursor && bIsVisible)) ? MouseWarpDelta : FIntVector2(0, 0);
	MouseWarpDelta = FIntVector2(0, 0);
	return Result;
}

void FMacCursor::SetHighPrecisionMouseMode(const bool bEnable)
{
	if (bUseHighPrecisionMode != bEnable)
	{
		bUseHighPrecisionMode = bEnable;

		CGAssociateMouseAndMouseCursorPosition(!bUseHighPrecisionMode);

		if (GMacDisableMouseCoalescing)
		{
			SCOPED_AUTORELEASE_POOL;
			[NSEvent setMouseCoalescingEnabled:!bUseHighPrecisionMode];
		}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (HIDInterface && GMacDisableMouseAcceleration && (!CurrentCursor || !bIsVisible))
		{
			if (!bUseHighPrecisionMode)
			{
				IOHIDSetAccelerationWithKey(HIDInterface, CFSTR(kIOHIDMouseAccelerationType), SavedAcceleration);
			}
			else
			{
				// Update the current saved acceleration.
				double CurrentSetting = 0;
				IOHIDGetAccelerationWithKey(HIDInterface, CFSTR(kIOHIDMouseAccelerationType), &CurrentSetting);
				
				// Need to check that we aren't picking up an invalid setting from ourselves.
				if (CurrentSetting >= 0 && CurrentSetting <= 3.0001)
				{
					SavedAcceleration = CurrentSetting;
				}

				IOHIDSetAccelerationWithKey(HIDInterface, CFSTR(kIOHIDMouseAccelerationType), -1);
			}
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		UpdateVisibility();

		// On disable put the cursor where the user would expect it
		if (!bEnable && (!CurrentCursor || !bIsVisible))
		{
			FIntVector2 Position = GetIntPosition();
			UpdateCursorClipping(Position);
			WarpCursor(Position.X, Position.Y);
		}
	}
}

void FMacCursor::SetTypeShape(EMouseCursor::Type InCursorType, void* InCursorHandle)
{
	NSCursor* CursorHandle = (NSCursor*)InCursorHandle;
	
	{
		SCOPED_AUTORELEASE_POOL;
		[CursorHandle retain];
		if (CursorOverrideHandles[InCursorType] != NULL)
		{
			[CursorOverrideHandles[InCursorType] release];
		}
		CursorOverrideHandles[InCursorType] = CursorHandle;
	}

	if (CurrentType == InCursorType)
	{
		SetType(CurrentType);
	}
}
