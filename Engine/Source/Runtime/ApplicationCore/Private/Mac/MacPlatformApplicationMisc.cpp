// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/MacPlatformApplicationMisc.h"
#include "Apple/AppleAsyncTask.h"
#include "Mac/MacPlatformMisc.h"
#include "Mac/MacApplication.h"
#include "Math/Color.h"
#include "Mac/MacMallocZone.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "Mac/MacConsoleOutputDevice.h"
#include "Mac/MacApplicationErrorOutputDevice.h"
#include "Mac/MacFeedbackContext.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/OutputDeviceRedirector.h"
#include "HAL/ThreadHeartBeat.h"
#include "HAL/FeedbackContextAnsi.h"
#include "Mac/CocoaMenu.h"
#include "Mac/CocoaThread.h"
#include "Mac/MacPlatformOutputDevices.h"

#include "Apple/PreAppleSystemHeaders.h"
#include <dlfcn.h>
#include <IOKit/IOKitLib.h>
#include <IOKit/kext/KextManager.h>
#include <IOKit/network/IOEthernetInterface.h>
#include <IOKit/network/IONetworkInterface.h>
#include <IOKit/network/IOEthernetController.h>
#include <IOKit/ps/IOPowerSources.h>
#include <IOKit/ps/IOPSKeys.h>
#include <IOKit/pwr_mgt/IOPMLib.h>
#include <mach-o/dyld.h>
#include <libproc.h>
#include <notify.h>
#include <uuid/uuid.h>
#import <ScreenCaptureKit/ScreenCaptureKit.h>

#include "Apple/ScopeAutoreleasePool.h"
#include "Apple/AppleStringUtils.h"
#include "Apple/PostAppleSystemHeaders.h"



extern FMacMallocCrashHandler* GCrashMalloc;

MacApplicationExternalCb FPlatformApplicationMisc::UpdateCachedMacMenuStateCb = nullptr;
MacApplicationExternalCb FPlatformApplicationMisc::PostInitMacMenuStartupCb = nullptr;
MacApplicationExternalCbOneBool FPlatformApplicationMisc::UpdateApplicationMenuCb = nullptr;
MacApplicationExternalCbOneBool FPlatformApplicationMisc::UpdateWindowMenuCb = nullptr;
MacApplicationExternalCb FPlatformApplicationMisc::LanguageChangedCb = nullptr;

bool FPlatformApplicationMisc::bChachedMacMenuStateNeedsUpdate = true;
bool FPlatformApplicationMisc::bLanguageChanged = false;
bool FPlatformApplicationMisc::bMacApplicationModalMode = false;
bool FPlatformApplicationMisc::bIsHighResolutionCapable = true;
bool FPlatformApplicationMisc::bDisplaySleepEnabled = true;

id<NSObject> FPlatformApplicationMisc::CommandletActivity = nil;

extern CORE_API TFunction<EAppReturnType::Type(EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption)> MessageBoxExtCallback;

static bool InitIsAppHighResolutionCapable()
{
	SCOPED_AUTORELEASE_POOL;

	static bool bInitialized = false;

	if (!bInitialized)
	{
		NSDictionary<NSString *,id>* BundleInfo = [[NSBundle mainBundle] infoDictionary];
		if (BundleInfo)
		{
			NSNumber* Value = (NSNumber*)[BundleInfo objectForKey:@"NSHighResolutionCapable"];
			if (Value)
			{
				FPlatformApplicationMisc::bIsHighResolutionCapable = [Value boolValue];
			}
		}
		
		bInitialized = true;
	}

	return FPlatformApplicationMisc::bIsHighResolutionCapable && GIsEditor;
}

EAppReturnType::Type MessageBoxExtImpl(EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption)
{
	FSlowHeartBeatScope SuspendHeartBeat;

	SCOPED_AUTORELEASE_POOL;

	EAppReturnType::Type ReturnValue = MainThreadReturn(^{
		EAppReturnType::Type RetValue = EAppReturnType::Cancel;
		NSInteger Result;

		NSAlert* AlertPanel = [NSAlert new];
		[AlertPanel setInformativeText:FString(Text).GetNSString()];
		[AlertPanel setMessageText:FString(Caption).GetNSString()];

		switch (MsgType)
		{
			case EAppMsgType::Ok:
				[AlertPanel addButtonWithTitle:@"OK"];
				[AlertPanel runModal];
				RetValue = EAppReturnType::Ok;
				break;

			case EAppMsgType::YesNo:
				[AlertPanel addButtonWithTitle:@"Yes"];
				[AlertPanel addButtonWithTitle:@"No"];
				Result = [AlertPanel runModal];
				if (Result == NSAlertFirstButtonReturn)
				{
					RetValue = EAppReturnType::Yes;
				}
				else if (Result == NSAlertSecondButtonReturn)
				{
					RetValue = EAppReturnType::No;
				}
				break;

			case EAppMsgType::OkCancel:
				[AlertPanel addButtonWithTitle:@"OK"];
				[AlertPanel addButtonWithTitle:@"Cancel"];
				Result = [AlertPanel runModal];
				if (Result == NSAlertFirstButtonReturn)
				{
					RetValue = EAppReturnType::Ok;
				}
				else if (Result == NSAlertSecondButtonReturn)
				{
					RetValue = EAppReturnType::Cancel;
				}
				break;

			case EAppMsgType::YesNoCancel:
				[AlertPanel addButtonWithTitle:@"Yes"];
				[AlertPanel addButtonWithTitle:@"No"];
				[AlertPanel addButtonWithTitle:@"Cancel"];
				Result = [AlertPanel runModal];
				if (Result == NSAlertFirstButtonReturn)
				{
					RetValue = EAppReturnType::Yes;
				}
				else if (Result == NSAlertSecondButtonReturn)
				{
					RetValue = EAppReturnType::No;
				}
				else
				{
					RetValue = EAppReturnType::Cancel;
				}
				break;

			case EAppMsgType::CancelRetryContinue:
				[AlertPanel addButtonWithTitle:@"Continue"];
				[AlertPanel addButtonWithTitle:@"Retry"];
				[AlertPanel addButtonWithTitle:@"Cancel"];
				Result = [AlertPanel runModal];
				if (Result == NSAlertFirstButtonReturn)
				{
					RetValue = EAppReturnType::Continue;
				}
				else if (Result == NSAlertSecondButtonReturn)
				{
					RetValue = EAppReturnType::Retry;
				}
				else
				{
					RetValue = EAppReturnType::Cancel;
				}
				break;

			case EAppMsgType::YesNoYesAllNoAll:
				[AlertPanel addButtonWithTitle:@"Yes"];
				[AlertPanel addButtonWithTitle:@"No"];
				[AlertPanel addButtonWithTitle:@"Yes to all"];
				[AlertPanel addButtonWithTitle:@"No to all"];
				Result = [AlertPanel runModal];
				if (Result == NSAlertFirstButtonReturn)
				{
					RetValue = EAppReturnType::Yes;
				}
				else if (Result == NSAlertSecondButtonReturn)
				{
					RetValue = EAppReturnType::No;
				}
				else if (Result == NSAlertThirdButtonReturn)
				{
					RetValue = EAppReturnType::YesAll;
				}
				else
				{
					RetValue = EAppReturnType::NoAll;
				}
				break;

			case EAppMsgType::YesNoYesAllNoAllCancel:
				[AlertPanel addButtonWithTitle:@"Yes"];
				[AlertPanel addButtonWithTitle:@"No"];
				[AlertPanel addButtonWithTitle:@"Yes to all"];
				[AlertPanel addButtonWithTitle:@"No to all"];
				[AlertPanel addButtonWithTitle:@"Cancel"];
				Result = [AlertPanel runModal];
				if (Result == NSAlertFirstButtonReturn)
				{
					RetValue = EAppReturnType::Yes;
				}
				else if (Result == NSAlertSecondButtonReturn)
				{
					RetValue = EAppReturnType::No;
				}
				else if (Result == NSAlertThirdButtonReturn)
				{
					RetValue = EAppReturnType::YesAll;
				}
				else if (Result == NSAlertThirdButtonReturn + 1)
				{
					RetValue = EAppReturnType::NoAll;
				}
				else
				{
					RetValue = EAppReturnType::Cancel;
				}
				break;

			case EAppMsgType::YesNoYesAll:
				[AlertPanel addButtonWithTitle:@"Yes"];
				[AlertPanel addButtonWithTitle:@"No"];
				[AlertPanel addButtonWithTitle:@"Yes to all"];
				Result = [AlertPanel runModal];
				if (Result == NSAlertFirstButtonReturn)
				{
					RetValue = EAppReturnType::Yes;
				}
				else if (Result == NSAlertSecondButtonReturn)
				{
					RetValue = EAppReturnType::No;
				}
				else
				{
					RetValue = EAppReturnType::YesAll;
				}
				break;

			default:
				break;
		}

		[AlertPanel release];

		return RetValue;
	});

	return ReturnValue;
}

void FMacPlatformApplicationMisc::PreInit()
{
	SCOPED_AUTORELEASE_POOL;

	// We don't support running from case-sensitive file systems on Mac yet
	NSURL* CurrentWorkingDirURL = [NSURL fileURLWithPath:[[NSFileManager defaultManager] currentDirectoryPath] isDirectory:YES];
	if (CurrentWorkingDirURL)
	{
		NSNumber* VolumeSupportsCaseSensitive;
		if ([CurrentWorkingDirURL getResourceValue:&VolumeSupportsCaseSensitive forKey:NSURLVolumeSupportsCaseSensitiveNamesKey error:nil])
		{
			if ([VolumeSupportsCaseSensitive boolValue])
			{
				MainThreadCall(^{
					NSAlert* AlertPanel = [NSAlert new];
					[AlertPanel setAlertStyle:NSAlertStyleCritical];
					[AlertPanel setInformativeText:[NSString stringWithFormat:@"Please install the application on a drive formatted as case-insensitive."]];
					[AlertPanel setMessageText:@"Unreal Engine does not support running from case-sensitive file systems."];
					[AlertPanel addButtonWithTitle:@"Quit"];
					[AlertPanel runModal];
					[AlertPanel release];
					exit(1);
				});
			}
		}
	}

	FMacApplication::UpdateScreensArray();
	MessageBoxExtCallback = MessageBoxExtImpl;
	FApp::SetHasFocusFunction(&FMacPlatformApplicationMisc::IsThisApplicationForeground);
}

void FMacPlatformApplicationMisc::PostInit()
{
	FMacPlatformMisc::PostInitMacAppInfoUpdate();

	InitIsAppHighResolutionCapable();

	if (MacApplication)
	{
		// Now that the engine is initialized we need to recalculate display work areas etc. that depend on DPI settings
		FMacApplication::OnDisplayReconfiguration(kCGNullDirectDisplay, kCGDisplayDesktopShapeChangedFlag, MacApplication);
	}
	
	if(PostInitMacMenuStartupCb != nullptr)
	{
		// Initial menu bar setup
		PostInitMacMenuStartupCb();
	}

	if (IsRunningDedicatedServer() || IsRunningCookCommandlet())
	{
		// During cooking and on dedicated server, we don't want macOS to put our app into App Nap mode.
		CommandletActivity = [[NSProcessInfo processInfo] beginActivityWithOptions:NSActivityUserInitiated reason:IsRunningCookCommandlet() ? @"Running cook commandlet" : @"Running dedicated server"];
		[CommandletActivity retain];
	}
	
	if(GIsEditor)
	{
		FInternationalization::Get().OnCultureChanged().AddLambda([]()
		{
			bLanguageChanged = true;
			bChachedMacMenuStateNeedsUpdate = true;
		});
	}
}

void FMacPlatformApplicationMisc::TearDown()
{
	if (CommandletActivity)
	{
		MainThreadCall(^{
			[[NSProcessInfo processInfo] endActivity:CommandletActivity];
			[CommandletActivity release];
		}, false);
		CommandletActivity = nil;
	}
}

void FMacPlatformApplicationMisc::LoadPreInitModules()
{
	FModuleManager::Get().LoadModule(TEXT("CoreAudio"));
	FModuleManager::Get().LoadModule(TEXT("AudioMixerCoreAudio"));
}

class FOutputDeviceConsole* FMacPlatformApplicationMisc::CreateConsoleOutputDevice()
{
	// this is a slightly different kind of singleton that gives ownership to the caller and should not be called more than once
	// this is a slightly different kind of singleton that gives ownership to the caller and should not be called more than once
	return new FMacConsoleOutputDevice();
}

class FOutputDeviceError* FMacPlatformApplicationMisc::GetErrorOutputDevice()
{
	static FMacApplicationErrorOutputDevice Singleton;
	return &Singleton;
}

class FFeedbackContext* FMacPlatformApplicationMisc::GetFeedbackContext()
{
#if WITH_EDITOR
	static FMacFeedbackContext Singleton;
	return &Singleton;
#else
	return FMacPlatformOutputDevices::GetFeedbackContext();
#endif
}

GenericApplication* FMacPlatformApplicationMisc::CreateApplication()
{
	return FMacApplication::CreateMacApplication();
}

void FMacPlatformApplicationMisc::RequestMinimize()
{
	[NSApp hide : nil];
}

bool FMacPlatformApplicationMisc::IsThisApplicationForeground()
{
	SCOPED_AUTORELEASE_POOL;
	return [NSApp isActive] && MacApplication && MacApplication->IsWorkspaceSessionActive();
}

bool FMacPlatformApplicationMisc::IsScreensaverEnabled()
{
	return bDisplaySleepEnabled;
}

bool FMacPlatformApplicationMisc::ControlScreensaver(EScreenSaverAction Action)
{
	static uint32 IOPMNoSleepAssertion = 0;
	
	switch(Action)
	{
		case EScreenSaverAction::Disable:
		{
			// Prevent display sleep.
			if(bDisplaySleepEnabled)
			{
				SCOPED_AUTORELEASE_POOL;
				
				//  NOTE: IOPMAssertionCreateWithName limits the string to 128 characters.
				FString ReasonForActivity = FString::Printf(TEXT("Running %s"), FApp::GetProjectName());
				
				CFStringRef ReasonForActivityCF = (CFStringRef)ReasonForActivity.GetNSString();
				
				IOReturn Success = IOPMAssertionCreateWithName(kIOPMAssertionTypeNoDisplaySleep, kIOPMAssertionLevelOn, ReasonForActivityCF, &IOPMNoSleepAssertion);
				bDisplaySleepEnabled = !(Success == kIOReturnSuccess);
				ensure(!bDisplaySleepEnabled);
			}
			break;
		}
		case EScreenSaverAction::Enable:
		{
			// Stop preventing display sleep now that we are done.
			if(!bDisplaySleepEnabled)
			{
				IOReturn Success = IOPMAssertionRelease(IOPMNoSleepAssertion);
				bDisplaySleepEnabled = (Success == kIOReturnSuccess);
				ensure(bDisplaySleepEnabled);
			}
			break;
		}
	}
	
	return true;
}

FLinearColor FMacPlatformApplicationMisc::GetScreenPixelColor(const FVector2D& InScreenPos, float /*InGamma*/)
{
	SCOPED_AUTORELEASE_POOL;
	__block bool ScreenshotDone = false;
	__block FColor Color(0, 0, 0);
#ifdef __MAC_15_2
	if (@available(macOS 15.2, *))
	{
	   const CGPoint Pt = FMacApplication::ConvertSlatePositionToCGPoint(InScreenPos.X, InScreenPos.Y);
	   [SCScreenshotManager captureImageInRect:CGRectMake(Pt.x, Pt.y, 1, 1) completionHandler:^(CGImageRef sampleBuffer, NSError *error)
		{
			if (!error && sampleBuffer)
			{
				CFDataRef ImageDataRef = CGDataProviderCopyData(CGImageGetDataProvider(sampleBuffer));
				const UInt8* rawData = CFDataGetBytePtr(ImageDataRef);
				Color = FColor(rawData[2], rawData[1], rawData[0]);
				CFRelease(ImageDataRef);
				// Somehow we need to release sampleBuffer manually, or else the memory will leak
				// This is contradictory to Cocoa API naming guidelines, and to a similar API captureImageWithFilter:configuration:completionHandler: 
				CGImageRelease(sampleBuffer);
			}
			ScreenshotDone = true;
		}];
	}
	else
#endif
	{
		[SCShareableContent getShareableContentWithCompletionHandler:^(SCShareableContent *shareableContent, NSError *error)
		 {
			if (!error && shareableContent)
			{
				const CGPoint Pt = FMacApplication::ConvertSlatePositionToCGPoint(InScreenPos.X, InScreenPos.Y);
				CGDirectDisplayID matchingDisplay;
				uint32_t matchingDisplayCount;
				CGError cgError = CGGetDisplaysWithRect(CGRectMake(Pt.x, Pt.y, 1, 1), 64, &matchingDisplay, &matchingDisplayCount);
				assert(cgError == kCGErrorSuccess);
				assert(matchingDisplayCount == 1);
				SCDisplay* shareableDisplay = nil;
				for (SCDisplay* scDisplay in shareableContent.displays)
				{
					if (scDisplay.displayID == matchingDisplay)
					{
						shareableDisplay = scDisplay;
						break;
					}
				}
				SCContentFilter* Filter = [[[SCContentFilter alloc] initWithDisplay:shareableDisplay excludingWindows:[NSArray array]] autorelease];
				FMacScreenRef Screen = FMacApplication::FindScreenBySlatePosition(InScreenPos.X, InScreenPos.Y);
				SCStreamConfiguration* stream = [[[SCStreamConfiguration alloc] init] autorelease];
				stream.width = 100;
				stream.height = 100;
				stream.sourceRect = CGRectMake(Pt.x - Screen->Frame.origin.x, Pt.y - Screen->Frame.origin.y, 1, 1);
				stream.pixelFormat = 'BGRA';
				stream.showsCursor = false;
				[SCScreenshotManager captureImageWithFilter:Filter
											  configuration:stream
										  completionHandler:^(CGImageRef sampleBuffer, NSError *error2)
				 {
					if (!error2 && sampleBuffer)
					{
						CFDataRef ImageDataRef = CGDataProviderCopyData(CGImageGetDataProvider(sampleBuffer));
						const UInt8* rawData = CFDataGetBytePtr(ImageDataRef);
						Color = FColor(rawData[2], rawData[1], rawData[0]);
						CFRelease(ImageDataRef);
					}
					ScreenshotDone = true;
				 }];
			}
			else
			{
				ScreenshotDone = true;
			}
		 }];
	}
	
	float Timer = 0;
	do {
		FPlatformProcess::Sleep(0.01f);
		if (ScreenshotDone || Timer > 1.0f)	// usually takes ~0.05s
			break; 
		else 
			Timer += 0.01f;
	} while (true);

	FLinearColor ScreenLinearColor = FLinearColor::FromPow22Color(Color);

	return ScreenLinearColor;
}

float FMacPlatformApplicationMisc::GetDPIScaleFactorAtPoint(float X, float Y)
{
	if (MacApplication && FPlatformApplicationMisc::IsHighDPIModeEnabled())
	{
		FMacScreenRef Screen = FMacApplication::FindScreenBySlatePosition(X, Y);
		return (float)Screen->Screen.backingScaleFactor;
	}
	return 1.0f;
}

void FMacPlatformApplicationMisc::PumpMessages(bool bFromMainLoop)
{
	if( bFromMainLoop )
	{
		ProcessGameThreadEvents();

		if (MacApplication && !MacApplication->IsProcessingDeferredEvents() && IsInGameThread())
		{
			if (UpdateCachedMacMenuStateCb && bChachedMacMenuStateNeedsUpdate)
			{
				MainThreadCall(^
				{
					if(bLanguageChanged)
					{
						LanguageChanged();
					}
					
					UpdateApplicationMenu();
					UpdateWindowMenu();
					UpdateCocoaWindows();
				}, false);

				UpdateCachedMacMenuStateCb();
				
                bChachedMacMenuStateNeedsUpdate = false;
                bLanguageChanged = false;
			}
		}
	}
}

void FMacPlatformApplicationMisc::ClipboardCopy(const TCHAR* Str)
{
	// Don't attempt to copy the text to the clipboard if we've crashed or we'll crash again & become unkillable.
	// The MallocZone used for crash reporting will be enabled before this call if we've crashed so that will do for testing.
	if (UE::Private::GMalloc != GCrashMalloc)
	{
		SCOPED_AUTORELEASE_POOL;

		NSString* CocoaString = FString(Str).GetNSString();
		if (CocoaString)
		{
			NSPasteboard *Pasteboard = [NSPasteboard generalPasteboard];
			[Pasteboard clearContents];
			NSPasteboardItem *Item = [[[NSPasteboardItem alloc] init] autorelease];
			[Item setString:CocoaString forType:NSPasteboardTypeString];
			[Pasteboard writeObjects:[NSArray arrayWithObject:Item]];
		}
		else
		{
			UE_LOGF(LogMac, Warning, "Failed to create NSString to copy text to the pasteboard.");
		}
	}
}

void FMacPlatformApplicationMisc::ClipboardPaste(class FString& Result)
{
	SCOPED_AUTORELEASE_POOL;

	NSPasteboard *Pasteboard = [NSPasteboard generalPasteboard];
	NSString *CocoaString = [Pasteboard stringForType: NSPasteboardTypeString];
	if (CocoaString)
	{
		Result = FString(CocoaString);
	}
	else
	{
		Result = TEXT("");
	}
}

void* FMacPlatformApplicationMisc::GetActiveWindowPtr()
{
	return [NSApp keyWindow];
}

void FMacPlatformApplicationMisc::ProcessAsyncTasks()
{
	[FAppleAsyncTask ProcessAsyncTasks];
}
bool FMacPlatformApplicationMisc::IsMainThread()
{
	return [NSThread isMainThread];
}

void FMacPlatformApplicationMisc::ActivateApplication()
{
	MainThreadCall(^{
		[NSApp activateIgnoringOtherApps:YES];
	}, false);
}

void FMacPlatformApplicationMisc::UpdateApplicationMenu()
{
	if(UpdateApplicationMenuCb != nullptr)
	{
		UpdateApplicationMenuCb(bMacApplicationModalMode);
	}
}

void FMacPlatformApplicationMisc::LanguageChanged()
{
	if(LanguageChangedCb != nullptr)
	{
		LanguageChangedCb();
	}
}

void FMacPlatformApplicationMisc::UpdateWindowMenu()
{
	if(UpdateWindowMenuCb != nullptr)
	{
		UpdateWindowMenuCb(bMacApplicationModalMode);
	}
}


void FMacPlatformApplicationMisc::UpdateCocoaWindows()
{
	MacApplication->GetWindowsArrayMutex().Lock();

    const TArray<TSharedRef<FMacWindow>>&AllWindows = MacApplication->GetAllWindows();
    for (auto Window : AllWindows)
    {
        NSWindow* WindowHandle = Window->GetWindowHandle();
        {
            NSButton* CloseButton = [WindowHandle standardWindowButton:NSWindowCloseButton];
            NSButton* MinimizeButton = [WindowHandle standardWindowButton:NSWindowMiniaturizeButton];
            NSButton* MaximizeButton = [WindowHandle standardWindowButton:NSWindowZoomButton];
            if(bMacApplicationModalMode && WindowHandle != [NSApp mainWindow])
            {
                CloseButton.enabled = false;
                MinimizeButton.enabled = false;
                MaximizeButton.enabled = false;

            	if (!Window->GetDefinition().IsModalWindow)
            	{
            		Window->GetWindowHandle().styleMask &= ~NSWindowStyleMaskResizable;
            	}
            }
            else if(!bMacApplicationModalMode)
            {
                CloseButton.enabled = Window->GetDefinition().HasCloseButton;
                MinimizeButton.enabled = Window->GetDefinition().SupportsMinimize;
                MaximizeButton.enabled = Window->GetDefinition().SupportsMaximize;
            	
            	if (!Window->GetDefinition().IsModalWindow && (Window->GetDefinition().SupportsMaximize || Window->GetDefinition().HasSizingFrame))
            	{
            		Window->GetWindowHandle().styleMask |= NSWindowStyleMaskResizable;
            	}
            }
        }
    }

	MacApplication->GetWindowsArrayMutex().Unlock();
}
