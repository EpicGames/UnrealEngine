// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/MacPlatformFramePacer.h"
#include "Mac/MacSystemIncludes.h"
#include "Containers/Array.h"
#include "HAL/ThreadingBase.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Parse.h"

#include "HAL/PlatformTime.h"



/*******************************************************************
 * FMacFramePacer implementation
 *******************************************************************/

struct FMacFramePacer
{
	void Run(CGDirectDisplayID Display);
	void Signal(CGDirectDisplayID Display, double OutputSeconds, double OutputDuration);
	void Stop(CGDirectDisplayID Display);
	void Stop();
	
	void AddEvent(CGDirectDisplayID Display, FEvent* Event);
	void AddHandler(FMacFramePacerHandler Handler);
	
	void RemoveHandler(FMacFramePacerHandler Handler);
	
	static CVReturn DisplayLinkCallback(CVDisplayLinkRef DisplayLink, const CVTimeStamp* Now, const CVTimeStamp* OutputTime, CVOptionFlags FlagsIn, CVOptionFlags* FlagsOut, void* DisplayLinkContext);

	static FCriticalSection Mutex;

	// Owned display links
	static TMap<CGDirectDisplayID, CVDisplayLinkRef> DisplayLinks;

	// Collection of events listing for a specific display.
	static TMap<CGDirectDisplayID, TArray<FEvent*>> SpecificEvents;

	// Collection of events listening for any DisplayLink event.
	static NSMutableSet<FMacFramePacerHandler>* ListeningHandlers;
};
FCriticalSection FMacFramePacer::Mutex;
TMap<CGDirectDisplayID, CVDisplayLinkRef> FMacFramePacer::DisplayLinks;
TMap<CGDirectDisplayID, TArray<FEvent*>> FMacFramePacer::SpecificEvents;
NSMutableSet<FMacFramePacerHandler>* FMacFramePacer::ListeningHandlers = [NSMutableSet new];


CVReturn FMacFramePacer::DisplayLinkCallback(CVDisplayLinkRef DisplayLink, const CVTimeStamp* Now, const CVTimeStamp* OutputTime, CVOptionFlags FlagsIn, CVOptionFlags* FlagsOut, void* DisplayLinkContext)
{
	check(DisplayLink);
	FMacFramePacer* Pacer = (FMacFramePacer*)DisplayLinkContext;
	double OutputSeconds = OutputTime && (OutputTime->flags & kCVTimeStampHostTimeValid) ? FPlatformTime::ToSeconds64(OutputTime->hostTime) : FPlatformTime::ToSeconds64(mach_absolute_time());
	double OutputDuration = CVDisplayLinkGetActualOutputVideoRefreshPeriod(DisplayLink);
	Pacer->Signal(CVDisplayLinkGetCurrentCGDisplay(DisplayLink), OutputSeconds, OutputDuration);
	return kCVReturnSuccess;
}

void FMacFramePacer::Signal(CGDirectDisplayID Display, double OutputSeconds, double OutputDuration)
{
	FScopeLock Lock(&Mutex);
	
	for (FMacFramePacerHandler Block in ListeningHandlers)
	{
		Block(Display, OutputSeconds, OutputDuration);
	}
	
	TArray<FEvent*> const& Events = SpecificEvents.FindRef(Display);
	for (FEvent* Event : Events)
	{
		Event->Trigger();
	}
}

void FMacFramePacer::Run(CGDirectDisplayID Display)
{
	FScopeLock Lock(&Mutex);
	
	if (!DisplayLinks.FindRef(Display))
	{
		CVDisplayLinkRef displayLink; //display link for managing rendering thread
	
		// Create a display link capable of being used with all active displays
	    CVDisplayLinkCreateWithActiveCGDisplays(&displayLink);
	 
	    // Set the renderer output callback function
	    CVDisplayLinkSetOutputCallback(displayLink, &FMacFramePacer::DisplayLinkCallback, this);
	    
	    CVDisplayLinkSetCurrentCGDisplay(displayLink, Display);
	    
	    CVDisplayLinkStart(displayLink);
	    
	    DisplayLinks.Add(Display, displayLink);
    }
}

void FMacFramePacer::Stop(CGDirectDisplayID Display)
{
	FScopeLock Lock(&Mutex);
	if (DisplayLinks.FindRef(Display))
	{
		CVDisplayLinkRef displayLink = DisplayLinks.FindChecked(Display);
		
		CVDisplayLinkStop(displayLink);
		
		DisplayLinks.Remove(Display);

		SpecificEvents.Remove(Display);
	}
}

void FMacFramePacer::Stop()
{
	FScopeLock Lock(&Mutex);
	for (TPair<CGDirectDisplayID, CVDisplayLinkRef> DisplayLinkPair : DisplayLinks)
	{
		CVDisplayLinkStop(DisplayLinkPair.Value);
		
		CVDisplayLinkRelease(DisplayLinkPair.Value);

		SpecificEvents.Remove(DisplayLinkPair.Key);
	}
	DisplayLinks.Empty();
	[ListeningHandlers removeAllObjects];
}

void FMacFramePacer::AddEvent(CGDirectDisplayID Display, FEvent* Event)
{
	FScopeLock Lock(&Mutex);
	TArray<FEvent*>& Events = SpecificEvents.FindOrAdd(Display);
	Events.Add(Event);
}

void FMacFramePacer::AddHandler(FMacFramePacerHandler Handler)
{
	FScopeLock Lock(&Mutex);
	FMacFramePacerHandler Block = (FMacFramePacerHandler)Block_copy(Handler);
	[ListeningHandlers addObject:Block];
	Block_release(Block);
}

void FMacFramePacer::RemoveHandler(FMacFramePacerHandler Handler)
{
	FScopeLock Lock(&Mutex);
	[ListeningHandlers removeObject:Handler];
}


/*******************************************************************
 * FMacPlatformRHIFramePacer implementation
 *******************************************************************/


FMacFramePacer* FMacPlatformRHIFramePacer::FramePacer = nil;

bool FMacPlatformRHIFramePacer::IsEnabled()
{
	static bool bIsRHIFramePacerEnabled = false;
	static bool bInitialized = false;

	if (!bInitialized)
	{
		GConfig->GetBool(TEXT("/Script/MacTargetPlatform.MacTargetSettings"), TEXT("FrameRateLock"), bIsRHIFramePacerEnabled, GEngineIni);
		bInitialized = true;
	}
	
	return bIsRHIFramePacerEnabled;
}

void FMacPlatformRHIFramePacer::InitWithEvent(FEvent* TriggeredEvent)
{
    // Create display link thread
    FramePacer = new FMacFramePacer;
        
    // Default to the main display, but we'll configure display links for all the displays active.
    FramePacer->AddEvent( CGMainDisplayID(), TriggeredEvent );
    
    // Run for the active displays by default
    // @todo we really need to handle display addition & subtraction!
    uint32_t NumDisplays = 0;
    CGGetActiveDisplayList(0, nullptr, &NumDisplays);
    if (NumDisplays > 0)
    {
	    CGDirectDisplayID* DisplayArray = (CGDirectDisplayID*)alloca(sizeof(CGDirectDisplayID) * NumDisplays);
	    CGGetActiveDisplayList(NumDisplays, DisplayArray, &NumDisplays);
	    
	    for (uint32 i = 0; i < NumDisplays; i++)
	    {
    		FramePacer->Run(DisplayArray[i]);
	    }
    }
}

void FMacPlatformRHIFramePacer::AddHandler(FMacFramePacerHandler Handler)
{
	check (FramePacer);
	FramePacer->AddHandler(Handler);
}

void FMacPlatformRHIFramePacer::AddEvent(uint32 Display, class FEvent* TriggeredEvent)
{
	check(FramePacer);
	
	FramePacer->AddEvent(Display, TriggeredEvent);
	FramePacer->Run(Display);
}

void FMacPlatformRHIFramePacer::RemoveHandler(FMacFramePacerHandler Handler)
{
	check(FramePacer);
	FramePacer->RemoveHandler(Handler);
}

void FMacPlatformRHIFramePacer::Destroy()
{
    if( FramePacer != nil )
    {
    	FramePacer->Stop();
    	delete FramePacer;
    	FramePacer = nil;
    }
}
