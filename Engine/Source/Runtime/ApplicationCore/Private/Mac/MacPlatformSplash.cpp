// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/MacPlatformSplash.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "Misc/App.h"
#include "Misc/ScopeLock.h"
#include "Misc/EngineVersion.h"
#include "Misc/EngineBuildSettings.h"
#include "Mac/CocoaThread.h"

#include "HAL/PlatformApplicationMisc.h"
#include "Misc/ScopeLock.h"

/**
 * Simple window class that overrides a couple of functions, so the splash window can be moved to front even if it's borderless
 */
@interface FSplashWindow : NSWindow <NSWindowDelegate>

@end

@implementation FSplashWindow

- (BOOL)canBecomeMainWindow
{
	return YES;
}

- (BOOL)canBecomeKeyWindow
{
	return YES;
}

@end

/**
 * Splash screen functions and static globals
 */

static FCriticalSection GSplashMutex;
static FSplashWindow *GSplashWindow = NULL;
static NSImage *GSplashScreenImage = NULL;
static FText GSplashScreenAppName;
static FText GSplashScreenText[ SplashTextType::NumTextTypes ];
static NSRect GSplashScreenTextRects[ SplashTextType::NumTextTypes ];

/**
 * Sets the text displayed on the splash screen (for startup/loading progress)
 *
 * @param	InType		Type of text to change
 * @param	InText		Text to display
 */
static void StartSetSplashText( const SplashTextType::Type InType, const FText& InText )
{
	// Update splash text
	FScopeLock Lock(&GSplashMutex);
	GSplashScreenText[ InType ] = InText;
}

@interface UnrealSplashView : NSView
{
}

- (void)drawRect: (NSRect)DirtyRect;
- (void)drawText: (NSString *)Text inRect: (NSRect)Rect withAlignment: (NSTextAlignment)align withColor: (NSColor *)Color fontName: (NSString *)FontName fontSize: (int32)FontSize;

@end

@implementation UnrealSplashView

- (void)drawRect: (NSRect)DirtyRect
{
	SCOPED_AUTORELEASE_POOL;
	FScopeLock Lock(&GSplashMutex);

	// Draw background
	[GSplashScreenImage drawAtPoint: NSMakePoint(0,0) fromRect: NSZeroRect operation: NSCompositingOperationCopy fraction: 1.0];

	for( int32 CurTypeIndex = 0; CurTypeIndex < SplashTextType::NumTextTypes; ++CurTypeIndex )
	{
		const FText& SplashText = GSplashScreenText[ CurTypeIndex ];
		const NSRect& TextRect = GSplashScreenTextRects[ CurTypeIndex ];

		if( SplashText.ToString().Len() > 0 )
		{
			int32 FontSize = 9;
			switch ( CurTypeIndex )
			{
				case SplashTextType::StartupProgress:
				case SplashTextType::VersionInfo1:
					FontSize = 12;
					break;
				case SplashTextType::GameName:
					FontSize = 18;
					break;
			}

			float Brightness = 160.0f / 255.0f;

			switch ( CurTypeIndex )
			{
				case SplashTextType::StartupProgress:
				case SplashTextType::VersionInfo1:
					Brightness = 180.0f / 255.0f;
					break;
				case SplashTextType::GameName:
					Brightness = 240.0f / 255.0f;
					break;
			}

			NSColor *TextColor = [NSColor colorWithDeviceRed: Brightness green: Brightness blue: Brightness alpha: 1.0f];

			NSString* FontName = @"Helvetica";
			switch ( CurTypeIndex )
			{
			case SplashTextType::GameName:
				FontName = @"Helvetica-Bold";
				break;
			}

			// Alignment
			NSTextAlignment align = NSTextAlignmentLeft;
			switch ( CurTypeIndex )
			{
			case SplashTextType::CopyrightInfo:
				align = NSTextAlignmentRight;
				break;
			}
			
			// Cope with the possibility that users disable or remove fonts so they cannot be used anymore.
			NSFont* Font = [NSFont fontWithName:FontName size:FontSize];
			if (Font == nil)
			{
				// Fallback to the system font for the given size, which must exist
				Font = [NSFont systemFontOfSize:FontSize];
				check(Font);
				
				// Use this font's name
				FontName = [Font fontName];
			}

			NSString *Text = (NSString *)FPlatformString::TCHARToCFString(*SplashText.ToString());
			if ( Text )
			{
				[self drawText: Text inRect: TextRect withAlignment: align withColor: TextColor fontName: FontName fontSize: FontSize];
				[Text release];
			}
		}
	}
}

- (void)drawText: (NSString *)Text inRect: (NSRect)Rect withAlignment: (NSTextAlignment)align withColor: (NSColor *)Color fontName: (NSString *)FontName fontSize: (int32)FontSize
{
	SCOPED_AUTORELEASE_POOL;

	NSMutableParagraphStyle *style = [[[NSParagraphStyle defaultParagraphStyle] mutableCopy] autorelease];
	[style setAlignment : align];

	NSDictionary* Dict =
		[NSDictionary dictionaryWithObjects:
			[NSArray arrayWithObjects:
				Color,
				[NSFont fontWithName: FontName size: FontSize],
				[NSColor colorWithDeviceRed: 0.0 green: 0.0 blue: 0.0 alpha: 1.0],
				style,
				nil ]
		forKeys:
			[NSArray arrayWithObjects:
				NSForegroundColorAttributeName,
				NSFontAttributeName,
				NSStrokeColorAttributeName,
				NSParagraphStyleAttributeName,
				nil]
		];
	[Text drawInRect: Rect withAttributes: Dict];
}

@end

void FMacPlatformSplash::Show()
{
	if( !GSplashWindow && FParse::Param(FCommandLine::Get(),TEXT("NOSPLASH")) != true )
	{
		SCOPED_AUTORELEASE_POOL;

        const FText GameName = FText::FromString(FApp::GetProjectName());
        
		const TCHAR* SplashImage = GIsEditor ? TEXT("EdSplash") : TEXT("Splash");

		// make sure a splash was found
		FString SplashPath;
		bool IsCustom;
		if ( GetSplashPath(SplashImage, SplashPath, IsCustom) == true )
		{
			// Don't set the game name if the splash screen is custom.
			if ( !IsCustom )
			{
				StartSetSplashText( SplashTextType::GameName, GameName );
			}

			// In the editor, we'll display loading info
			if( GIsEditor )
			{
				// Set initial startup progress info
				{
					StartSetSplashText( SplashTextType::StartupProgress,
						NSLOCTEXT("UnrealEd", "SplashScreen_InitialStartupProgress", "Loading..." ) );
				}

				// Set version info
				{
					const FText Version = FText::FromString( FEngineVersion::Current().ToString( FEngineBuildSettings::IsPerforceBuild() ? EVersionComponent::Branch : EVersionComponent::Patch ) );

					FText VersionInfo;
					FText AppName;
					if( GameName.IsEmpty() )
					{
						VersionInfo = FText::Format( NSLOCTEXT( "UnrealEd", "UnrealEdTitleWithVersionNoGameName_F", "Unreal Editor {0}" ), Version );
						AppName = NSLOCTEXT( "UnrealEd", "UnrealEdTitleNoGameName_F", "Unreal Editor" );
						StartSetSplashText( SplashTextType::GameName, AppName);
					}
					else
					{
						VersionInfo = FText::Format( NSLOCTEXT( "UnrealEd", "UnrealEdTitleWithVersionWithGameName_F", "Unreal Editor {0}  -  {1}" ), Version, GameName );
						AppName = FText::Format( NSLOCTEXT( "UnrealEd", "UnrealEdTitle_F", "Unreal Editor - {0}" ), GameName );
					}

					StartSetSplashText( SplashTextType::VersionInfo1, VersionInfo );

					// Change the window text (which will be displayed in the taskbar)
					GSplashScreenAppName = AppName;
				}

				// Display copyright information in editor splash screen
				{
					const FText CopyrightInfo = NSLOCTEXT( "UnrealEd", "SplashScreen_CopyrightInfo", "Copyright \u00a9   Epic Games, Inc.   All rights reserved." );
					StartSetSplashText( SplashTextType::CopyrightInfo, CopyrightInfo );
				}
			}
		}

		NSRect ContentRect;
		{
			FScopeLock Lock(&GSplashMutex);
			
			NSString *SplashScreenFileName = (NSString *)FPlatformString::TCHARToCFString(*SplashPath);
			GSplashScreenImage = [[NSImage alloc] initWithContentsOfFile: SplashScreenFileName];
			[SplashScreenFileName release];
			
			NSBitmapImageRep* BitmapRep = [NSBitmapImageRep imageRepWithData: [GSplashScreenImage TIFFRepresentation]];
			float ImageWidth = static_cast<float>([BitmapRep pixelsWide]);
			float ImageHeight = static_cast<float>([BitmapRep pixelsHigh]);
			[GSplashScreenImage setSize:NSMakeSize(ImageWidth, ImageHeight)];
			
			ContentRect.origin.x = 0;
			ContentRect.origin.y = 0;
			ContentRect.size.width = ImageWidth;
			ContentRect.size.height = ImageHeight;
			
			int32 OriginX = 10;
			int32 OriginY = 6;
			int32 FontHeight = 14;
			
			// Setup bounds for game name
			GSplashScreenTextRects[ SplashTextType::GameName ].origin.x = OriginX;
			GSplashScreenTextRects[ SplashTextType::GameName ].origin.y = GIsEditor ? OriginY + 34 : OriginY + 18;
			GSplashScreenTextRects[ SplashTextType::GameName ].size.width = ImageWidth - static_cast<float>(2 * OriginX);
			GSplashScreenTextRects[ SplashTextType::GameName ].size.height = 2 * FontHeight;
			
			// Setup bounds for texts
			GSplashScreenTextRects[ SplashTextType::VersionInfo1 ].origin.x =
			GSplashScreenTextRects[ SplashTextType::CopyrightInfo ].origin.x =
			GSplashScreenTextRects[ SplashTextType::StartupProgress ].origin.x = OriginX;
			
			GSplashScreenTextRects[ SplashTextType::VersionInfo1 ].size.width =
			GSplashScreenTextRects[ SplashTextType::CopyrightInfo ].size.width =
			GSplashScreenTextRects[ SplashTextType::StartupProgress ].size.width = ImageWidth - static_cast<float>(2 * OriginX);
			
			GSplashScreenTextRects[ SplashTextType::VersionInfo1 ].size.height =
			GSplashScreenTextRects[ SplashTextType::StartupProgress ].size.height = FontHeight;
			GSplashScreenTextRects[ SplashTextType::CopyrightInfo ].size.height = FontHeight;
			
			GSplashScreenTextRects[ SplashTextType::VersionInfo1 ].origin.y = OriginY + 22;
			GSplashScreenTextRects[ SplashTextType::StartupProgress ].origin.y = OriginY + 3;
			GSplashScreenTextRects[ SplashTextType::CopyrightInfo ].origin.y = OriginY - 2;
		}

		MainThreadCall(^{
			FScopeLock Lock(&GSplashMutex);

			// Create bordeless window with size from NSImage
			GSplashWindow = [[FSplashWindow alloc] initWithContentRect: ContentRect styleMask: 0 backing: NSBackingStoreBuffered defer: NO];
			[GSplashWindow setContentView: [[UnrealSplashView alloc] initWithFrame: ContentRect]];

			if( GSplashWindow )
			{
				// Show window
				[GSplashWindow setHasShadow:YES];
				[GSplashWindow center];
				[GSplashWindow orderFront: nil];
				[NSApp activateIgnoringOtherApps:YES];
			}
			else
			{
				[GSplashScreenImage release];
				GSplashScreenImage = NULL;
			}
		});

		FMacPlatformApplicationMisc::PumpMessages(true);
	}
}

void FMacPlatformSplash::Hide()
{
	if (GSplashWindow)
	{
		MainThreadCall(^{
			SCOPED_AUTORELEASE_POOL;
			FScopeLock Lock(&GSplashMutex);

			[GSplashWindow close];
			GSplashWindow = NULL;

			[GSplashScreenImage release];
			GSplashScreenImage = NULL;
		});

		FMacPlatformApplicationMisc::PumpMessages(true);
	}
}

bool FMacPlatformSplash::IsShown()
{
	return (GSplashWindow != NULL);
}

void FMacPlatformSplash::SetSplashText(const SplashTextType::Type InType, const FText& InText)
{
	// We only want to bother drawing startup progress in the editor, since this information is
	// not interesting to an end-user (also, it's not usually localized properly.)
	if( GSplashWindow )
	{
		// Only allow copyright text displayed while loading the game.  Editor displays all.
		if( InType == SplashTextType::CopyrightInfo || GIsEditor )
		{
			bool bWasUpdated = false;
			{
				FScopeLock Lock(&GSplashMutex);

				// Update splash text
				if (!InText.IdenticalTo(GSplashScreenText[InType], ETextIdenticalModeFlags::LexicalCompareInvariants))
				{
					GSplashScreenText[ InType ] = InText;
					bWasUpdated = true;
				}
			}

			if( bWasUpdated )
			{
				dispatch_async(dispatch_get_main_queue(), ^{
					[[GSplashWindow contentView] setNeedsDisplayInRect: GSplashScreenTextRects[InType]];
				});

				FMacPlatformApplicationMisc::PumpMessages(true);
			}
		}
	}
}
