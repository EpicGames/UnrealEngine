// Copyright Epic Games, Inc. All Rights Reserved.

/*=================================================================================
 MacPlatformSurvey.mm: Mac OS X platform hardware-survey classes
 =================================================================================*/

#include "Mac/MacPlatformSurvey.h"
#include "Mac/MacSystemIncludes.h"
#include "Containers/UnrealString.h"
#include "SynthBenchmark.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformProcess.h"
#include "CoreGlobals.h"
#include "GenericPlatform/GenericApplication.h"

#import <IOKit/graphics/IOGraphicsLib.h>
#import <IOKit/ps/IOPowerSources.h>
#import <IOKit/ps/IOPSKeys.h>

bool FMacPlatformSurvey::GetSurveyResults( FHardwareSurveyResults& OutResults, bool bWait )
{
	FMemory::Memset(&OutResults, 0, sizeof(FHardwareSurveyResults));
	WriteFStringToResults(OutResults.Platform, TEXT("Mac"));

	// Get memory
	vm_size_t PageSize;
	host_page_size(mach_host_self(), &PageSize);
	vm_statistics Stats;
	mach_msg_type_number_t StatsSize = HOST_VM_INFO_COUNT;
	host_statistics(mach_host_self(), HOST_VM_INFO, (host_info_t)&Stats, &StatsSize);
	uint64_t FreeMem = (Stats.free_count + Stats.inactive_count) * PageSize;
	uint64_t UsedMem = (Stats.active_count + Stats.wire_count) * PageSize;
	uint64_t TotalPhys = FreeMem + UsedMem;
	OutResults.MemoryMB = uint32(float(static_cast<double>(TotalPhys)/1024.0/1024.0)+ .1f);

	// Identify Display Devices
	FDisplayMetrics DisplayMetrics;
	FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);

	OutResults.DisplayCount = FMath::Min(DisplayMetrics.MonitorInfo.Num(), FHardwareSurveyResults::MaxDisplayCount);

	for (uint32 DisplayIndex = 0; DisplayIndex < OutResults.DisplayCount; ++DisplayIndex)
	{
		FMonitorInfo& Info = DisplayMetrics.MonitorInfo[DisplayIndex];
		OutResults.Displays[DisplayIndex].CurrentModeHeight = Info.NativeHeight;
		OutResults.Displays[DisplayIndex].CurrentModeWidth = Info.NativeWidth;
	}

	// Get CPU count
	int32 NumCPUs = 1;
	size_t Size = sizeof(int32);
	if (sysctlbyname("hw.ncpu", &NumCPUs, &Size, NULL, 0) == 0)
	{
		OutResults.CPUCount = NumCPUs;
	}
	else
	{
		OutResults.CPUCount = 0;
	}

	ISynthBenchmark::Get().Run(OutResults.SynthBenchmark, true, 5.f);

	FString RHIName;
	ISynthBenchmark::Get().GetRHIInfo(OutResults.RHIAdapter, RHIName);
	WriteFStringToResults(OutResults.RenderingAPI, RHIName);

	// Get CPU speed
	if (OutResults.CPUCount > 0)
	{
		int64 CPUSpeed = 0;
		Size = sizeof(int64);
		if (sysctlbyname("hw.cpufrequency", &CPUSpeed, &Size, NULL, 0) == 0)
		{
			OutResults.CPUClockGHz = (float)(0.000000001 * static_cast<double>(CPUSpeed));
		}
		else
		{
			OutResults.ErrorCount++;
			WriteFStringToResults(OutResults.LastSurveyError, TEXT("FMacPlatformSurvey::TickSurveyHardware() failed to get processor speed from sysctlbyname()"));
			WriteFStringToResults(OutResults.LastSurveyErrorDetail, TEXT(""));
		}
	}
	else
	{
		OutResults.ErrorCount++;
		WriteFStringToResults(OutResults.LastSurveyError, TEXT("FMacPlatformSurvey::TickSurveyHardware() failed to get processor count from sysctlbyname()"));
		WriteFStringToResults(OutResults.LastSurveyErrorDetail, TEXT(""));
	}

	// Get CPU brand
	FString CPUBrand = FMacPlatformMisc::GetCPUVendor();
	WriteFStringToResults(OutResults.CPUBrand, CPUBrand);
	if (CPUBrand.Len() == 0)
	{
		OutResults.ErrorCount++;
		WriteFStringToResults(OutResults.LastSurveyError, TEXT("FMacPlatformSurvey::TickSurveyHardware() failed to get processor brand from FMacPlatformMisc::GetCPUVendor()"));
		WriteFStringToResults(OutResults.LastSurveyErrorDetail, TEXT(""));
	}

	// Get CPU name
	ANSICHAR TempANSICHARBuffer[FHardwareSurveyResults::MaxStringLength];
	Size = sizeof(TempANSICHARBuffer);
	if (sysctlbyname("machdep.cpu.brand_string", TempANSICHARBuffer, &Size, NULL, 0) == 0)
	{
		WriteFStringToResults(OutResults.CPUNameString, FString(StringCast<TCHAR>(&TempANSICHARBuffer[0]).Get()));
	}
	else
	{
		OutResults.ErrorCount++;
		WriteFStringToResults(OutResults.LastSurveyError, TEXT("FMacPlatformSurvey::TickSurveyHardware() failed to get processor name from sysctlbyname()"));
		WriteFStringToResults(OutResults.LastSurveyErrorDetail, TEXT(""));
	}

	// Get CPU info
	OutResults.CPUInfo = FMacPlatformMisc::GetCPUInfo();

	// get HDD details
	OutResults.HardDriveGB = -1;
	OutResults.HardDriveFreeMB = -1;

	uint64 TotalDiskSpace = 0;
	uint64 FreeDiskSpace = 0;
	if (FPlatformMisc::GetDiskTotalAndFreeSpace(FPlatformProcess::BaseDir(), TotalDiskSpace, FreeDiskSpace))
	{
		OutResults.HardDriveGB = (uint32)(TotalDiskSpace / uint64(1024 * 1024 * 1024));
		OutResults.HardDriveFreeMB = (uint32)(FreeDiskSpace / uint64(1024 * 1024));
	}
	else
	{
		UE_LOGF(LogMac, Warning, "FMacPlatformSurvey::TickSurveyHardware() failed to get root-folder drive size" );
		OutResults.ErrorCount++;
		WriteFStringToResults(OutResults.LastSurveyError, TEXT("attributesOfFileSystemForPath failed"));
		WriteFStringToResults(OutResults.LastSurveyErrorDetail, TEXT(""));
	}

	// OS info
	FString OSXVersion, OSXBuild;
	FMacPlatformMisc::GetOSVersions(OSXVersion, OSXBuild);
	WriteFStringToResults(OutResults.OSVersion, FString("Mac OS X ") + OSXVersion);
	WriteFStringToResults(OutResults.OSSubVersion, OSXBuild);
	OutResults.OSBits = 64;

	// OS language
	NSArray* Languages = [[NSUserDefaults standardUserDefaults] objectForKey: @"AppleLanguages"];
	NSString* PreferredLang = [Languages objectAtIndex: 0];
	FPlatformString::CFStringToTCHAR((CFStringRef)PreferredLang, OutResults.OSLanguage);

	// Get system power info to determine whether we're running on a laptop or desktop computer
	OutResults.bIsLaptopComputer = false;
	CFTypeRef PowerSourcesInfo = IOPSCopyPowerSourcesInfo();
	if (PowerSourcesInfo)
	{
		CFArrayRef PowerSourcesArray = IOPSCopyPowerSourcesList(PowerSourcesInfo);
		for (CFIndex Index = 0; Index < CFArrayGetCount(PowerSourcesArray); Index++)
		{
			CFTypeRef PowerSource = CFArrayGetValueAtIndex(PowerSourcesArray, Index);
			NSDictionary* Description = (NSDictionary*)IOPSGetPowerSourceDescription(PowerSourcesInfo, PowerSource);
			if ([(NSString*)[Description objectForKey: @kIOPSTypeKey] isEqualToString: @kIOPSInternalBatteryType])
			{
				OutResults.bIsLaptopComputer = true;
				break;
			}
		}
		CFRelease(PowerSourcesArray);
		CFRelease(PowerSourcesInfo);
	}
	else
	{
		UE_LOGF(LogMac, Warning, "FMacPlatformSurvey::TickSurveyHardware() failed to get system power sources info. Assuming desktop Mac." );
		OutResults.ErrorCount++;
		WriteFStringToResults(OutResults.LastSurveyError, TEXT("IOPSCopyPowerSourcesInfo() failed to get system power sources info"));
		WriteFStringToResults(OutResults.LastSurveyErrorDetail, TEXT(""));
	}

	return true;
}

void FMacPlatformSurvey::WriteFStringToResults(TCHAR* OutBuffer, const FString& InString)
{
	FMemory::Memset( OutBuffer, 0, sizeof(TCHAR) * FHardwareSurveyResults::MaxStringLength );
	TCHAR* Cursor = OutBuffer;
	for (int32 i = 0; i < FMath::Min(InString.Len(), FHardwareSurveyResults::MaxStringLength - 1); i++)
	{
		*Cursor++ = InString[i];
	}
}
