// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/MacProgramDelegate.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "CoreMinimal.h"
#include "HAL/ExceptionHandling.h"
#include "Misc/CommandLine.h"
#include "Mac/CocoaThread.h"
#include "Mac/MacSystemIncludes.h"

@implementation MacProgramDelegate

-(id)initWithProgramMain:(ProgramMainType)programMain programExit:(ProgramExitType)programExit
{
	[self init];
	ProgramMain = programMain;
	ProgramExit = programExit;
	return self;
}

//handler for the quit apple event used by the Dock menu
- (void)handleQuitEvent:(NSAppleEventDescriptor*)Event withReplyEvent:(NSAppleEventDescriptor*)ReplyEvent
{
	[self requestQuit:self];
}

- (IBAction)requestQuit:(id)Sender
{
	RequestEngineExit(TEXT("requestQuit"));
}

- (void)runGameThread:(id)Arg
{
	FPlatformMisc::SetGracefulTerminationHandler();
	FPlatformMisc::SetCrashHandler(nullptr);
	
#if !UE_BUILD_SHIPPING
	if (FParse::Param(*SavedCommandLine,TEXT("crashreports")))
	{
		GAlwaysReportCrash = true;
	}
#endif
		
#if UE_BUILD_DEBUG
	if (!GAlwaysReportCrash)
#else
		if (FPlatformMisc::IsDebuggerPresent() && !GAlwaysReportCrash)
#endif
		{
			ProgramMain(*SavedCommandLine);
		}
		else
		{
			GIsGuarded = 1;
			ProgramMain(*SavedCommandLine);
			GIsGuarded = 0;
		}

	ProgramExit();
    
    dispatch_async(dispatch_get_main_queue(), ^{
        [NSApp terminate: nil];
    });
}

- (void) applicationWillTerminate:(NSNotification*)notification
{
	FTaskTagScope::SetTagStaticInit();
}

- (NSApplicationTerminateReply)applicationShouldTerminate:(NSApplication *)Sender;
{
	if(!IsEngineExitRequested() || ([NSThread gameThread] && [NSThread gameThread] != [NSThread mainThread]))
	{
		[self requestQuit:self];
		return NSTerminateLater;
	}
	else
	{
		return NSTerminateNow;
	}
}

- (void)applicationDidFinishLaunching:(NSNotification *)Notification
{
	//install the custom quit event handler
	NSAppleEventManager* appleEventManager = [NSAppleEventManager sharedAppleEventManager];
	[appleEventManager setEventHandler:self andSelector:@selector(handleQuitEvent:withReplyEvent:) forEventClass:kCoreEventClass andEventID:kAEQuitApplication];
	
	RunGameThread(self, @selector(runGameThread:));
}



+(int)mainWithArgc:(int)argc argv:(char*[])argv programMain:(ProgramMainType)programMain
	programExit:(ProgramExitType)programExit
{
	MacProgramDelegate* StandardDelegate = [[MacProgramDelegate alloc] initWithProgramMain:programMain programExit:programExit];
	return [MacProgramDelegate mainWithArgc:argc argv:argv existingDelegate:StandardDelegate];
}

+(int)mainWithArgc:(int)argc argv:(char*[])argv existingDelegate:(MacProgramDelegate*)delegate
{
	FTaskTagScope::SetTagNone();

	for (int32 Option = 1; Option < argc; Option++)
	{
		delegate->SavedCommandLine += TEXT(" ");
		FString Argument(ANSI_TO_TCHAR(argv[Option]));
		if (Argument.Contains(TEXT(" ")))
		{
			if (Argument.Contains(TEXT("=")))
			{
				FString ArgName;
				FString ArgValue;
				Argument.Split( TEXT("="), &ArgName, &ArgValue );
				Argument = FString::Printf( TEXT("%s=\"%s\""), *ArgName, *ArgValue );
			}
			else
			{
				Argument = FString::Printf(TEXT("\"%s\""), *Argument);
			}
		}
		delegate->SavedCommandLine += Argument;
	}

	SCOPED_AUTORELEASE_POOL;
	[NSApplication sharedApplication];
	[NSApp setDelegate:delegate];
	[NSApp run];
	return 0;

}

@end

