// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/MacTextInputMethodSystem.h"
#include "Apple/ScopeAutoreleasePool.h"
#include "Mac/MacApplication.h"
#include "Mac/MacWindow.h"
#include "Mac/CocoaTextView.h"
#include "Mac/CocoaThread.h"
#include "Mac/MacApplication.h"

DEFINE_LOG_CATEGORY_STATIC(LogMacTextInputMethodSystem, Log, All);

namespace
{
	class FTextInputMethodChangeNotifier : public ITextInputMethodChangeNotifier
	{
	public:
		FTextInputMethodChangeNotifier(const TSharedRef<ITextInputMethodContext>& InContext)
		:	Context(InContext)
		,	ContextWindowNumber(0)
		{
		}
		
		virtual ~FTextInputMethodChangeNotifier() {}
		
		void SetContextWindowNumber(int32 WindowNumber);
		int32 GetContextWindowNumber() const;
		
		virtual void NotifyLayoutChanged(const ELayoutChangeType ChangeType) override;
		virtual void NotifySelectionChanged() override;
		virtual void NotifyTextChanged(const uint32 BeginIndex, const uint32 OldLength, const uint32 NewLength) override;
		virtual void CancelComposition() override;
		
	private:
		TWeakPtr<ITextInputMethodContext> Context;
		int32 ContextWindowNumber;
	};

	void FTextInputMethodChangeNotifier::SetContextWindowNumber(int32 WindowNumber)
	{
		ContextWindowNumber = WindowNumber;
	}

	int32 FTextInputMethodChangeNotifier::GetContextWindowNumber() const
	{
		return ContextWindowNumber;
	}

	void FTextInputMethodChangeNotifier::NotifyLayoutChanged(const ELayoutChangeType ChangeType)
	{
		if(ChangeType != ELayoutChangeType::Created && ContextWindowNumber != 0)
		{
			FCocoaWindow* CocoaWindow = (FCocoaWindow*)[NSApp windowWithWindowNumber:ContextWindowNumber];
			if (CocoaWindow)
			{
				MacApplication->InvalidateTextLayout( CocoaWindow );
			}
		}
	}
	
	void FTextInputMethodChangeNotifier::NotifySelectionChanged()
	{
		if(ContextWindowNumber != 0)
		{
			FCocoaWindow* CocoaWindow = (FCocoaWindow*)[NSApp windowWithWindowNumber:ContextWindowNumber];
			MainThreadCall(^{
				SCOPED_AUTORELEASE_POOL;
				if(CocoaWindow && [CocoaWindow openGLView])
				{
					FCocoaTextView* TextView = (FCocoaTextView*)[CocoaWindow openGLView];
					[[TextView inputContext] invalidateCharacterCoordinates];
				}
			}, true, UnrealIMEEventMode);
		}
	}
	
	void FTextInputMethodChangeNotifier::NotifyTextChanged(const uint32 BeginIndex, const uint32 OldLength, const uint32 NewLength)
	{
		if(ContextWindowNumber != 0)
		{
			FCocoaWindow* CocoaWindow = (FCocoaWindow*)[NSApp windowWithWindowNumber:ContextWindowNumber];
			MainThreadCall(^{
				SCOPED_AUTORELEASE_POOL;
				if(CocoaWindow && [CocoaWindow openGLView])
				{
					FCocoaTextView* TextView = (FCocoaTextView*)[CocoaWindow openGLView];
					[[TextView inputContext] invalidateCharacterCoordinates];
				}
			}, true, UnrealIMEEventMode);
		}
	}
	
	void FTextInputMethodChangeNotifier::CancelComposition()
	{
		if(ContextWindowNumber != 0)
		{
			FCocoaWindow* CocoaWindow = (FCocoaWindow*)[NSApp windowWithWindowNumber:ContextWindowNumber];
			MainThreadCall(^{
				SCOPED_AUTORELEASE_POOL;
				if(CocoaWindow && [CocoaWindow openGLView])
				{
					FCocoaTextView* TextView = (FCocoaTextView*)[CocoaWindow openGLView];
					[[TextView inputContext] discardMarkedText];
					[TextView unmarkText];
				}
			}, true, UnrealIMEEventMode);
		}
	}
}

bool FMacTextInputMethodSystem::Initialize()
{
	// Nothing to do here for Cocoa
	return true;
}

void FMacTextInputMethodSystem::Terminate()
{
	// Nothing to do here for Cocoa
}

// ITextInputMethodSystem Interface Begin
void FMacTextInputMethodSystem::ApplyDefaults(const TSharedRef<FGenericWindow>& InWindow)
{
}

TSharedPtr<ITextInputMethodChangeNotifier> FMacTextInputMethodSystem::RegisterContext(const TSharedRef<ITextInputMethodContext>& Context)
{
	TSharedRef<ITextInputMethodChangeNotifier> Notifier = MakeShareable( new FTextInputMethodChangeNotifier(Context) );
	TWeakPtr<ITextInputMethodChangeNotifier>& NotifierRef = ContextMap.Add(Context);
	NotifierRef = Notifier;
	return Notifier;
}

void FMacTextInputMethodSystem::UnregisterContext(const TSharedRef<ITextInputMethodContext>& Context)
{
	SCOPED_AUTORELEASE_POOL;
	TWeakPtr<ITextInputMethodChangeNotifier>& NotifierRef = ContextMap[Context];
	if(!NotifierRef.IsValid())
	{
		UE_LOGF(LogMacTextInputMethodSystem, Error, "Unregistering a context failed when its registration couldn't be found.");
		return;
	}
	
	TSharedPtr<ITextInputMethodChangeNotifier> Notifier(NotifierRef.Pin());
	FTextInputMethodChangeNotifier* MacNotifier = (FTextInputMethodChangeNotifier*)Notifier.Get();
	if(MacNotifier->GetContextWindowNumber() != 0)
	{
		DeactivateContext(Context);
	}
	
	ContextMap.Remove(Context);
}

void FMacTextInputMethodSystem::ActivateContext(const TSharedRef<ITextInputMethodContext>& Context)
{
	SCOPED_AUTORELEASE_POOL;
	TWeakPtr<ITextInputMethodChangeNotifier>& NotifierRef = ContextMap[Context];
	if(!NotifierRef.IsValid())
	{
		UE_LOGF(LogMacTextInputMethodSystem, Error, "Activating a context failed when its registration couldn't be found.");
		return;
	}
	
	TSharedPtr<ITextInputMethodChangeNotifier> Notifier(NotifierRef.Pin());
	FTextInputMethodChangeNotifier* MacNotifier = (FTextInputMethodChangeNotifier*)Notifier.Get();
	TSharedPtr<FGenericWindow> GenericWindow = Context->GetWindow();
	bool bActivatedContext = false;
	if(GenericWindow.IsValid())
	{
		FCocoaWindow* CocoaWindow = (FCocoaWindow*)GenericWindow->GetOSWindowHandle();
		MacNotifier->SetContextWindowNumber([CocoaWindow windowNumber]);
		bActivatedContext = MainThreadReturn(^{
			bool bSuccess = false;
			if(CocoaWindow && [CocoaWindow openGLView])
			{
				NSView* GLView = [CocoaWindow openGLView];
				if([GLView isKindOfClass:[FCocoaTextView class]])
				{
					FCocoaTextView* TextView = (FCocoaTextView*)GLView;
					[TextView activateInputMethod:Context];
					bSuccess = true;
				}
			}
			return bSuccess;
		}, UnrealIMEEventMode);
	}
	if(!bActivatedContext)
	{
		UE_LOGF(LogMacTextInputMethodSystem, Error, "Activating a context failed when its window couldn't be found.");
	}
}

void FMacTextInputMethodSystem::DeactivateContext(const TSharedRef<ITextInputMethodContext>& Context)
{
	SCOPED_AUTORELEASE_POOL;
	TWeakPtr<ITextInputMethodChangeNotifier>& NotifierRef = ContextMap[Context];
	if(!NotifierRef.IsValid())
	{
		UE_LOGF(LogMacTextInputMethodSystem, Error, "Deactivating a context failed when its registration couldn't be found.");
		return;
	}

	TSharedPtr<ITextInputMethodChangeNotifier> Notifier(NotifierRef.Pin());
	FTextInputMethodChangeNotifier* MacNotifier = (FTextInputMethodChangeNotifier*)Notifier.Get();
	bool bDeactivatedContext = false;
	const int32 ContextWindowNumber = MacNotifier->GetContextWindowNumber();
	if (ContextWindowNumber != 0)
	{
		bDeactivatedContext = MainThreadReturn(^{
			FCocoaWindow* CocoaWindow = (FCocoaWindow*)[NSApp windowWithWindowNumber:ContextWindowNumber];
			if (CocoaWindow)
			{
				bool bSuccess = false;
				if(CocoaWindow && [CocoaWindow openGLView])
				{
					NSView* GLView = [CocoaWindow openGLView];
					if([GLView isKindOfClass:[FCocoaTextView class]])
					{
						FCocoaTextView* TextView = (FCocoaTextView*)GLView;
						[TextView deactivateInputMethod];
						bSuccess = true;
					}
				}
				return bSuccess;
			}
			else
			{
				// If the window is no longer open, the context is already inactive (handled by MessageHandler->OnWindowActivationChanged() called in FMacApplication::OnWindowDestroyed())
				return true;
			}
		}, UnrealIMEEventMode);
	}
	if(!bDeactivatedContext)
	{
		UE_LOGF(LogMacTextInputMethodSystem, Error, "Deactivating a context failed when its window couldn't be found.");
	}
}

bool FMacTextInputMethodSystem::IsActiveContext(const TSharedRef<ITextInputMethodContext>& Context) const
{
	SCOPED_AUTORELEASE_POOL;
	const TWeakPtr<ITextInputMethodChangeNotifier>& NotifierRef = ContextMap[Context];
	if(!NotifierRef.IsValid())
	{
		UE_LOGF(LogMacTextInputMethodSystem, Error, "Checking for an active context failed when its registration couldn't be found.");
		return false;
	}
	
	TSharedPtr<ITextInputMethodChangeNotifier> Notifier(NotifierRef.Pin());
	FTextInputMethodChangeNotifier* MacNotifier = (FTextInputMethodChangeNotifier*)Notifier.Get();
	if(MacNotifier->GetContextWindowNumber() != 0)
	{
		FCocoaWindow* CocoaWindow = (FCocoaWindow*)[NSApp windowWithWindowNumber:MacNotifier->GetContextWindowNumber()];
		if(CocoaWindow && [CocoaWindow openGLView])
		{
			NSView* GLView = [CocoaWindow openGLView];
			if([GLView isKindOfClass:[FCocoaTextView class]])
			{
				FCocoaTextView* TextView = (FCocoaTextView*)GLView;
				return [TextView isActiveInputMethod:Context];
			}
		}
	}
	return false;
}
// ITextInputMethodSystem Interface End
