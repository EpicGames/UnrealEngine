// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/MacWindow.h"

#include "Apple/ScopeAutoreleasePool.h"
#include "Apple/AppleStringUtils.h"
#include "Mac/MacApplication.h"
#include "Mac/MacCursor.h"
#include "Mac/CocoaTextView.h"
#include "Mac/CocoaThread.h"
#include "Mac/MacPlatformApplicationMisc.h"
#include "Mac/MacPlatformMiscEx.h"

#include "HAL/PlatformProcess.h"

FMacWindow::FMacWindow()
:	WindowHandle(nullptr)
,	DisplayID(kCGNullDirectDisplay)
,   CachedOpacity(1.0f)
,	bIsVisible(false)
,	bIsClosed(false)
,	bIsFirstTimeVisible(true)
,   bIsMainEditorWindow(false)
{
}

FMacWindow::~FMacWindow()
{
}

TSharedRef<FMacWindow> FMacWindow::Make()
{
	// First, allocate the new native window object.  This doesn't actually create a native window or anything,
	// we're simply instantiating the object so that we can keep shared references to it.
	return MakeShareable( new FMacWindow() );
}

void FMacWindow::Initialize( FMacApplication* const Application, const TSharedRef< FGenericWindowDefinition >& InDefinition, const TSharedPtr< FMacWindow >& InParent, const bool bShowImmediately )
{
	SCOPED_AUTORELEASE_POOL;

	OwningApplication = Application;
	Definition = InDefinition;
    
    // The main Editor window, like Batman, doesn't have a valid parent
    bIsMainEditorWindow = !InParent.IsValid();

	// Finally, let's initialize the new native window object.  Calling this function will often cause OS
	// window messages to be sent! (such as activation messages)

	FMacScreenRef TargetScreen = FMacApplication::FindScreenBySlatePosition(Definition->XDesiredPositionOnScreen, Definition->YDesiredPositionOnScreen);

	const int32 SizeX = FMath::Max(FMath::CeilToInt( Definition->WidthDesiredOnScreen ), 1);
	const int32 SizeY = FMath::Max(FMath::CeilToInt( Definition->HeightDesiredOnScreen ), 1);

	PositionX = FMath::TruncToInt(Definition->XDesiredPositionOnScreen);
	PositionY = FMath::TruncToInt(Definition->YDesiredPositionOnScreen >= TargetScreen->VisibleFramePixels.origin.y ? Definition->YDesiredPositionOnScreen : TargetScreen->VisibleFramePixels.origin.y);

	const double ScreenDPIScaleFactor = FPlatformApplicationMisc::IsHighDPIModeEnabled() ? TargetScreen->Screen.backingScaleFactor : 1.0;
	const FVector2D CocoaPosition = FMacApplication::ConvertSlatePositionToCocoa(PositionX, PositionY);
	const NSRect ViewRect = NSMakeRect(CocoaPosition.X, CocoaPosition.Y - (SizeY / ScreenDPIScaleFactor) + 1, SizeX / ScreenDPIScaleFactor, SizeY / ScreenDPIScaleFactor);
	uint32 WindowStyle = 0;

	if( Definition->IsRegularWindow )
	{
		if( Definition->HasCloseButton )
		{
			WindowStyle = NSWindowStyleMaskClosable;
		}

		// In order to support rounded, shadowed windows set the window to be titled - we'll set the content view to cover the whole window
		WindowStyle |= NSWindowStyleMaskTitled | NSWindowStyleMaskFullSizeContentView;
		
		if( Definition->SupportsMinimize )
		{
			WindowStyle |= NSWindowStyleMaskMiniaturizable;
		}
		if( Definition->SupportsMaximize || Definition->HasSizingFrame )
		{
			WindowStyle |= NSWindowStyleMaskResizable;
		}
	}
	else
	{
		WindowStyle = NSWindowStyleMaskBorderless;
	}

	if( Definition->HasOSWindowBorder )
	{
		WindowStyle |= NSWindowStyleMaskTitled;
		WindowStyle &= ~NSWindowStyleMaskFullSizeContentView;
	}

	MainThreadCall(^{
		SCOPED_AUTORELEASE_POOL;
		WindowHandle = [[FCocoaWindow alloc] initWithContentRect: ViewRect styleMask: WindowStyle backing: NSBackingStoreBuffered defer: NO];
		WindowHandle.Type = Definition->Type;
		
		if( WindowHandle != nullptr )
		{
			[WindowHandle setReleasedWhenClosed:NO];
			[WindowHandle setWindowMode: EWindowMode::Windowed];
			[WindowHandle setAcceptsInput: Definition->AcceptsInput];
			[WindowHandle setAcceptsMouseMovedEvents: YES];
			[WindowHandle setDelegate: WindowHandle];

			int32 WindowLevel = NSNormalWindowLevel;

			if (Definition->IsModalWindow)
			{
				WindowLevel = NSFloatingWindowLevel;
			}
			else
			{
				switch (Definition->Type)
				{
					case EWindowType::Normal:
						WindowLevel = NSNormalWindowLevel;
						break;

					case EWindowType::Menu:
						WindowLevel = NSStatusWindowLevel;
						break;

					case EWindowType::ToolTip:
						WindowLevel = NSPopUpMenuWindowLevel;
						break;

					case EWindowType::Notification:
						WindowLevel = NSModalPanelWindowLevel;
						break;

					case EWindowType::CursorDecorator:
						WindowLevel = NSMainMenuWindowLevel;
						break;
				}
			}

			[WindowHandle setLevel:WindowLevel];

			WindowedModeSavedState.WindowLevel = WindowLevel;

			if( !Definition->HasOSWindowBorder )
			{
				[WindowHandle setBackgroundColor: [NSColor clearColor]];
				[WindowHandle setHasShadow: YES];
			}

			[WindowHandle setOpaque: NO];

			[WindowHandle setMinSize:NSMakeSize(Definition->SizeLimits.GetMinWidth().Get(10.0f), Definition->SizeLimits.GetMinHeight().Get(10.0f))];
			[WindowHandle setMaxSize:NSMakeSize(Definition->SizeLimits.GetMaxWidth().Get(10000.0f), Definition->SizeLimits.GetMaxHeight().Get(10000.0f))];

			ReshapeWindow(PositionX, PositionY, SizeX, SizeY);

			if (Definition->ShouldPreserveAspectRatio)
			{
				[WindowHandle setContentAspectRatio:NSMakeSize((float)SizeX / (float)SizeY, 1.0f)];
			}

			if (Definition->IsRegularWindow)
			{
				[NSApp addWindowsItem:WindowHandle title:Definition->Title.GetNSString() filename:NO];

				// Tell Cocoa that we are opting into drag and drop.
				// Only makes sense for regular windows (windows that last a while.)
				[WindowHandle registerForDraggedTypes:@[NSPasteboardTypeFileURL, NSPasteboardTypeString]];

				if( Definition->HasOSWindowBorder )
				{
					[WindowHandle setCollectionBehavior: NSWindowCollectionBehaviorFullScreenPrimary|NSWindowCollectionBehaviorDefault|NSWindowCollectionBehaviorManaged|NSWindowCollectionBehaviorParticipatesInCycle];

					// By default NSWindowStyleMaskResizable enables zoom button as well, so we need to disable it if that's what the project requests
					if (Definition->HasSizingFrame && !Definition->SupportsMaximize)
					{
						[[WindowHandle standardWindowButton:NSWindowZoomButton] setEnabled:NO];
					}
				}
				else
				{
					[WindowHandle setCollectionBehavior: NSWindowCollectionBehaviorFullScreenAuxiliary|NSWindowCollectionBehaviorDefault|NSWindowCollectionBehaviorManaged|NSWindowCollectionBehaviorParticipatesInCycle];

					WindowHandle.titlebarAppearsTransparent = YES;
					WindowHandle.titleVisibility = NSWindowTitleHidden;
				}

				SetText(*Definition->Title);
			}
			else if(Definition->AppearsInTaskbar)
			{
				if (!Definition->Title.IsEmpty())
				{
					[NSApp addWindowsItem:WindowHandle title:Definition->Title.GetNSString() filename:NO];
				}

				[WindowHandle setCollectionBehavior:NSWindowCollectionBehaviorFullScreenAuxiliary|NSWindowCollectionBehaviorDefault|NSWindowCollectionBehaviorManaged|NSWindowCollectionBehaviorParticipatesInCycle];
			}
			else
			{
				[WindowHandle setCollectionBehavior:NSWindowCollectionBehaviorCanJoinAllSpaces|NSWindowCollectionBehaviorTransient|NSWindowCollectionBehaviorIgnoresCycle];
			}

			if (Definition->TransparencySupport == EWindowTransparency::PerWindow)
			{
				SetOpacity(Definition->Opacity);
			}
			else
			{
				SetOpacity(1.0f);
			}

			OnWindowDidChangeScreen();
		}
		else
		{
			// @todo Error message should be localized!
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			NSRunInformationalAlertPanel( @"Error", @"Window creation failed!", @"Yes", NULL, NULL );
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			check(0);
		}
	}, true, UnrealShowEventMode);
}

FCocoaWindow* FMacWindow::GetWindowHandle() const
{
	return WindowHandle;
}

void FMacWindow::ReshapeWindow(int32 X, int32 Y, int32 Width, int32 Height)
{
	if (WindowHandle)
	{
		ApplySizeAndModeChanges(X, Y, Width, Height, WindowHandle.TargetWindowMode);
	}
}

bool FMacWindow::GetFullScreenInfo( int32& X, int32& Y, int32& Width, int32& Height ) const
{
	SCOPED_AUTORELEASE_POOL;
	bool const bIsFullscreen = (GetWindowMode() == EWindowMode::Fullscreen);
	const NSRect Frame = WindowHandle.screen.frame;
	const FVector2D SlatePosition = FMacApplication::ConvertCocoaPositionToSlate(Frame.origin.x, Frame.origin.y - Frame.size.height + 1.0f);
	X = FMath::TruncToInt(SlatePosition.X);
	Y = FMath::TruncToInt(SlatePosition.Y);
	const double DPIScaleFactor = FPlatformApplicationMisc::IsHighDPIModeEnabled() ? WindowHandle.screen.backingScaleFactor : 1.0;
	Width = FMath::TruncToInt(Frame.size.width * DPIScaleFactor);
	Height = FMath::TruncToInt(Frame.size.height * DPIScaleFactor);
	return true;
}

void FMacWindow::MoveWindowTo( int32 X, int32 Y )
{
	MainThreadCall(^{
		SCOPED_AUTORELEASE_POOL;
		const FVector2D Point = FMacApplication::ConvertSlatePositionToCocoa(X, Y);
		[WindowHandle setFrameOrigin:NSMakePoint(Point.X, Point.Y - [WindowHandle openGLFrame].size.height + 1)];
	}, true, UnrealResizeEventMode);
}

void FMacWindow::BringToFront( bool bForce )
{
	bIsVisible = (bIsVisible || bForce);
    
	if (!bIsClosed && bIsVisible && !bIsMainEditorWindow) // Don't try to bring editor window to front
	{
		SCOPED_AUTORELEASE_POOL;

		bool bOrderAndKey = IsRegularWindow() || bForce;

		FCocoaWindow* WindowHandleCopy = WindowHandle;
		MainThreadCall(^{
			SCOPED_AUTORELEASE_POOL;
			[WindowHandleCopy orderFrontAndMakeMain:bOrderAndKey andKey:bOrderAndKey];
		}, true, UnrealShowEventMode);

		MacApplication->OnWindowOrderedFront(SharedThis(this));
	}
}

void FMacWindow::Destroy()
{
	if (WindowHandle)
	{
		SCOPED_AUTORELEASE_POOL;
		bIsClosed = true;

		FCocoaWindow* WindowHandleCopy = WindowHandle;
		MainThreadCall(^{
			SCOPED_AUTORELEASE_POOL;
			[WindowHandleCopy setAlphaValue:0.0f];
			[WindowHandleCopy setBackgroundColor:[NSColor clearColor]];
		}, false);

		MacApplication->OnWindowDestroyed(SharedThis(this));
		WindowHandle = nullptr;
	}
}

void FMacWindow::Minimize()
{
	MainThreadCall(^{
		SCOPED_AUTORELEASE_POOL;
		[WindowHandle miniaturize:nil];
	}, true, UnrealResizeEventMode);
}

void FMacWindow::Maximize()
{
	MainThreadCall(^{
		SCOPED_AUTORELEASE_POOL;
		if (!WindowHandle.zoomed)
		{
			WindowHandle->bZoomed = true;
			[WindowHandle zoom:nil];
		}
	}, true, UnrealResizeEventMode);
}

void FMacWindow::Restore()
{
	MainThreadCall(^{
		SCOPED_AUTORELEASE_POOL;
		if (WindowHandle.miniaturized)
		{
			[WindowHandle deminiaturize:nil];
		}
		else if (WindowHandle.zoomed)
		{
			[WindowHandle zoom:nil];
		}
	}, true, UnrealResizeEventMode);

	WindowHandle->bZoomed = WindowHandle.zoomed;
}

void FMacWindow::Show()
{
	if (!bIsClosed && !bIsVisible)
	{
		// Should the show command include activation?
		// Do not activate windows that do not take input; e.g. tool-tips and cursor decorators
		bool bShouldActivate = false;
		if (Definition->AcceptsInput)
		{
			bShouldActivate = Definition->ActivationPolicy == EWindowActivationPolicy::Always;
			if (bIsFirstTimeVisible && Definition->ActivationPolicy == EWindowActivationPolicy::FirstShown)
			{
				bShouldActivate = true;
			}
		}

		bIsFirstTimeVisible = false;

		if (bShouldActivate)
		{
			// Tell MacApplication to send window deactivate and activate messages to Slate without waiting for Cocoa events.
			MacApplication->OnWindowActivated(SharedThis(this));
		}
		else
		{
			MacApplication->OnWindowOrderedFront(SharedThis(this));
		}

		FCocoaWindow* WindowHandleCopy = WindowHandle;
		MainThreadCall(^{
			SCOPED_AUTORELEASE_POOL;
			[WindowHandleCopy orderFrontAndMakeMain:bShouldActivate andKey:bShouldActivate];
		}, true, UnrealShowEventMode);

		bIsVisible = true;
	}
}

void FMacWindow::Hide()
{
	if (bIsVisible)
	{
		bIsVisible = false;

		MainThreadCall(^{
			SCOPED_AUTORELEASE_POOL;
			[WindowHandle orderOut:nil];
		}, true, UnrealCloseEventMode);
	}
}

void FMacWindow::SetWindowMode(EWindowMode::Type NewWindowMode)
{
	if(WindowHandle)
	{
		ApplySizeAndModeChanges(PositionX, PositionY, FMath::TruncToInt(WindowHandle.contentView.frame.size.width),
								FMath::TruncToInt(WindowHandle.contentView.frame.size.height), NewWindowMode);
	}
}

EWindowMode::Type FMacWindow::GetWindowMode() const
{
	if (WindowHandle)
	{
		return [WindowHandle windowMode];
	}
	return EWindowMode::Windowed;
}

bool FMacWindow::IsMaximized() const
{
	return WindowHandle && WindowHandle->bZoomed;
}

bool FMacWindow::IsMinimized() const
{
	SCOPED_AUTORELEASE_POOL;
	// @TODO: UE-227397
	// Currently using cached property to avoid triggering a MainThreadChecker issue.
	return WindowHandle == nullptr || WindowHandle->bMinimized;
}

bool FMacWindow::IsVisible() const
{
	SCOPED_AUTORELEASE_POOL;
	return bIsVisible && [NSApp isHidden] == false;
}

bool FMacWindow::GetRestoredDimensions(int32& X, int32& Y, int32& Width, int32& Height)
{
	if (WindowHandle)
	{
		SCOPED_AUTORELEASE_POOL;

		NSRect Frame = WindowHandle.frame;

		const FVector2D SlatePosition = FMacApplication::ConvertCocoaPositionToSlate(Frame.origin.x, Frame.origin.y);

		const double DPIScaleFactor = FPlatformApplicationMisc::IsHighDPIModeEnabled() ? WindowHandle.backingScaleFactor : 1.0;
		Width = FMath::TruncToInt(Frame.size.width * DPIScaleFactor);
		Height = FMath::TruncToInt(Frame.size.height * DPIScaleFactor);

		X = FMath::TruncToInt(SlatePosition.X);
		Y = FMath::TruncToInt(SlatePosition.Y - Height + 1);

		return true;
	}
	else
	{
		return false;
	}
}

void FMacWindow::SetWindowFocus()
{
    BringToFront(true);
}

void FMacWindow::SetOpacity( const float InOpacity )
{
	MainThreadCall(^{
		SCOPED_AUTORELEASE_POOL;
        CachedOpacity = InOpacity;
		[WindowHandle setAlphaValue:InOpacity];
	}, true, UnrealNilEventMode);
}

bool FMacWindow::IsPointInWindow( int32 X, int32 Y ) const
{
	SCOPED_AUTORELEASE_POOL;

	bool PointInWindow = false;
	// @TODO: UE-227397
	// Currently using cached property to avoid triggering a MainThreadChecker issue.
	if (!WindowHandle->bMinimized)
	{
		NSRect WindowFrame = [WindowHandle frame];
		WindowFrame.size = [WindowHandle openGLFrame].size;
		NSRect VisibleFrame = WindowFrame;
		VisibleFrame.origin = NSMakePoint(0, 0);

		// Only the editor needs to handle the space-per-display logic introduced in Mavericks.
#if WITH_EDITOR
		// Only fetch the spans-displays once - it requires a log-out to change.
		// Note that we have no way to tell if the current setting is actually in effect,
		// so this won't work if the user schedules a change but doesn't logout to confirm it.
		static bool bScreensHaveSeparateSpaces = false;
		static bool bSettingFetched = false;
		if (!bSettingFetched)
		{
			bSettingFetched = true;
			bScreensHaveSeparateSpaces = [NSScreen screensHaveSeparateSpaces];
		}
		if (bScreensHaveSeparateSpaces)
		{
			// @TODO: UE-227397
			// Currently using cached property to avoid triggering a MainThreadChecker issue.
			NSRect ScreenFrame = WindowHandle->CachedScreen.frame;
			NSRect Intersection = NSIntersectionRect(ScreenFrame, WindowFrame);
			VisibleFrame.size = Intersection.size;
			VisibleFrame.origin.x = Intersection.origin.x - WindowFrame.origin.x;
			VisibleFrame.origin.y = Intersection.origin.y - WindowFrame.origin.y;
		}
#endif

		if (WindowHandle->bIsOnActiveSpace)
		{
			const float DPIScaleFactor = GetDPIScaleFactor();
			NSPoint CursorPoint = NSMakePoint(static_cast<float>(X) / DPIScaleFactor, WindowFrame.size.height - (static_cast<float>(Y) / DPIScaleFactor + 1));
			PointInWindow = (NSPointInRect(CursorPoint, VisibleFrame) == YES);
		}
	}
	return PointInWindow;
}

int32 FMacWindow::GetWindowBorderSize() const
{
	return 0;
}

bool FMacWindow::IsForegroundWindow() const
{
	// @TODO: UE-227397
	// Currently using cached property to avoid triggering a MainThreadChecker issue.
	if (WindowHandle)
	{
		return WindowHandle->bKeyWindow;
	}
	return false;
}

void FMacWindow::SetText(const TCHAR* const Text)
{
	SCOPED_AUTORELEASE_POOL;
	MainThreadCall(^{
		if (FString(WindowHandle.title) != FString(Text))
		{
			CFStringRef CFName = FPlatformString::TCHARToCFString( Text );

			[WindowHandle setTitle: (NSString*)CFName];
			if (IsRegularWindow())
			{
				[NSApp changeWindowsItem: WindowHandle title: (NSString*)CFName filename: NO];
			}
			CFRelease( CFName );
		}
	}, true, UnrealNilEventMode);
}

bool FMacWindow::IsRegularWindow() const
{
	return Definition->IsRegularWindow;
}

float FMacWindow::GetDPIScaleFactor() const
{
	// @TODO: UE-227397
	// Currently using cached property to avoid triggering a MainThreadChecker issue.
	if (WindowHandle && FPlatformApplicationMisc::IsHighDPIModeEnabled())
	{
		return (float)WindowHandle->CachedBackingScaleFactor;
	}
	return 1.0f;
}

void FMacWindow::SetNativeWindowButtonsVisibility(bool bVisible)
{
	const bool bHidden = !(bVisible && Definition->IsRegularWindow);
	MainThreadCall(^{
		SCOPED_AUTORELEASE_POOL;

		NSButton* CloseButton = [WindowHandle standardWindowButton:NSWindowCloseButton];
		if (CloseButton)
		{
			CloseButton.hidden = bHidden;
		}

		NSButton* MinimizeButton = [WindowHandle standardWindowButton:NSWindowMiniaturizeButton];
		if (MinimizeButton)
		{
			MinimizeButton.hidden = bHidden;
		}

		NSButton* MaximizeButton = [WindowHandle standardWindowButton:NSWindowZoomButton];
		if (MaximizeButton)
		{
			MaximizeButton.hidden = bHidden;
		}
	}, false);
}

void FMacWindow::OnDisplayReconfiguration(CGDirectDisplayID Display, CGDisplayChangeSummaryFlags Flags)
{
	SCOPED_AUTORELEASE_POOL;
	if(WindowHandle)
	{
		if(Flags & kCGDisplayBeginConfigurationFlag)
		{
			[WindowHandle setMovable: YES];
			[WindowHandle setMovableByWindowBackground: NO];
		}
	}
}

void FMacWindow::OnWindowDidChangeScreen()
{
	SCOPED_AUTORELEASE_POOL;
	DisplayID = [[WindowHandle.screen.deviceDescription objectForKey:@"NSScreenNumber"] unsignedIntegerValue];
}

void FMacWindow::ApplySizeAndModeChanges(int32 X, int32 Y, int32 Width, int32 Height, EWindowMode::Type WindowMode)
{
	// It's possible for Window handle to become nil in FMacWindow::Destroy() after a call to UpdateFullScreenState() in this function,
	// in rare cases due to FPlatformApplicationMisc::PumpMessages
	
	SCOPED_AUTORELEASE_POOL;

	// Wait if we're in a middle of fullscreen transition
	WaitForFullScreenTransition();

	bool bIsFullScreen = [WindowHandle windowMode] == EWindowMode::WindowedFullscreen || [WindowHandle windowMode] == EWindowMode::Fullscreen;
	const bool bWantsFullScreen = WindowMode == EWindowMode::WindowedFullscreen || WindowMode == EWindowMode::Fullscreen;

	__block CGDisplayFadeReservationToken FadeReservationToken = 0;

	if ([WindowHandle windowMode] == EWindowMode::Fullscreen || WindowMode == EWindowMode::Fullscreen)
	{
		MainThreadCall(^{
			CGError Error = CGAcquireDisplayFadeReservation(kCGMaxDisplayReservationInterval, &FadeReservationToken);
			if (Error == kCGErrorSuccess)
			{
				CGDisplayFade(FadeReservationToken, 0.3f, kCGDisplayBlendNormal, kCGDisplayBlendSolidColor, 0, 0, 0, true);
			}
		}, true, UnrealResizeEventMode);
	}

	if (WindowMode == EWindowMode::Windowed || WindowMode == EWindowMode::WindowedFullscreen)
	{
		if (WindowedModeSavedState.CapturedDisplayID != kCGNullDirectDisplay)
		{
			MainThreadCall(^{
				CGDisplaySetDisplayMode(WindowedModeSavedState.CapturedDisplayID, WindowedModeSavedState.DesktopDisplayMode, nullptr);
			}, true, UnrealResizeEventMode);

			CGDisplayModeRelease(WindowedModeSavedState.DesktopDisplayMode);
			WindowedModeSavedState.DesktopDisplayMode = nullptr;

			CGDisplayRelease(WindowedModeSavedState.CapturedDisplayID);
			WindowedModeSavedState.CapturedDisplayID = kCGNullDirectDisplay;

			WindowHandle.TargetWindowMode = EWindowMode::Windowed;
			UpdateFullScreenState(true);
			bIsFullScreen = false;
		}
		
		if(WindowHandle)
		{
			WindowHandle.TargetWindowMode = WindowMode;

			const float DPIScaleFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(static_cast<float>(X), static_cast<float>(Y));
			Width = FMath::CeilToInt(static_cast<float>(Width) / DPIScaleFactor);
			Height = FMath::CeilToInt(static_cast<float>(Height) / DPIScaleFactor);

			const FVector2D CocoaPosition = FMacApplication::ConvertSlatePositionToCocoa(X, Y);
			NSRect Rect = NSMakeRect(CocoaPosition.X, CocoaPosition.Y - Height + 1, FMath::Max(Width, 1), FMath::Max(Height, 1));
			if (Definition->HasOSWindowBorder)
			{
				Rect = [WindowHandle frameRectForContentRect:Rect];
			}

			UpdateFullScreenState(bWantsFullScreen != bIsFullScreen);

			if (WindowHandle && WindowMode == EWindowMode::Windowed && !NSEqualRects([WindowHandle frame], Rect))
			{
				MainThreadCall(^{
					SCOPED_AUTORELEASE_POOL;
					if (!bIsFullScreen) // Don't set the frame rect when switching from fullscreen because we don't know the correct window size yet. The OS will resize the window for us.
					{
						[WindowHandle setFrame:Rect display:YES];
					}
                    
                    const float WindowOpacity = (Definition->TransparencySupport == EWindowTransparency::PerWindow) ? CachedOpacity : 1.0f;
					[WindowHandle setAlphaValue:(Width > 0 && Height > 0) ? WindowOpacity : 0.0f];

					if (Definition->ShouldPreserveAspectRatio)
					{
						[WindowHandle setContentAspectRatio:NSMakeSize((float)Width / (float)Height, 1.0f)];
					}
				}, true, UnrealResizeEventMode);
			}
		}
	}
	else
	{
		WindowHandle.TargetWindowMode = WindowMode;

		if (WindowedModeSavedState.CapturedDisplayID == kCGNullDirectDisplay)
		{
			const CGError Result = CGDisplayCapture(DisplayID);
			if (Result == kCGErrorSuccess)
			{
				WindowedModeSavedState.DesktopDisplayMode = CGDisplayCopyDisplayMode(DisplayID);
				WindowedModeSavedState.CapturedDisplayID = DisplayID;
			}
		}

		if (WindowedModeSavedState.CapturedDisplayID != kCGNullDirectDisplay)
		{
			CGDisplayModeRef DisplayMode = FMacPlatformMiscEx::GetSupportedDisplayMode(WindowedModeSavedState.CapturedDisplayID, Width, Height);
			MainThreadCall(^{
				CGDisplaySetDisplayMode(WindowedModeSavedState.CapturedDisplayID, DisplayMode, nullptr);
			}, true, UnrealResizeEventMode);

			UpdateFullScreenState(bIsFullScreen != bWantsFullScreen);

			MacApplication->DeferEvent([NSNotification notificationWithName:NSWindowDidResizeNotification object:WindowHandle]);
		}
	}

	if(WindowHandle)
	{
		// Should be run on the main cocoa thread but apparently it cause more race condition then it fixes.
		//MainThreadCall(^{
		WindowHandle->bZoomed = WindowHandle.zoomed;
		//});
	}

	if (FadeReservationToken != 0)
	{
		MainThreadCall(^{
			CGDisplayFade(FadeReservationToken, 0.5, kCGDisplayBlendNormal, kCGDisplayBlendSolidColor, 0, 0, 0, false);
			CGReleaseDisplayFadeReservation(FadeReservationToken);
		}, false);
	}
}

void FMacWindow::UpdateFullScreenState(bool bToggleFullScreen)
{
	MainThreadCall(^{
		SCOPED_AUTORELEASE_POOL;
		if (bToggleFullScreen)
		{
			[WindowHandle toggleFullScreen:nil];
		}
		else
		{
			[WindowHandle setWindowMode:WindowHandle.TargetWindowMode];
		}

		if (WindowHandle.TargetWindowMode == EWindowMode::Fullscreen)
		{
			if (WindowHandle.level < CGShieldingWindowLevel())
			{
				WindowedModeSavedState.WindowLevel = WindowHandle.level;
				[WindowHandle setLevel:CGShieldingWindowLevel() + 1];
			}
			
			// -toggleFullScreen implicitly sets these, but it doesn't hurt to set them ourselves to match.
			[NSApp setPresentationOptions:NSApplicationPresentationAutoHideDock | NSApplicationPresentationAutoHideMenuBar];
		}
		else if (WindowHandle.level != WindowedModeSavedState.WindowLevel)
		{
			[WindowHandle setLevel:WindowedModeSavedState.WindowLevel];
			[NSApp setPresentationOptions:NSApplicationPresentationDefault];
		}
	}, true, UnrealFullscreenEventMode);

	// If we toggle fullscreen, ensure that the window has transitioned BEFORE leaving this function.
	// This prevents problems with failure to correctly update mouse locks and rendering contexts due to bad event ordering.
	WaitForFullScreenTransition();

	// Restore window size limits if needed
	MacApplication->OnCursorLock();
}

void FMacWindow::WaitForFullScreenTransition()
{
	bool bModeChanged = false;
	do
	{
		FPlatformProcess::Sleep(0.0f);
		FPlatformApplicationMisc::PumpMessages(true);
		bModeChanged = [WindowHandle windowMode] == WindowHandle.TargetWindowMode;
	} while (!bModeChanged);
}
