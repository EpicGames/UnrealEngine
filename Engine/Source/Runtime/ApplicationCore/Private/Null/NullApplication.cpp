// Copyright Epic Games, Inc. All Rights Reserved.

#include "Null/NullApplication.h"

#include "HAL/PlatformTime.h"
#include "Misc/StringUtility.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"
#include "Features/IModularFeatures.h"
#include "Null/NullPlatformApplicationMisc.h"
#include "IInputDeviceModule.h"
#include "IHapticDevice.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"

FNullApplication* NullApplication = NULL;

FNullApplication* FNullApplication::CreateNullApplication()
{
	if (!NullApplication)
	{
		NullApplication = new FNullApplication();
	}

	return NullApplication;
}

void FNullApplication::MoveWindowTo(FGenericWindow* Window, const int32 X, const int32 Y)
{
	if (FNullApplication* Application = CreateNullApplication())
	{
		TSharedPtr<FNullWindow> NullWindow = Application->FindWindowByPtr(Window);
		Application->GetMessageHandler()->OnMovedWindow(NullWindow.ToSharedRef(), X, Y);
	}
}

void FNullApplication::OnSizeChanged(FGenericWindow* Window, const int32 Width, const int32 Height)
{
	if (FNullApplication* Application = CreateNullApplication())
	{
		TSharedPtr<FNullWindow> NullWindow = Application->FindWindowByPtr(Window);
		Application->GetMessageHandler()->OnSizeChanged(NullWindow.ToSharedRef(), Width, Height, false);
	}
}

void FNullApplication::GetFullscreenInfo(int32& X, int32& Y, int32& Width, int32& Height)
{
	if (FNullApplication* Application = CreateNullApplication())
	{
		FPlatformRect WorkArea;
		WorkArea = Application->GetWorkArea(WorkArea);

		X = WorkArea.Left;
		Y = WorkArea.Top;
		Width = WorkArea.Right - WorkArea.Left;
		Height = WorkArea.Bottom - WorkArea.Top;
	}
}

void FNullApplication::ShowWindow(FGenericWindow* Window)
{
	if (FNullApplication* Application = CreateNullApplication())
	{
		TSharedPtr<FNullWindow> NullWindow = Application->FindWindowByPtr(Window);
		Application->ActivateWindow(NullWindow);
	}
}

void FNullApplication::HideWindow(FGenericWindow* Window)
{
	if (FNullApplication* Application = CreateNullApplication())
	{
		TSharedPtr<FNullWindow> NullWindow = Application->FindWindowByPtr(Window);
		// Commented as we currently don't want to hide windows.
		// Application->GetMessageHandler()->OnWindowActivationChanged(NullWindow.ToSharedRef(), EWindowActivation::Deactivate);
	}
}

void FNullApplication::DestroyWindow(FGenericWindow* Window)
{
	if (FNullApplication* Application = CreateNullApplication())
	{
		TSharedPtr<FNullWindow> NullWindow = Application->FindWindowByPtr(Window);
		if (NullWindow)
		{
			Application->DestroyWindow(NullWindow.ToSharedRef());
		}
	}
}

FNullApplication::FNullApplication()
	: GenericApplication(MakeShareable(new FNullCursor()))
	, bHasLoadedInputPlugins(false)
{
	// TODO (william.belcher): There's no way of grabbing a work area from an OS that doesn't exist.
	// Therefore, we have theoretically uncapped minimum and maximum. Let's just say you've got a
	// 4096x4096 square as that's the maximum supported resolution for NvEnc
    WorkArea.Left = 0;
	WorkArea.Top = 0;
	WorkArea.Right = 4096;
	WorkArea.Bottom = 4096;


    // Default work area to the config res
    static const IConsoleVariable* CVarSystemResolution = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SetRes"));
    FString ResolutionStr = CVarSystemResolution->GetString();

    uint32 ResolutionX, ResolutionY = 0;
    if(ParseResolution(*ResolutionStr, ResolutionX, ResolutionY))
    {
        WorkArea.Right = ResolutionX;
        WorkArea.Bottom = ResolutionY;
    }
	
    // Listen for res changes to update our virtual display
    FCoreDelegates::OnSystemResolutionChanged.AddLambda([this](uint32 ResolutionX, uint32 ResolutionY) {
        WorkArea.Right = ResolutionX;
		WorkArea.Bottom = ResolutionY;

        FDisplayMetrics DisplayMetrics;
		FNullPlatformDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);
        BroadcastDisplayMetricsChanged(DisplayMetrics);
    });
}

FNullApplication::~FNullApplication()
{
	IModularFeatures::Get().OnModularFeatureRegistered().RemoveAll(this);
}

void FNullApplication::DestroyApplication()
{
}

TSharedRef<FGenericWindow> FNullApplication::MakeWindow()
{
	return FNullWindow::Make();
}

void FNullApplication::InitializeWindow(const TSharedRef<FGenericWindow>& InWindow,
	const TSharedRef<FGenericWindowDefinition>& InDefinition,
	const TSharedPtr<FGenericWindow>& InParent,
	const bool bShowImmediately)
{
	const TSharedRef<FNullWindow> Window = StaticCastSharedRef<FNullWindow>(InWindow);
	const TSharedPtr<FNullWindow> ParentWindow = StaticCastSharedPtr<FNullWindow>(InParent);

	Window->Initialize(this, InDefinition, ParentWindow, bShowImmediately);
	Windows.Add(Window);
}

void FNullApplication::DestroyWindow(TSharedRef<FNullWindow> WindowToRemove)
{
	Windows.Remove(WindowToRemove);
}

TSharedPtr<FNullWindow> FNullApplication::FindWindowByPtr(FGenericWindow* WindowToFind)
{
	for (int32 WindowIndex = 0; WindowIndex < Windows.Num(); ++WindowIndex)
	{
		TSharedRef<FNullWindow> Window = Windows[WindowIndex];
		if ((&Window.Get()) == static_cast<FNullWindow*>(WindowToFind))
		{
			return Window;
		}
	}

	return TSharedPtr<FNullWindow>(nullptr);
}

void FNullApplication::ActivateWindow(const TSharedPtr<FNullWindow>& Window)
{
	const FGenericWindowDefinition Definition = Window->GetDefinition();
	bool bIsTooltipWindow = (Definition.Type == EWindowType::ToolTip);
	bool bIsNotificationWindow = (Definition.Type == EWindowType::Notification);

	if (bIsTooltipWindow || bIsNotificationWindow)
	{
		// We don't want to force window activation changed with tooltips or notifications
		// as doing so will close any context menus that are open
		return;
	}

	if (CurrentlyActiveWindow != Window)
	{
		PreviousActiveWindow = CurrentlyActiveWindow;
		CurrentlyActiveWindow = Window;
		if (PreviousActiveWindow.IsValid())
		{
			MessageHandler->OnWindowActivationChanged(PreviousActiveWindow.ToSharedRef(), EWindowActivation::Deactivate);
		}
		MessageHandler->OnWindowActivationChanged(CurrentlyActiveWindow.ToSharedRef(), EWindowActivation::Activate);
	}
}

void FNullApplication::SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
{
	GenericApplication::SetMessageHandler(InMessageHandler);
}

void FNullApplication::PumpMessages(const float TimeDelta)
{
}

bool FNullApplication::IsCursorDirectlyOverSlateWindow() const
{
	return true;
}

TSharedPtr<FGenericWindow> FNullApplication::GetWindowUnderCursor()
{
	return nullptr;
}

void FNullApplication::ProcessDeferredEvents(const float TimeDelta)
{
}

void FNullApplication::PollGameDeviceState(const float TimeDelta)
{
	IPlatformInputDeviceMapper& Mapper = IPlatformInputDeviceMapper::Get();
	if (!bHasLoadedInputPlugins)
	{
		// Create all externally-implemented input devices from loaded modules.
		TArray<IInputDeviceModule*> PluginImplementations = IModularFeatures::Get().GetModularFeatureImplementations<IInputDeviceModule>(IInputDeviceModule::GetModularFeatureName());
		for (auto InputPluginIt = PluginImplementations.CreateIterator(); InputPluginIt; ++InputPluginIt)
		{
			TSharedPtr<IInputDevice> Device = (*InputPluginIt)->CreateInputDevice(MessageHandler);
			if (Device.IsValid())
			{
				UE_LOGF(LogInit, Log, "Adding external input plugin.");
				ExternalInputDevices.Add(MoveTemp(Device));
			}
		}

		// Modules loaded after this point are handled by this callback
		IModularFeatures::Get().OnModularFeatureRegistered().AddRaw(this, &FNullApplication::OnInputDeviceModuleRegistered);

		bHasLoadedInputPlugins = true;
	}

	// Poll externally-implemented devices
	for (auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt)
	{
		(*DeviceIt)->Tick(TimeDelta);
		(*DeviceIt)->SendControllerEvents();
	}
}

FModifierKeysState FNullApplication::GetModifierKeys() const
{
	return FModifierKeysState();
}

void FNullApplication::SetCapture(const TSharedPtr<FGenericWindow>& InWindow)
{
}

void* FNullApplication::GetCapture(void) const
{
	return NULL;
}

bool FNullApplication::IsGamepadAttached() const
{
	return false;
}

void FNullApplication::SetHighPrecisionMouseMode(const bool Enable, const TSharedPtr<FGenericWindow>& InWindow)
{
	MessageHandler->OnCursorSet();
	bUsingHighPrecisionMouseInput = Enable;
}

FPlatformRect FNullApplication::GetWorkArea(const FPlatformRect& CurrentWindow) const
{
	return WorkArea;
}

void FNullApplication::SetWorkArea(const FPlatformRect& NewWorkArea)
{
	WorkArea = NewWorkArea;
}

bool FNullApplication::Exec_Dev(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	// Ignore any execs that doesn't start with NullApp
	if (!FParse::Command(&Cmd, TEXT("NullApp")))
	{
		return false;
	}

	if (FParse::Command(&Cmd, TEXT("Cursor")))
	{
		return HandleCursorCommand(Cmd, Ar);
	}
	else if (FParse::Command(&Cmd, TEXT("Window")))
	{
		return HandleWindowCommand(Cmd, Ar);
	}

	return false;
}

bool FNullApplication::HandleCursorCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("Status")))
	{
		FNullCursor* NullCursor = static_cast<FNullCursor*>(Cursor.Get());
		FVector2D CurrentPosition = NullCursor->GetPosition();

		Ar.Logf(TEXT("Cursor status:"));
		Ar.Logf(TEXT("Position: (%f, %f)"), CurrentPosition.X, CurrentPosition.Y);
		return true;
	}

	return false;
}

bool FNullApplication::HandleWindowCommand(const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("List")))
	{
		Ar.Logf(TEXT("Window list:"));
		for (int WindowIdx = 0, NumWindows = Windows.Num(); WindowIdx < NumWindows; ++WindowIdx)
		{
			Ar.Logf(TEXT("%d"), WindowIdx);
		}

		return true;
	}

	return false;
}

void FNullApplication::SetForceFeedbackChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value)
{
	if (FApp::UseVRFocus() && !FApp::HasVRFocus())
	{
		return; // do not proceed if the app uses VR focus but doesn't have it
	}
	// send vibration to externally-implemented devices
	for (auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt)
	{
		(*DeviceIt)->SetChannelValue(ControllerId, ChannelType, Value);
	}
}
void FNullApplication::SetForceFeedbackChannelValues(int32 ControllerId, const FForceFeedbackValues& Values)
{
	if (FApp::UseVRFocus() && !FApp::HasVRFocus())
	{
		return; // do not proceed if the app uses VR focus but doesn't have it
	}

	// send vibration to externally-implemented devices
	for (auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt)
	{
		// *N.B 06/20/2016*: Ideally, we would want to use GetHapticDevice instead
		// but they're not implemented for SteamController and SteamVRController
		if ((*DeviceIt)->IsGamepadAttached())
		{
			(*DeviceIt)->SetChannelValues(ControllerId, Values);
		}
	}
}

void FNullApplication::SetHapticFeedbackValues(int32 ControllerId, int32 Hand, const FHapticFeedbackValues& Values)
{
	if (FApp::UseVRFocus() && !FApp::HasVRFocus())
	{
		return; // do not proceed if the app uses VR focus but doesn't have it
	}

	for (auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt)
	{
		IHapticDevice* HapticDevice = (*DeviceIt)->GetHapticDevice();
		if (HapticDevice)
		{
			HapticDevice->SetHapticFeedbackValues(ControllerId, Hand, Values);
		}
	}
}

bool FNullApplication::ParseResolution(const TCHAR* InResolution, uint32& OutX, uint32& OutY)
{
	if (*InResolution)
	{
		FString CmdString(InResolution);
		CmdString = CmdString.TrimStartAndEnd().ToLower();

		// Retrieve the X dimensional value
		const uint32 X = FMath::Max(FCString::Atoi(*CmdString), 0);

		// Determine whether the user has entered a resolution and extract the Y dimension.
		FString YString;

		// Find separator between values (Example of expected format: 1280x768)
		const TCHAR* YValue = NULL;
		if (FCString::Strchr(*CmdString, 'x'))
		{
			YValue = const_cast<TCHAR*>(FCString::Strchr(*CmdString, 'x') + 1);
			YString = YValue;
			// Remove any whitespace from the end of the string
			YString = YString.TrimStartAndEnd();
		}

		// If the Y dimensional value exists then setup to use the specified resolution.
		uint32 Y = 0;
		if (YValue && YString.Len() > 0)
		{
			if (YString.IsNumeric())
			{
				Y = FMath::Max(FCString::Atoi(YValue), 0);
				OutX = X;
				OutY = Y;
				return true;
			}
		}
	}
	return false;
}

void FNullApplication::OnInputDeviceModuleRegistered(const FName& Type, IModularFeature* ModularFeature)
{
	if (Type == IInputDeviceModule::GetModularFeatureName())
	{
		IInputDeviceModule* Module = static_cast<IInputDeviceModule*>(ModularFeature);

		TSharedPtr<IInputDevice> Device = Module->CreateInputDevice(MessageHandler);
		if (Device.IsValid())
		{
			UE_LOGF(LogInit, Log, "Adding external input plugin.");
			ExternalInputDevices.Add(MoveTemp(Device));
		}
	}
}

void FNullPlatformDisplayMetrics::RebuildDisplayMetrics(struct FDisplayMetrics& OutDisplayMetrics)
{
	int32 X, Y, Width, Height;
	FNullApplication::GetFullscreenInfo(X, Y, Width, Height);
	// Total screen size of the primary monitor
	OutDisplayMetrics.PrimaryDisplayWidth = Width;
	OutDisplayMetrics.PrimaryDisplayHeight = Height;


	OutDisplayMetrics.PrimaryDisplayWorkAreaRect.Left = X;
	OutDisplayMetrics.PrimaryDisplayWorkAreaRect.Top = Y;
	OutDisplayMetrics.PrimaryDisplayWorkAreaRect.Right = X + Width;
	OutDisplayMetrics.PrimaryDisplayWorkAreaRect.Bottom = Y + Height;
	
	// Virtual desktop area
	OutDisplayMetrics.VirtualDisplayRect.Left = X;
	OutDisplayMetrics.VirtualDisplayRect.Top = Y;
	OutDisplayMetrics.VirtualDisplayRect.Right = X + Width;
	OutDisplayMetrics.VirtualDisplayRect.Bottom = Y + Height;
}