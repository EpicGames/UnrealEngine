// Copyright Epic Games, Inc. All Rights Reserved.

#include "Null/NullCursor.h"

#include "Misc/App.h"
#include "Null/NullApplication.h"
#include "Null/NullPlatformApplicationMisc.h"

FNullCursor::FNullCursor()
{
	if (!FApp::CanEverRender())
	{
		// assume that non-rendering application will be fine with token cursor
		UE_LOGF(LogInit, Log, "Not creating cursor resources due to headless application.");
		return;
	}
}

FNullCursor::~FNullCursor()
{
}

FVector2D FNullCursor::GetPosition() const
{
	return FVector2D(CachedGlobalXPosition, CachedGlobalYPosition);
}

void FNullCursor::SetPosition(const int32 X, const int32 Y)
{
	CachedGlobalXPosition = X;
	CachedGlobalYPosition = Y;
}

void FNullCursor::SetType(const EMouseCursor::Type InNewCursor)
{
}

EMouseCursor::Type FNullCursor::GetType() const
{
	return EMouseCursor::Type::Default;
}

void FNullCursor::GetSize(int32& Width, int32& Height) const
{
	Width = 16;
	Height = 16;
}

void FNullCursor::Show(bool bShow)
{
}

void FNullCursor::Lock(const RECT* const Bounds)
{
}

void FNullCursor::SetTypeShape(EMouseCursor::Type InCursorType, void* InCursorHandle)
{
}
