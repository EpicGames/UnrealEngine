// Copyright Epic Games, Inc. All Rights Reserved.

#include "Null/NullPlatformApplicationMisc.h"
#include "Null/NullApplication.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"

GenericApplication* FNullPlatformApplicationMisc::CreateApplication()
{
	return FNullApplication::CreateNullApplication();
}

bool FNullPlatformApplicationMisc::IsUsingNullApplication()
{
	// NullPlatform Application only supported on Linux and Windows
	// Theoretically it's supported on all platforms but we need to use the Pixel Streaming plugin to view
	// the output, so we don't want to enable the null application on platforms that Pixel Streaming doesn't support
#if PLATFORM_WINDOWS || PLATFORM_LINUX
	static bool bIsRenderingOffScreen = FParse::Param(FCommandLine::Get(), TEXT("RenderOffScreen"));
	return bIsRenderingOffScreen;
#else
	return false;
#endif
}