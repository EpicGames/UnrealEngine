// Copyright Epic Games, Inc. All Rights Reserved.

#include "Null/NullWindow.h"
#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "Null/NullApplication.h"

DEFINE_LOG_CATEGORY_STATIC(LogNullPlatformWindow, Log, All);

TSharedRef<FNullWindow> FNullWindow::Make()
{
	return MakeShareable(new FNullWindow());
}

FNullWindow::FNullWindow()
	: DPIScaleFactor(1.f)
	, ScreenPosition(FIntPoint::ZeroValue)
	, SizeInScreen(FIntPoint::ZeroValue)
	, bIsVisible(false)
	, bIsInitialized(false)
{
}

FNullWindow::~FNullWindow()
{
}

void FNullWindow::Initialize(class FNullApplication* const Application, const TSharedRef<FGenericWindowDefinition>& InDefinition, const TSharedPtr<FNullWindow>& InParent, const bool bShowImmediately)
{
	Definition = InDefinition;

	ScreenPosition.X = static_cast<int32>(InDefinition->XDesiredPositionOnScreen);
	ScreenPosition.Y = static_cast<int32>(InDefinition->YDesiredPositionOnScreen);
	SizeInScreen.X = static_cast<int32>(InDefinition->WidthDesiredOnScreen);
	SizeInScreen.Y = static_cast<int32>(InDefinition->HeightDesiredOnScreen);

	bIsInitialized = true;
}

void FNullWindow::ReshapeWindow(int32 X, int32 Y, int32 Width, int32 Height)
{
	MoveWindowTo(X, Y);

	SizeInScreen.X = Width;
	SizeInScreen.Y = Height;
	FNullApplication::OnSizeChanged(this, Width, Height);
}

bool FNullWindow::GetFullScreenInfo(int32& X, int32& Y, int32& Width, int32& Height) const
{
	FNullApplication::GetFullscreenInfo(X, Y, Width, Height);
	return true;
}

void FNullWindow::MoveWindowTo(int32 X, int32 Y)
{
	ScreenPosition.X = X;
	ScreenPosition.Y = Y;

	FNullApplication::MoveWindowTo(this, X, Y);
}

void FNullWindow::BringToFront(bool bForce)
{
	// empty default functionality
}

void FNullWindow::HACK_ForceToFront()
{
	// empty default functionality
}

void FNullWindow::Destroy()
{
	FNullApplication::DestroyWindow(this);
}

void FNullWindow::Minimize()
{
	// Empty functionality as we don't want to minimize because we'll never be able to get it back
}

void FNullWindow::Maximize()
{
	int32 X, Y, Width, Height;
	if (GetFullScreenInfo(X, Y, Width, Height))
	{
		ReshapeWindow(X, Y, Width, Height);
	}
}

void FNullWindow::Restore()
{
	int32 X, Y, Width, Height;
	if (GetFullScreenInfo(X, Y, Width, Height))
	{
		int32 NewWidth = Width / 2;
		int32 NewHeight = Height / 2;

		ReshapeWindow(NewWidth / 2, NewHeight / 2, NewWidth, NewHeight);
	}
}

void FNullWindow::Show()
{
	if (!bIsVisible)
	{
		FNullApplication::ShowWindow(this);
		bIsVisible = true;
	}
}

void FNullWindow::Hide()
{
	if (bIsVisible)
	{
		FNullApplication::HideWindow(this);
		bIsVisible = false;
	}
}

void FNullWindow::SetWindowMode(EWindowMode::Type InNewWindowMode)
{
	// empty default functionality
}

EWindowMode::Type FNullWindow::GetWindowMode() const
{
	// default functionality
	return EWindowMode::Fullscreen;
}

bool FNullWindow::IsMaximized() const
{
	int32 X, Y, Width, Height;
	GetFullScreenInfo(X, Y, Width, Height);
	return (SizeInScreen.X == Width && SizeInScreen.Y == Height);
}

bool FNullWindow::IsMinimized() const
{
	return false;
}

bool FNullWindow::IsVisible() const
{
	// empty default functionality
	return bIsVisible;
}

bool FNullWindow::GetRestoredDimensions(int32& X, int32& Y, int32& Width, int32& Height)
{
	return false;
}

void FNullWindow::SetWindowFocus()
{
	// empty default functionality
}

void FNullWindow::SetOpacity(const float InOpacity)
{
	// empty default functionality
}

void FNullWindow::Enable(bool bEnable)
{
	// empty default functionality
}

bool FNullWindow::IsPointInWindow(int32 X, int32 Y) const
{
	// empty default functionality
	return true;
}

int32 FNullWindow::GetWindowBorderSize() const
{
	// empty default functionality
	return 0;
}

int32 FNullWindow::GetWindowTitleBarSize() const
{
	// empty default functionality
	return 0;
}

void* FNullWindow::GetOSWindowHandle() const
{
	return nullptr;
}

bool FNullWindow::IsForegroundWindow() const
{
	// empty default functionality
	return true;
}

bool FNullWindow::IsFullscreenSupported() const
{
	// empty default functionality
	return true;
}

void FNullWindow::SetText(const TCHAR* const Text)
{
	// empty default functionality
}

const FGenericWindowDefinition& FNullWindow::GetDefinition() const
{
	return *Definition.Get();
}

bool FNullWindow::IsDefinitionValid() const
{
	return Definition.IsValid();
}

void FNullWindow::AdjustCachedSize(FVector2D& Size) const
{
	if (bIsInitialized)
	{
		Size.X = SizeInScreen.X;
		Size.Y = SizeInScreen.Y;
	}
}

bool FNullWindow::IsManualManageDPIChanges() const
{
	// returns false by default so the application can auto-manage the size of its windows, in response to DPI variations
	return false;
}

void FNullWindow::SetManualManageDPIChanges(const bool bAutoHandle)
{
	// empty default functionality
}

void FNullWindow::DrawAttention(const FWindowDrawAttentionParameters& Parameters)
{
	// empty default functionality
}

void FNullWindow::SetNativeWindowButtonsVisibility(bool bVisible)
{
	// empty default functionality
}

bool FNullWindow::IsRenderingOffScreen() const
{
	return true;
}
