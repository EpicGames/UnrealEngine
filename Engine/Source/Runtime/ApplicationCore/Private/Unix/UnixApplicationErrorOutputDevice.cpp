// Copyright Epic Games, Inc. All Rights Reserved.

#include "Unix/UnixApplicationErrorOutputDevice.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/App.h"

void FUnixApplicationErrorOutputDevice::HandleErrorRestoreUI()
{
	// do not copy if graphics have not been initialized or if we're on the wrong thread
	if (FApp::CanEverRender() && IsInGameThread())
	{
		FPlatformApplicationMisc::ClipboardCopy(GErrorHist);
	}
}
