// Copyright Epic Games, Inc. All Rights Reserved.

#include "Unix/UnixFeedbackContext.h"

#include "Logging/StructuredLog.h"

void FUnixFeedbackContext::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category)
{
	Serialize(V, Verbosity, Category, -1.0);
}

void FUnixFeedbackContext::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time)
{
	if (Verbosity == ELogVerbosity::Error || (Verbosity == ELogVerbosity::Warning && TreatWarningsAsErrors))
	{
		// send errors (warnings are too spammy) to syslog too (for zabbix etc)
		LogErrorToSysLog(V, Category, Time);
	}

	FFeedbackContext::Serialize(V, Verbosity, Category, Time);
}

void FUnixFeedbackContext::SerializeRecord(const UE::FLogRecord& Record)
{
	const ELogVerbosity::Type Verbosity = Record.GetVerbosity();
	if (Verbosity == ELogVerbosity::Error || (Verbosity == ELogVerbosity::Warning && TreatWarningsAsErrors))
	{
		// send errors (warnings are too spammy) to syslog too (for zabbix etc)
		LogErrorRecordToSysLog(Record);
	}

	FFeedbackContext::SerializeRecord(Record);
}

/** Ask the user a binary question, returning their answer */
bool FUnixFeedbackContext::YesNof(const FText& Question)
{
	if ((GIsClient || GIsEditor) && !GIsSilent && !FApp::IsUnattended())
	{
		//return( ::MessageBox( NULL, TempStr, *NSLOCTEXT("Core", "Question", "Question").ToString(), MB_YESNO|MB_TASKMODAL ) == IDYES);
		STUBBED("+++++++++++++++ UNIXPLATFORMFEEDBACKCONTEXT.CPP DIALOG BOX PROMPT +++++++++++++++++++");
		return true;
	}
	else
	{
		return false;
	}
}

FORCENOINLINE void FUnixFeedbackContext::LogErrorToSysLog(const TCHAR* V, const FName& Category, double Time) const
{
	TStringBuilder<512> Line;
	FormatLine(Line, V, ELogVerbosity::Error, Category, Time);
	syslog(LOG_ERR | LOG_USER, "%s", (const char*)StringCast<UTF8CHAR>(*Line).Get());
}

FORCENOINLINE void FUnixFeedbackContext::LogErrorRecordToSysLog(const UE::FLogRecord& Record) const
{
	TStringBuilder<512> Line;
	FormatRecordLine(Line, Record);
	syslog(LOG_ERR | LOG_USER, "%s", (const char*)StringCast<UTF8CHAR>(*Line).Get());
}
