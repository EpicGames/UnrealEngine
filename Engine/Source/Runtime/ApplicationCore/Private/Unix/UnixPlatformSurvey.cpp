// Copyright Epic Games, Inc. All Rights Reserved.

#include "Unix/UnixPlatformSurvey.h"
#include "Math/UnrealMathUtility.h"
#include "HAL/UnrealMemory.h"

#include "SynthBenchmark.h"

bool FUnixPlatformSurvey::GetSurveyResults(FHardwareSurveyResults& OutResults, bool bWait)
{
	FMemory::Memset(&OutResults, 0, sizeof(FHardwareSurveyResults));
	WriteFStringToResults(OutResults.Platform, TEXT("Unix"));
	
	// OS
	FString OSVersionLabel;
	FString OSSubVersionLabel;
	FPlatformMisc::GetOSVersions(OSVersionLabel, OSSubVersionLabel);
	WriteFStringToResults(OutResults.OSVersion, OSVersionLabel);
	WriteFStringToResults(OutResults.OSSubVersion, OSSubVersionLabel);
	OutResults.OSBits = 64;

	// CPU
	OutResults.CPUCount = FPlatformMisc::NumberOfCores();  // TODO [RCL] 2015-07-15: parse /proc/cpuinfo for GHz/brand

	// Memory
	const FPlatformMemoryConstants& MemoryConstants = FPlatformMemory::GetConstants();
	OutResults.MemoryMB = MemoryConstants.TotalPhysical / ( 1024ULL * 1024ULL );

	// Misc
	OutResults.bIsRemoteSession = FPlatformMisc::HasBeenStartedRemotely();
	OutResults.bIsLaptopComputer = FPlatformMisc::IsRunningOnBattery();	// FIXME [RCL] 2015-07-15: incorrect. Laptops don't have to run on battery

	// Synth benchmark
	ISynthBenchmark::Get().Run(OutResults.SynthBenchmark, true, 5.f);

	OutResults.ErrorCount++;
	WriteFStringToResults(OutResults.LastSurveyError, TEXT("Survey is incomplete"));
	WriteFStringToResults(OutResults.LastSurveyErrorDetail, TEXT("CPU, OS details are missing"));

	return true;
}

void FUnixPlatformSurvey::WriteFStringToResults(TCHAR* OutBuffer, const FString& InString)
{
	FMemory::Memset( OutBuffer, 0, sizeof(TCHAR) * FHardwareSurveyResults::MaxStringLength );
	TCHAR* Cursor = OutBuffer;
	for (int32 i = 0; i < FMath::Min(InString.Len(), FHardwareSurveyResults::MaxStringLength - 1); i++)
	{
		*Cursor++ = InString[i];
	}
}
