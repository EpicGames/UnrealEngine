// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_ACCESSIBILITY && UE_WINDOWS_USING_UIA

#include "Windows/Accessibility/WindowsUIABaseProvider.h"

#include "Misc/AssertionMacros.h"
#include "Windows/Accessibility/WindowsUIAManager.h"
#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"

FWindowsUIABaseProvider::FWindowsUIABaseProvider(FWindowsUIAManager& InManager, TSharedRef<IAccessibleWidget> InWidget)
	: UIAManager(&InManager)
	, Widget(InWidget)
	, RefCount(1)
{
	// Register with the UIA Manager in order to receive OnUIAManagerDestroyed signal.
	UIAManager->ProviderList.Add(this);
}

FWindowsUIABaseProvider::~FWindowsUIABaseProvider()
{
	if (UIAManager)
	{
		UIAManager->ProviderList.Remove(this);
	}
}

void FWindowsUIABaseProvider::OnUIAManagerDestroyed()
{
	UIAManager = nullptr;
}

uint32 FWindowsUIABaseProvider::IncrementRef()
{
	// todo: check if this needs to be threadsafe using InterlockedIncrement
	return ++RefCount;
}

uint32 FWindowsUIABaseProvider::DecrementRef()
{
	ensure(RefCount > 0);
	if (--RefCount == 0)
	{
		delete this;
		return 0;
	}
	return RefCount;
}

bool FWindowsUIABaseProvider::IsValid() const
{
	return UIAManager != nullptr && Widget->IsValid();
}

#endif
