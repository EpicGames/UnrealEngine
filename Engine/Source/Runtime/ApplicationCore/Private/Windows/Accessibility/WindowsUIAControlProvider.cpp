// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_ACCESSIBILITY && UE_WINDOWS_USING_UIA

#include "Windows/Accessibility/WindowsUIAControlProvider.h"
#include "Windows/Accessibility/WindowsUIAWidgetProvider.h"
#include "Windows/Accessibility/WindowsUIAPropertyGetters.h"
#include "Windows/Accessibility/WindowsUIAManager.h"
#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"

// FWindowsUIATextRangeProvider

FWindowsUIATextRangeProvider::FWindowsUIATextRangeProvider(FWindowsUIAManager& InManager, TSharedRef<IAccessibleWidget> InWidget, FTextRange InRange)
	: FWindowsUIABaseProvider(InManager, InWidget)
	, TextRange(InRange)
{
}

FWindowsUIATextRangeProvider::~FWindowsUIATextRangeProvider()
{
}

HRESULT STDCALL FWindowsUIATextRangeProvider::QueryInterface(REFIID riid, void** ppInterface)
{
	*ppInterface = nullptr;

	if (riid == __uuidof(IUnknown))
	{
		*ppInterface = static_cast<ITextRangeProvider*>(this);
	}
	else if (riid == __uuidof(ITextRangeProvider))
	{
		*ppInterface = static_cast<ITextRangeProvider*>(this);
	}

	if (*ppInterface)
	{
		AddRef();
		return S_OK;
	}
	else
	{
		return E_NOINTERFACE;
	}
}

FString FWindowsUIATextRangeProvider::TextFromTextRange()
{
	return TextFromTextRange(Widget->AsText()->GetText(), TextRange);
}

FString FWindowsUIATextRangeProvider::TextFromTextRange(const FString& InString, const FTextRange& InRange)
{
	return InString.Mid(InRange.BeginIndex, InRange.EndIndex);
}

ULONG STDCALL FWindowsUIATextRangeProvider::AddRef()
{
	return FWindowsUIABaseProvider::IncrementRef();
}

ULONG STDCALL FWindowsUIATextRangeProvider::Release()
{
	return FWindowsUIABaseProvider::DecrementRef();
}

HRESULT STDCALL FWindowsUIATextRangeProvider::Clone(ITextRangeProvider** pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		if (IsValid())
		{
			*pRetVal = new FWindowsUIATextRangeProvider(*UIAManager, Widget, TextRange);
			ReturnValue = S_OK;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIATextRangeProvider::Compare(ITextRangeProvider* range, BOOL* pRetVal)
{
	// The documentation states that different endpoints that produce the same text are not equal,
	// but doesn't say anything about the same endpoints that come from different control providers.
	// Perhaps we can assume that comparing text ranges from different Widgets is not valid.
	UIAManager->RunInGameThreadBlocking(
		[this, &range, &pRetVal]() {

		*pRetVal = TextRange == static_cast<FWindowsUIATextRangeProvider*>(range)->TextRange;
	});
	return S_OK;
}

HRESULT STDCALL FWindowsUIATextRangeProvider::CompareEndpoints(TextPatternRangeEndpoint endpoint, ITextRangeProvider* targetRange, TextPatternRangeEndpoint targetEndpoint, int* pRetVal)
{
	FWindowsUIATextRangeProvider* CastedRange = static_cast<FWindowsUIATextRangeProvider*>(targetRange);
	int32 ThisEndpointIndex = (endpoint == TextPatternRangeEndpoint_Start) ? TextRange.BeginIndex : TextRange.EndIndex;
	int32 OtherEndpointIndex = (targetEndpoint == TextPatternRangeEndpoint_Start) ? CastedRange->TextRange.BeginIndex : CastedRange->TextRange.EndIndex;
	*pRetVal = ThisEndpointIndex - OtherEndpointIndex;
	return S_OK;
}

HRESULT STDCALL FWindowsUIATextRangeProvider::ExpandToEnclosingUnit(TextUnit unit)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &unit]() {
		if (IsValid())
		{
			switch (unit)
			{
			case TextUnit_Character:
				TextRange.EndIndex = FMath::Min(TextRange.BeginIndex + 1, Widget->AsText()->GetText().Len());
				break;
			case TextUnit_Format:
				ReturnValue = E_NOTIMPL;
				break;
			case TextUnit_Word:
				ReturnValue = E_NOTIMPL;
				break;
			case TextUnit_Line:
				ReturnValue = E_NOTIMPL;
				break;
			case TextUnit_Paragraph:
				ReturnValue = E_NOTIMPL;
				break;
			case TextUnit_Page:
				ReturnValue = E_NOTIMPL;
				break;
			case TextUnit_Document:
				TextRange = FTextRange(0, Widget->AsText()->GetText().Len());
				break;
			}
			ReturnValue = S_OK;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIATextRangeProvider::FindAttribute(TEXTATTRIBUTEID attributeId, VARIANT val, BOOL backward, ITextRangeProvider** pRetVal)
{
	return E_NOTIMPL;
}

HRESULT STDCALL FWindowsUIATextRangeProvider::FindText(BSTR text, BOOL backward, BOOL ignoreCase, ITextRangeProvider** pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &text, &backward, &ignoreCase, &pRetVal]() {
		if (IsValid())
		{
			FString TextToSearch(text);
			int32 FoundIndex = TextFromTextRange().Find(TextToSearch, ignoreCase ? ESearchCase::IgnoreCase : ESearchCase::CaseSensitive, backward ? ESearchDir::FromEnd : ESearchDir::FromStart);
			if (FoundIndex == INDEX_NONE)
			{
				*pRetVal = nullptr;
			}
			else
			{
				const int32 StartIndex = TextRange.BeginIndex + FoundIndex;
				*pRetVal = new FWindowsUIATextRangeProvider(*UIAManager, Widget, FTextRange(StartIndex, StartIndex + TextToSearch.Len()));
			}
			ReturnValue = S_OK;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIATextRangeProvider::GetAttributeValue(TEXTATTRIBUTEID attributeId, VARIANT* pRetVal)
{
	return E_NOTIMPL;
}

HRESULT STDCALL FWindowsUIATextRangeProvider::GetBoundingRectangles(SAFEARRAY** pRetVal)
{
	return E_NOTIMPL;
}

HRESULT STDCALL FWindowsUIATextRangeProvider::GetEnclosingElement(IRawElementProviderSimple** pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		if (IsValid())
		{
			*pRetVal = static_cast<IRawElementProviderSimple*>(&UIAManager->GetWidgetProvider(Widget));
			ReturnValue = S_OK;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIATextRangeProvider::GetText(int maxLength, BSTR* pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &maxLength, &pRetVal]() {
		if (IsValid())
		{
			*pRetVal = SysAllocString(*TextFromTextRange().Left(maxLength));
			ReturnValue = S_OK;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIATextRangeProvider::Move(TextUnit unit, int count, int* pRetVal)
{
	return E_NOTIMPL;
}

HRESULT STDCALL FWindowsUIATextRangeProvider::MoveEndpointByUnit(TextPatternRangeEndpoint endpoint, TextUnit unit, int count, int* pRetVal)
{
	return E_NOTIMPL;
}

HRESULT STDCALL FWindowsUIATextRangeProvider::MoveEndpointByRange(TextPatternRangeEndpoint endpoint, ITextRangeProvider* targetRange, TextPatternRangeEndpoint targetEndpoint)
{
	UIAManager->RunInGameThreadBlocking(
		[this, &endpoint, &targetRange, &targetEndpoint]() {

		FWindowsUIATextRangeProvider* CastedRange = static_cast<FWindowsUIATextRangeProvider*>(targetRange);
		int32 NewIndex = (targetEndpoint == TextPatternRangeEndpoint_Start) ? CastedRange->TextRange.BeginIndex : CastedRange->TextRange.EndIndex;
		if (endpoint == TextPatternRangeEndpoint_Start)
		{
			TextRange.BeginIndex = NewIndex;
			if (TextRange.BeginIndex > TextRange.EndIndex)
			{
				TextRange.EndIndex = TextRange.BeginIndex;
			}
		}
		else
		{
			TextRange.EndIndex = NewIndex;
			if (TextRange.BeginIndex > TextRange.EndIndex)
			{
				TextRange.BeginIndex = TextRange.EndIndex;
			}
		}
	});
	return S_OK;
}

HRESULT STDCALL FWindowsUIATextRangeProvider::Select()
{
	return E_NOTIMPL;
}

HRESULT STDCALL FWindowsUIATextRangeProvider::AddToSelection()
{
	return E_NOTIMPL;
}

HRESULT STDCALL FWindowsUIATextRangeProvider::RemoveFromSelection()
{
	return E_NOTIMPL;
}

HRESULT STDCALL FWindowsUIATextRangeProvider::ScrollIntoView(BOOL alignToTop)
{
	return E_NOTIMPL;
}

HRESULT STDCALL FWindowsUIATextRangeProvider::GetChildren(SAFEARRAY** pRetVal)
{
	*pRetVal = nullptr;
	return S_OK;
}

// ~

// FWindowsUIAControlProvider

FWindowsUIAControlProvider::FWindowsUIAControlProvider(FWindowsUIAManager& InManager, TSharedRef<IAccessibleWidget> InWidget)
	: FWindowsUIABaseProvider(InManager, InWidget)
{
}

FWindowsUIAControlProvider::~FWindowsUIAControlProvider()
{
}

HRESULT STDCALL FWindowsUIAControlProvider::QueryInterface(REFIID riid, void** ppInterface)
{
	*ppInterface = nullptr;

	if (riid == __uuidof(IInvokeProvider))
	{
		*ppInterface = static_cast<IInvokeProvider*>(this);
	}
	else if (riid == __uuidof(IRangeValueProvider))
	{
		*ppInterface = static_cast<IRangeValueProvider*>(this);
	}
	else if (riid == __uuidof(ITextProvider))
	{
		*ppInterface = static_cast<ITextProvider*>(this);
	}
	else if (riid == __uuidof(IToggleProvider))
	{
		*ppInterface = static_cast<IToggleProvider*>(this);
	}
	else if (riid == __uuidof(IValueProvider))
	{
		*ppInterface = static_cast<IValueProvider*>(this);
	}
	else if (riid == __uuidof(IWindowProvider))
	{
		*ppInterface = static_cast<IWindowProvider*>(this);
	}
	else if (riid == __uuidof(ISelectionProvider))
	{
		*ppInterface = static_cast<ISelectionProvider*>(this);
	}
	else if (riid == __uuidof(ISelectionItemProvider))
	{
		*ppInterface = static_cast<ISelectionItemProvider*>(this);
	}
	if (*ppInterface)
	{
		AddRef();
		return S_OK;
	}
	else
	{
		return E_NOINTERFACE;
	}
}

ULONG STDCALL FWindowsUIAControlProvider::AddRef()
{
	return FWindowsUIABaseProvider::IncrementRef();
}

ULONG STDCALL FWindowsUIAControlProvider::Release()
{
	return FWindowsUIABaseProvider::DecrementRef();
}

HRESULT STDCALL FWindowsUIAControlProvider::Invoke()
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue]() {
		if (IsValid())
		{
			Widget->AsActivatable()->Activate();
			ReturnValue = S_OK;
		}
	});
	return ReturnValue;
	
}

HRESULT STDCALL FWindowsUIAControlProvider::SetValue(double val)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &val]() {
		if (IsValid())
		{
			Widget->AsProperty()->SetValue(FString::SanitizeFloat(val));
			ReturnValue = S_OK;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAControlProvider::get_Value(double* pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		if (IsValid())
		{
			*pRetVal = WindowsUIAPropertyGetters::GetPropertyValue(Widget, UIA_RangeValueValuePropertyId).GetValue<double>();
			ReturnValue = S_OK;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAControlProvider::get_IsReadOnly(BOOL* pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		if (IsValid())
		{
			*pRetVal = WindowsUIAPropertyGetters::GetPropertyValue(Widget, UIA_ValueIsReadOnlyPropertyId).GetValue<bool>();
			ReturnValue = S_OK;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAControlProvider::get_Maximum(double* pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		if (IsValid())
		{
			*pRetVal = WindowsUIAPropertyGetters::GetPropertyValue(Widget, UIA_RangeValueMaximumPropertyId).GetValue<double>();
			ReturnValue = S_OK;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAControlProvider::get_Minimum(double* pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		if (IsValid())
		{
			*pRetVal = WindowsUIAPropertyGetters::GetPropertyValue(Widget, UIA_RangeValueMinimumPropertyId).GetValue<double>();
			ReturnValue = S_OK;
		}

	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAControlProvider::get_LargeChange(double* pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		if (IsValid())
		{
			*pRetVal = WindowsUIAPropertyGetters::GetPropertyValue(Widget, UIA_RangeValueLargeChangePropertyId).GetValue<double>();
			ReturnValue = S_OK;
		}

	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAControlProvider::get_SmallChange(double* pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		if (IsValid())
		{
			*pRetVal = WindowsUIAPropertyGetters::GetPropertyValue(Widget, UIA_RangeValueSmallChangePropertyId).GetValue<double>();
			ReturnValue = S_OK;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAControlProvider::get_DocumentRange(ITextRangeProvider** pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		if (IsValid())
		{
			*pRetVal = static_cast<ITextRangeProvider*>(new FWindowsUIATextRangeProvider(*UIAManager, Widget, FTextRange(0, Widget->AsText()->GetText().Len())));
			ReturnValue = S_OK;
		}

	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAControlProvider::get_SupportedTextSelection(SupportedTextSelection* pRetVal)
{
	// todo: implement selection
	*pRetVal = SupportedTextSelection_None;
	return S_OK;
}

HRESULT STDCALL FWindowsUIAControlProvider::GetSelection(SAFEARRAY** pRetVal)
{
	HRESULT ReturnValue = E_NOTIMPL;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
			if (IsValid())
			{
				if (Widget->AsTable())
				{
					// @TODOAccessibility: MSDN does not state what to do if there is nothing selected for the C++ implementation 
					// We'll return an empty array and hope nothing blows up 
					TArray<TSharedPtr<IAccessibleWidget>> SelectedItems = Widget->AsTable()->GetSelectedItems();
					ULONG NumElements = static_cast<ULONG>(SelectedItems.Num());
					*pRetVal = SafeArrayCreateVector(VT_UNKNOWN, 0, NumElements);
					for(int32 Index = 0; Index < SelectedItems.Num(); ++Index)
					{ 
						const TSharedPtr<IAccessibleWidget>& CurrentSelectedItem = SelectedItems[Index];
						FScopedWidgetProvider ScopedProvider(UIAManager->GetWidgetProvider(CurrentSelectedItem.ToSharedRef()));
						LONG PutIndex = static_cast<LONG>(Index);
						ReturnValue = SafeArrayPutElement(*pRetVal, &PutIndex, &ScopedProvider.Provider);
						if (ReturnValue != S_OK)
						{
							break;
						}
					}
					
				}
			}
		});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAControlProvider::GetVisibleRanges(SAFEARRAY** pRetVal)
{
	return E_NOTIMPL;
}

HRESULT STDCALL FWindowsUIAControlProvider::RangeFromChild(IRawElementProviderSimple* childElement, ITextRangeProvider** pRetVal)
{
	return E_NOTIMPL;
}

HRESULT STDCALL FWindowsUIAControlProvider::RangeFromPoint(UiaPoint point, ITextRangeProvider** pRetVal)
{
	return E_NOTIMPL;
}

HRESULT STDCALL FWindowsUIAControlProvider::get_ToggleState(ToggleState* pRetVal)
{
	//ECheckBoxState CheckState = Widget->AsActivatable()->GetCheckedState();
	//switch (CheckState)
	//{
	//case ECheckBoxState::Checked:
	//	*pRetVal = ToggleState_On;
	//	break;
	//case ECheckBoxState::Unchecked:
	//	*pRetVal = ToggleState_Off;
	//	break;
	//case ECheckBoxState::Undetermined:
	//	*pRetVal = ToggleState_Indeterminate;
	//	break;
	//}

	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		if (IsValid())
		{
			*pRetVal = static_cast<ToggleState>(WindowsUIAPropertyGetters::GetPropertyValue(Widget, UIA_ToggleToggleStatePropertyId).GetValue<int32>());
			ReturnValue = S_OK;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAControlProvider::Toggle()
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue]() {
		if (IsValid())
		{
			Widget->AsActivatable()->Activate();
			ReturnValue = S_OK;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAControlProvider::get_CanMove(BOOL *pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		if (IsValid())
		{
			*pRetVal = WindowsUIAPropertyGetters::GetPropertyValue(Widget, UIA_TransformCanMovePropertyId).GetValue<bool>();
			ReturnValue = S_OK;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAControlProvider::get_CanResize(BOOL *pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		if (IsValid())
		{
			*pRetVal = WindowsUIAPropertyGetters::GetPropertyValue(Widget, UIA_TransformCanResizePropertyId).GetValue<bool>();
			ReturnValue = S_OK;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAControlProvider::get_CanRotate(BOOL *pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		if (IsValid())
		{
			*pRetVal = WindowsUIAPropertyGetters::GetPropertyValue(Widget, UIA_TransformCanRotatePropertyId).GetValue<bool>();
			ReturnValue = S_OK;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAControlProvider::Move(double x, double y)
{
	return E_NOTIMPL;
}

HRESULT STDCALL FWindowsUIAControlProvider::Resize(double width, double height)
{
	return E_NOTIMPL;
}

HRESULT STDCALL FWindowsUIAControlProvider::Rotate(double degrees)
{
	return E_NOTIMPL;
}

HRESULT STDCALL FWindowsUIAControlProvider::SetValue(LPCWSTR val)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &val]() {
		if (IsValid())
		{
			Widget->AsProperty()->SetValue(FString(val));
			ReturnValue = S_OK;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAControlProvider::get_Value(BSTR* pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		if (IsValid())
		{
			*pRetVal = SysAllocString(*WindowsUIAPropertyGetters::GetPropertyValue(Widget, UIA_ValueValuePropertyId).GetValue<FString>());
			ReturnValue = S_OK;
		}

	});
	return ReturnValue;
}

//HRESULT STDCALL FWindowsUIAControlProvider::get_IsReadOnly(BOOL* pRetVal)
//{
//	*pRetVal = Widget->AsProperty()->IsReadOnly();
//	return S_OK;
//}

HRESULT STDCALL FWindowsUIAControlProvider::Close()
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue]() {
		if (IsValid())
		{
			Widget->AsWindow()->Close();
			ReturnValue = S_OK;
		}

	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAControlProvider::get_CanMaximize(BOOL* pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		if (IsValid())
		{
			*pRetVal = WindowsUIAPropertyGetters::GetPropertyValue(Widget, UIA_WindowCanMaximizePropertyId).GetValue<bool>();
			ReturnValue = S_OK;
		}

	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAControlProvider::get_CanMinimize(BOOL* pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		if (IsValid())
		{
			*pRetVal = WindowsUIAPropertyGetters::GetPropertyValue(Widget, UIA_WindowCanMinimizePropertyId).GetValue<bool>();
			ReturnValue = S_OK;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAControlProvider::get_IsModal(BOOL* pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		if (IsValid())
		{
			*pRetVal = WindowsUIAPropertyGetters::GetPropertyValue(Widget, UIA_WindowIsModalPropertyId).GetValue<bool>();
			ReturnValue = S_OK;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAControlProvider::get_IsTopmost(BOOL* pRetVal)
{
	HRESULT ReturnValue = E_NOTIMPL;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		// todo: not 100% sure what this is looking for. top window in hierarchy of child windows? on top of all other windows in Windows OS?
		*pRetVal = WindowsUIAPropertyGetters::GetPropertyValue(Widget, UIA_WindowIsTopmostPropertyId).GetValue<bool>();
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAControlProvider::get_WindowInteractionState(WindowInteractionState* pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &pRetVal]() {
		if (IsValid())
		{
			// todo: do we have a way to identify if the app is processing data vs idling?
			*pRetVal = static_cast<WindowInteractionState>(WindowsUIAPropertyGetters::GetPropertyValue(Widget, UIA_WindowWindowInteractionStatePropertyId).GetValue<int32>());
		}
		else
		{
			*pRetVal = WindowInteractionState_Closing;
		}

	});
	return S_OK;
}

HRESULT STDCALL FWindowsUIAControlProvider::get_WindowVisualState(WindowVisualState* pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		if (IsValid())
		{
			*pRetVal = static_cast<WindowVisualState>(WindowsUIAPropertyGetters::GetPropertyValue(Widget, UIA_WindowWindowVisualStatePropertyId).GetValue<int32>());
			ReturnValue = S_OK;
		}

	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAControlProvider::SetVisualState(WindowVisualState state)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &state]() {
		if (IsValid())
		{
			switch (state)
			{
			case WindowVisualState_Normal:
				Widget->AsWindow()->SetDisplayState(IAccessibleWindow::EWindowDisplayState::Normal);
				break;
			case WindowVisualState_Minimized:
				Widget->AsWindow()->SetDisplayState(IAccessibleWindow::EWindowDisplayState::Minimize);
				break;
			case WindowVisualState_Maximized:
				Widget->AsWindow()->SetDisplayState(IAccessibleWindow::EWindowDisplayState::Maximize);
				break;
			}
			ReturnValue = S_OK;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAControlProvider::WaitForInputIdle(int milliseconds, BOOL* pRetVal)
{
	return E_NOTIMPL;
}

// ISelectionProvider
// Left here for clarity. There is a clash between ISelectionProvider::GetSelection() and ITextProvider::GetSelection().
// We implement both versions in ITextProvider
//HRESULT STDMETHODCALLTYPE FWindowsUIAControlProvider::GetSelection(SAFEARRAY** pRetVal)
//{
//	return E_NOTIMPL;
//	}

HRESULT STDMETHODCALLTYPE FWindowsUIAControlProvider::get_CanSelectMultiple(BOOL* pRetVal)
{
	HRESULT ReturnValue = E_NOTIMPL;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
			if (IsValid())
			{
				if (Widget->AsTable())
				{
					*pRetVal = Widget->AsTable()->CanSupportMultiSelection();
					ReturnValue = S_OK;
				}
			}
		});
	return ReturnValue;
}

HRESULT STDMETHODCALLTYPE FWindowsUIAControlProvider::get_IsSelectionRequired(BOOL* pRetVal)
{
	HRESULT ReturnValue = E_NOTIMPL;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
			if (IsValid())
			{
				if (Widget->AsTable())
				{
					*pRetVal = Widget->AsTable()->IsSelectionRequired();
					ReturnValue = S_OK;
				}
			}
		});
	return ReturnValue;
}
// ~

// ISelectionItemProvider
HRESULT STDMETHODCALLTYPE FWindowsUIAControlProvider::Select()
{
	HRESULT ReturnValue = E_NOTIMPL;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue]() {
			if (IsValid())
			{
				if (Widget->AsTableRow())
				{
					Widget->AsTableRow()->Select();
					ReturnValue = S_OK;
				}
			}
		});
	return ReturnValue;
}

HRESULT STDMETHODCALLTYPE FWindowsUIAControlProvider::AddToSelection()
{
	return E_NOTIMPL;
}

HRESULT STDMETHODCALLTYPE FWindowsUIAControlProvider::RemoveFromSelection()
{
	return E_NOTIMPL;
}

HRESULT STDMETHODCALLTYPE FWindowsUIAControlProvider::get_IsSelected(BOOL* pRetVal)
{
	HRESULT ReturnValue = E_NOTIMPL;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
			if (IsValid())
			{
				if (Widget->AsTableRow())
				{
					*pRetVal = Widget->AsTableRow()->IsSelected();
					ReturnValue = S_OK;
				}
			}
		});
	return ReturnValue;
}

HRESULT STDMETHODCALLTYPE FWindowsUIAControlProvider::get_SelectionContainer(IRawElementProviderSimple** pRetVal)
{
	HRESULT ReturnValue = E_NOTIMPL;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
			if (IsValid())
			{
				if (Widget->AsTableRow())
				{
					TSharedPtr<IAccessibleWidget> AccessibleContainer = Widget->AsTableRow()->GetOwningTable();
					if (AccessibleContainer.IsValid())
					{
						FScopedWidgetProvider ScopedProvider(UIAManager->GetWidgetProvider(AccessibleContainer.ToSharedRef()));
						*pRetVal = &ScopedProvider.Provider;
						ReturnValue = S_OK;
					}
				}
			}
		});
	return ReturnValue;
}
// ~
#endif
