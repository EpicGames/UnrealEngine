// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_ACCESSIBILITY && UE_WINDOWS_USING_UIA

#include "Windows/Accessibility/WindowsUIAManager.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include <Ole2.h>
#include "Windows/HideWindowsPlatformTypes.h"
#include <UIAutomation.h>

#include "Misc/Variant.h"
#include "Windows/Accessibility/WindowsUIAControlProvider.h"
#include "Windows/Accessibility/WindowsUIAWidgetProvider.h"
#include "Windows/Accessibility/WindowsUIAPropertyGetters.h"
#include "Windows/WindowsApplication.h"
#include "Windows/WindowsWindow.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Culture.h"


TMap<EAccessibleWidgetType, ULONG> FWindowsUIAManager::WidgetTypeToWindowsTypeMap;
TMap<EAccessibleWidgetType, FText> FWindowsUIAManager::WidgetTypeToTextMap;

#define LOCTEXT_NAMESPACE "SlateAccessibility"

/**
 * Helper function to raise a UIA property changed event.
 */
void EmitPropertyChangedEvent(FWindowsUIAWidgetProvider* Provider, PROPERTYID Property, const FVariant& OldValue, const FVariant& NewValue)
{
	UE_LOGF(LogAccessibility, VeryVerbose, "UIA Property Changed: %i", Property);
	UiaRaiseAutomationPropertyChangedEvent(static_cast<IRawElementProviderSimple*>(Provider), Property,
										   WindowsUIAPropertyGetters::FVariantToWindowsVariant(OldValue),
										   WindowsUIAPropertyGetters::FVariantToWindowsVariant(NewValue));
}

FWindowsUIAManager::FWindowsUIAManager(const FWindowsApplication& InApplication)
	: WindowsApplication(InApplication)
	, OnCultureChangedHandle()
	, CachedCurrentLocaleLCID(0)
{
	OnAccessibleMessageHandlerChanged();

#if !UE_BUILD_SHIPPING
	IConsoleManager::Get().RegisterConsoleCommand
	(
		TEXT("Accessibility.DumpStatsWindows"),
		TEXT("Writes to LogAccessibility the memory stats for the platform-level accessibility data (Providers) required for Windows support."),
		FConsoleCommandDelegate::CreateRaw(this, &FWindowsUIAManager::DumpAccessibilityStats),
		ECVF_Default
	);
#endif

	if (WidgetTypeToWindowsTypeMap.Num() == 0)
	{
			WidgetTypeToWindowsTypeMap.Add(EAccessibleWidgetType::Button, UIA_ButtonControlTypeId);
			WidgetTypeToWindowsTypeMap.Add(EAccessibleWidgetType::CheckBox, UIA_CheckBoxControlTypeId);
			WidgetTypeToWindowsTypeMap.Add(EAccessibleWidgetType::ComboBox, UIA_ComboBoxControlTypeId);
			WidgetTypeToWindowsTypeMap.Add(EAccessibleWidgetType::Hyperlink, UIA_HyperlinkControlTypeId);
			WidgetTypeToWindowsTypeMap.Add(EAccessibleWidgetType::Image, UIA_ImageControlTypeId);
			WidgetTypeToWindowsTypeMap.Add(EAccessibleWidgetType::Layout, UIA_PaneControlTypeId);
			WidgetTypeToWindowsTypeMap.Add(EAccessibleWidgetType::ScrollBar, UIA_ScrollBarControlTypeId);
			WidgetTypeToWindowsTypeMap.Add(EAccessibleWidgetType::Slider, UIA_SliderControlTypeId);
			WidgetTypeToWindowsTypeMap.Add(EAccessibleWidgetType::Text, UIA_TextControlTypeId);
			WidgetTypeToWindowsTypeMap.Add(EAccessibleWidgetType::TextEdit, UIA_EditControlTypeId);
			WidgetTypeToWindowsTypeMap.Add(EAccessibleWidgetType::Window, UIA_WindowControlTypeId);
			WidgetTypeToWindowsTypeMap.Add(EAccessibleWidgetType::List, UIA_ListControlTypeId);
			WidgetTypeToWindowsTypeMap.Add(EAccessibleWidgetType::ListItem, UIA_ListItemControlTypeId);

			WidgetTypeToTextMap.Add(EAccessibleWidgetType::Button, LOCTEXT("ControlTypeButton", "button"));
			WidgetTypeToTextMap.Add(EAccessibleWidgetType::CheckBox, LOCTEXT("ControlTypeCheckBox", "check box"));
			WidgetTypeToTextMap.Add(EAccessibleWidgetType::ComboBox, LOCTEXT("ControlTypeComboBox", "combobox"));
			WidgetTypeToTextMap.Add(EAccessibleWidgetType::Hyperlink, LOCTEXT("ControlTypeHyperlink", "hyperlink"));
			WidgetTypeToTextMap.Add(EAccessibleWidgetType::Image, LOCTEXT("ControlTypeImage", "image"));
			WidgetTypeToTextMap.Add(EAccessibleWidgetType::Layout, LOCTEXT("ControlTypeLayout", "pane"));
			WidgetTypeToTextMap.Add(EAccessibleWidgetType::ScrollBar, LOCTEXT("ControlTypeScrollBar", "scroll bar"));
			WidgetTypeToTextMap.Add(EAccessibleWidgetType::Slider, LOCTEXT("ControlTypeSlider", "slider"));
			WidgetTypeToTextMap.Add(EAccessibleWidgetType::Text, LOCTEXT("ControlTypeText", "text"));
			WidgetTypeToTextMap.Add(EAccessibleWidgetType::TextEdit, LOCTEXT("ControlTypeTextEdit", "edit"));
			WidgetTypeToTextMap.Add(EAccessibleWidgetType::Window, LOCTEXT("ControlTypeWindow", "window"));
			WidgetTypeToTextMap.Add(EAccessibleWidgetType::List, LOCTEXT("ControlTypeList", "List"));
			WidgetTypeToTextMap.Add(EAccessibleWidgetType::ListItem, LOCTEXT("ControlTypeListItem", "ListItem"));
	}
}

void FWindowsUIAManager::OnAccessibleMessageHandlerChanged()
{
	TSharedRef<FGenericAccessibleMessageHandler> MessageHandler = WindowsApplication.GetAccessibleMessageHandler();
	// We register the primary user (keyboard) 
	// This user is what UIA will interact with
	FGenericAccessibleUserRegistry& UserRegistry = MessageHandler->GetAccessibleUserRegistry();
	// We failed to register the primary user, this should only happen if another user with the 0th index has already been registered.
	ensure(UserRegistry.RegisterUser(MakeShared<FGenericAccessibleUser>(FGenericAccessibleUserRegistry::GetPrimaryUserIndex())));

	MessageHandler->SetAccessibleEventDelegate(FGenericAccessibleMessageHandler::FAccessibleEvent::CreateRaw(this, &FWindowsUIAManager::OnEventRaised));
}

FWindowsUIAManager::~FWindowsUIAManager()
{
	WindowsApplication.GetAccessibleMessageHandler()->SetAccessibleEventDelegate(FGenericAccessibleMessageHandler::FAccessibleEvent());
	for (FWindowsUIABaseProvider* Provider : ProviderList)
	{
		// The UIA Manager may be deleted before the Providers are, and external applications may attempt to call functions on those Providers.
		Provider->OnUIAManagerDestroyed();
	}

	if (OnCultureChangedHandle.IsValid()&& FInternationalization::Get().IsAvailable())
	{
		FInternationalization::Get().OnCultureChanged().Remove(OnCultureChangedHandle);
		OnCultureChangedHandle.Reset();
	}
}

FWindowsUIAWidgetProvider& FWindowsUIAManager::GetWidgetProvider(TSharedRef<IAccessibleWidget> InWidget)
{
	FWindowsUIAWidgetProvider** CachedProvider = CachedWidgetProviders.Find(InWidget);
	if (CachedProvider)
	{
		(*CachedProvider)->AddRef();
		return **CachedProvider;
	}
	else if (InWidget->AsWindow())
	{
		return *CachedWidgetProviders.Add(InWidget, new FWindowsUIAWindowProvider(*this, InWidget));
	}
	else
	{
		return *CachedWidgetProviders.Add(InWidget, new FWindowsUIAWidgetProvider(*this, InWidget));
	}
}

FWindowsUIAWindowProvider& FWindowsUIAManager::GetWindowProvider(TSharedRef<FWindowsWindow> InWindow)
{
	if (CachedWidgetProviders.Num() == 0)
	{
		// The first Get*Provider() request / MUST go through this function since IAccessibleWidgets will not exist untilthe accessible message handler is set active.
		OnAccessibilityEnabled();
	}

	TSharedPtr<IAccessibleWidget> AccessibleWindow = WindowsApplication.GetAccessibleMessageHandler()->GetAccessibleWindow(InWindow);
	checkf(AccessibleWindow.IsValid(), TEXT("%s is not an accessible window. All windows must be accessible."), *InWindow->GetDefinition().Title);
	return static_cast<FWindowsUIAWindowProvider&>(GetWidgetProvider(AccessibleWindow.ToSharedRef()));
}

void FWindowsUIAManager::OnAccessibilityEnabled()
{
	WindowsApplication.GetAccessibleMessageHandler()->SetActive(true);
	// Register for language and locale changes for internationalization.
	// Updates the LCID to be returned in FWindowsUIAWidgetProvider  as a UIA Property 
	UpdateCachedCurrentLocaleLCID();
	OnCultureChangedHandle = FInternationalization::Get().OnCultureChanged().AddRaw(this, &FWindowsUIAManager::UpdateCachedCurrentLocaleLCID);
}

void FWindowsUIAManager::OnWidgetProviderRemoved(TSharedRef<IAccessibleWidget> InWidget)
{
	CachedWidgetProviders.Remove(InWidget);

	if (CachedWidgetProviders.Num() == 0)
	{
		// If the last widget Provider is being removed, we can disable application accessibility.
		// Technically an external application could still be running listening for mouse/keyboard events,
		// but in practice I don't think its realistic to do this while having no references to any Provider.
		//
		// Note that there could still be control Providers with valid references. In practice I don't think
		// this is possible, but if we run into problems then we can simply call AddRef and Release on the
		// underlying widget Provider whenever a control Provider gets added/removed.
		OnAccessibilityDisabled();
	}
}

void FWindowsUIAManager::OnAccessibilityDisabled()
{
	WindowsApplication.GetAccessibleMessageHandler()->SetActive(false);
	CachedCurrentLocaleLCID = 0;
	if (OnCultureChangedHandle.IsValid())
	{
		FInternationalization::Get().OnCultureChanged().Remove(OnCultureChangedHandle);
		OnCultureChangedHandle.Reset();
	}
}

void FWindowsUIAManager::OnEventRaised(const FAccessibleEventArgs& Args)
{
	if (UiaClientsAreListening())
	{
		FScopedWidgetProvider ScopedProvider(GetWidgetProvider(Args.Widget));

		switch (Args.Event)
		{
		case EAccessibleEvent::FocusChange:
		{
			// On focus change, emit a generic FocusChanged event as well as a per-Provider PropertyChanged event
			// todo: handle difference between any focus vs keyboard focus
			if (Args.Widget->HasUserFocus(0))
			{
				UiaRaiseAutomationEvent(&ScopedProvider.Provider, UIA_AutomationFocusChangedEventId);
			}
			EmitPropertyChangedEvent(&ScopedProvider.Provider, UIA_HasKeyboardFocusPropertyId, Args.OldValue, Args.NewValue);
			break;
		}
		case EAccessibleEvent::Activate:
			if (ScopedProvider.Provider.SupportsInterface(UIA_TogglePatternId))
			{
				EmitPropertyChangedEvent(&ScopedProvider.Provider, UIA_ToggleToggleStatePropertyId, Args.OldValue, Args.NewValue);

			}
			else if (ScopedProvider.Provider.SupportsInterface(UIA_InvokePatternId))
			{
				UiaRaiseAutomationEvent(&ScopedProvider.Provider, UIA_Invoke_InvokedEventId);
			}
			break;
		case EAccessibleEvent::Notification:
		{
			// By right the signature should be
			// typedef HRESULT(WINAPI* UiaRaiseNotificationEventFunc)(IRawElementProviderSimple*, NotificationKind, NotificationProcessing, BSTR, BSTR);
			// But NotificationKind and NotificationProcessing are Windows 10 only enums.
			// Since WINVER only keeps track of the minimum supported version, we use
			// ints instead of the enums to ensure older versions of Windows can compile the code and Windows 10 users can use the functionality.
			typedef HRESULT(WINAPI* UiaRaiseNotificationEventFunc)(IRawElementProviderSimple*, int, int, BSTR, BSTR);
			static UiaRaiseNotificationEventFunc NotificationFunc = nullptr;
			static bool bDoOnce = true;
#if WINVER >= 0x0A00
			const int NotificationKindEnum = NotificationKind_ActionCompleted;
			const int NotificationProcessingEnum = NotificationProcessing_All;
#else
			const int NotificationKindEnum = 2; // NotificationKind_ActionCompleted
			const int NotificationProcessingEnum = 2; // NotificationProcessing_All
#endif
			if (bDoOnce)
			{
				HMODULE Handle = GetModuleHandle(TEXT("Uiautomationcore.dll"));
				bDoOnce = false;
				if (Handle)
				{
					// Cast to intermediary void* to avoid compiler warning 4191, since GetProcAddress doesn't know function arguments
					NotificationFunc = (UiaRaiseNotificationEventFunc)(void*)GetProcAddress(Handle, "UiaRaiseNotificationEvent");
				}
			}
			if (NotificationFunc)
			{
				NotificationFunc(&ScopedProvider.Provider, NotificationKindEnum, NotificationProcessingEnum, SysAllocString(*Args.NewValue.GetValue<FString>()), SysAllocString(TEXT("")));
			}
			break;
		}
		// IMPORTANT: Calling UiaRaiseStructureChangedEvent seems to raise our per-frame timing for accessibility by
		// over 10x (.04ms to .7ms, tested by clicking on the "All Classes" button in the modes panel of the Editor).
		// For now, I'm disabling this until we figure out if it's absolutely necessary.
		//case EAccessibleEvent::ParentChanged:
		//{
		//	const AccessibleWidgetId OldId = Args.OldValue.GetValue<AccessibleWidgetId>();
		//	const AccessibleWidgetId NewId = Args.NewValue.GetValue<AccessibleWidgetId>();
		//	if (OldId != IAccessibleWidget::InvalidAccessibleWidgetId)
		//	{
		//		GetWidgetProvider(WindowsApplication.GetAccessibleMessageHandler()->GetAccessibleWidgetFromId(OldId).ToSharedRef());
		//		int id[2] = { UiaAppendRuntimeId, Widget->GetId() };
		//		FScopedWidgetProvider ParentProvider(GetWidgetProvider(WindowsApplication.GetAccessibleMessageHandler()->GetAccessibleWidgetFromId(OldId).ToSharedRef()));
		//		UiaRaiseStructureChangedEvent(&ParentProvider.Provider, StructureChangeType_ChildRemoved, id, ARRAYSIZE(id));
		//	}
		//	if (NewId != IAccessibleWidget::InvalidAccessibleWidgetId)
		//	{
		//		FScopedWidgetProvider ParentProvider(GetWidgetProvider(WindowsApplication.GetAccessibleMessageHandler()->GetAccessibleWidgetFromId(NewId).ToSharedRef()));
		//		UiaRaiseStructureChangedEvent(&ParentProvider.Provider, StructureChangeType_ChildAdded, nullptr, 0);
		//	}
		//	break;
		//}
		case EAccessibleEvent::WidgetRemoved:
		{
			typedef HRESULT(WINAPI* UiaDisconnectProviderFunc)(IRawElementProviderSimple*);
			static UiaDisconnectProviderFunc DisconnectFunc = nullptr;
			static bool bDoOnce = true;
			if (bDoOnce)
			{
				bDoOnce = false;
				HMODULE Handle = GetModuleHandle(TEXT("Uiautomationcore.dll"));
				if (Handle)
				{
					// Cast to intermediary void* to avoid compiler warning 4191, since GetProcAddress doesn't know function arguments
					DisconnectFunc = (UiaDisconnectProviderFunc)(void*)GetProcAddress(Handle, "UiaDisconnectProvider");
				}
			}
			if (DisconnectFunc)
			{
				DisconnectFunc(&ScopedProvider.Provider);
			}
			break;
		}
		}
	}
}

void FWindowsUIAManager::UpdateCachedCurrentLocaleLCID()
{
	CachedCurrentLocaleLCID = FInternationalization::Get().GetCurrentLocale()->GetLCID();
	// an LCID of 0 is invalid  and and UIA won't do anything with it.
	// Get the default OS locale LCID to give to screen readers  in that case
	if (CachedCurrentLocaleLCID == 0)
	{
		CachedCurrentLocaleLCID = FInternationalization::Get().GetDefaultLocale()->GetLCID();
	}
}

void FWindowsUIAManager::RunInGameThreadBlocking(CONST TFunction<void()>& Function) const
{
		WindowsApplication.GetAccessibleMessageHandler()->RunInThread(Function, true, ENamedThreads::GameThread);
}

#if !UE_BUILD_SHIPPING
void FWindowsUIAManager::DumpAccessibilityStats() const
{
	const uint32 NumWidgetProviders = CachedWidgetProviders.Num();
	// This isn't exactly right since some ControlProviders will be TextRangeProviders, but it should be close
	const uint32 NumControlProviders = ProviderList.Num() - NumWidgetProviders;

	const SIZE_T SizeOfWidgetProvider = sizeof(FWindowsUIAWidgetProvider);
	const SIZE_T SizeOfControlProvider = sizeof(FWindowsUIAControlProvider);
	const SIZE_T SizeOfCachedWidgetProviders = CachedWidgetProviders.GetAllocatedSize();
	const SIZE_T SizeOfProviderList = ProviderList.GetAllocatedSize();
	const SIZE_T CacheSize = NumWidgetProviders * SizeOfWidgetProvider + NumControlProviders * SizeOfControlProvider + SizeOfCachedWidgetProviders + SizeOfProviderList;

	UE_LOGF(LogAccessibility, Log, "Dumping Windows accessibility stats:");
	UE_LOGF(LogAccessibility, Log, "Number of Widget Providers: %i", NumWidgetProviders);
	UE_LOGF(LogAccessibility, Log, "Number of non-Widget Providers: %i", NumControlProviders);
	UE_LOG(LogAccessibility, Log, TEXT("Size of FWindowsUIAWidgetProvider: %" SIZE_T_FMT), SizeOfWidgetProvider);
	UE_LOG(LogAccessibility, Log, TEXT("Size of FWindowsUIAControlProvider: %" SIZE_T_FMT), SizeOfControlProvider);
	UE_LOG(LogAccessibility, Log, TEXT("Size of WidgetProvider* map: %" SIZE_T_FMT), SizeOfCachedWidgetProviders);
	UE_LOG(LogAccessibility, Log, TEXT("Size of all Provider* set: %" SIZE_T_FMT), SizeOfProviderList);
	UE_LOG(LogAccessibility, Log, TEXT("Memory stored in cache: %" SIZE_T_FMT " kb"), CacheSize / 1000);
}
#endif

#undef LOCTEXT_NAMESPACE

#endif
