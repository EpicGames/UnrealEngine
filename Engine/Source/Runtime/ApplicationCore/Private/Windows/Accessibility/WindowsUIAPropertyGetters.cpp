// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_ACCESSIBILITY && UE_WINDOWS_USING_UIA

#include "Windows/Accessibility/WindowsUIAPropertyGetters.h"

#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"
#include "Misc/Variant.h"

namespace WindowsUIAPropertyGetters
{

VARIANT GetPropertyValueWindows(TSharedRef<IAccessibleWidget> AccessibleWidget, PROPERTYID WindowsPropertyId)
{
	return FVariantToWindowsVariant(GetPropertyValue(AccessibleWidget, WindowsPropertyId));
}

FVariant GetPropertyValue(TSharedRef<IAccessibleWidget> AccessibleWidget, PROPERTYID WindowsPropertyId)
{
	switch (WindowsPropertyId)
	{
	case UIA_ValueIsReadOnlyPropertyId:
	case UIA_RangeValueIsReadOnlyPropertyId:
		return AccessibleWidget->AsProperty()->IsReadOnly();
	case UIA_RangeValueLargeChangePropertyId:
	case UIA_RangeValueSmallChangePropertyId:
		return static_cast<double>(AccessibleWidget->AsProperty()->GetStepSize());
	case UIA_RangeValueMaximumPropertyId:
		return static_cast<double>(AccessibleWidget->AsProperty()->GetMaximum());
	case UIA_RangeValueMinimumPropertyId:
		return static_cast<double>(AccessibleWidget->AsProperty()->GetMinimum());
	case UIA_RangeValueValuePropertyId:
		return static_cast<double>(FCString::Atof(*AccessibleWidget->AsProperty()->GetValue()));
	case UIA_ToggleToggleStatePropertyId:
		return static_cast<int32>(AccessibleWidget->AsActivatable()->GetCheckedState() ? ToggleState_On : ToggleState_Off);
	case UIA_TransformCanMovePropertyId:
		return false;
	case UIA_TransformCanResizePropertyId:
		return false;
	case UIA_TransformCanRotatePropertyId:
		return false;
	case UIA_ValueValuePropertyId:
		return AccessibleWidget->AsProperty()->GetValue();
	case UIA_WindowCanMaximizePropertyId:
		return AccessibleWidget->AsWindow()->SupportsDisplayState(IAccessibleWindow::EWindowDisplayState::Maximize);
	case UIA_WindowCanMinimizePropertyId:
		return AccessibleWidget->AsWindow()->SupportsDisplayState(IAccessibleWindow::EWindowDisplayState::Minimize);
	case UIA_WindowIsModalPropertyId:
		return AccessibleWidget->AsWindow()->IsModal();
	case UIA_WindowIsTopmostPropertyId:
		return false;
	case UIA_WindowWindowInteractionStatePropertyId:
		return static_cast<int32>(WindowInteractionState_Running);
	case UIA_WindowWindowVisualStatePropertyId:
		switch (AccessibleWidget->AsWindow()->GetDisplayState())
		{
		case IAccessibleWindow::EWindowDisplayState::Normal:
			return static_cast<int32>(WindowVisualState_Normal);
		case IAccessibleWindow::EWindowDisplayState::Minimize:
			return static_cast<int32>(WindowVisualState_Minimized);
		case IAccessibleWindow::EWindowDisplayState::Maximize:
			return static_cast<int32>(WindowVisualState_Maximized);
		}
	}

	return FVariant();
}

VARIANT FVariantToWindowsVariant(const FVariant& Value)
{
	VARIANT OutValue;
	switch (Value.GetType())
	{
	case EVariantTypes::Bool:
		OutValue.vt = VT_BOOL;
		OutValue.boolVal = Value.GetValue<bool>() ? VARIANT_TRUE : VARIANT_FALSE;
		break;
	case EVariantTypes::Double:
		OutValue.vt = VT_R8;
		OutValue.dblVal = Value.GetValue<double>();
		break;
	case EVariantTypes::Float:
		OutValue.vt = VT_R4;
		OutValue.fltVal = Value.GetValue<float>();
		break;
	case EVariantTypes::Int32:
		OutValue.vt = VT_I4;
		// todo: verify correct behavior on different operating systems
		OutValue.intVal = Value.GetValue<int32>();
		break;
	case EVariantTypes::String:
		OutValue.vt = VT_BSTR;
		OutValue.bstrVal = SysAllocString(*Value.GetValue<FString>());
		break;
	case EVariantTypes::Empty:
	default:
		OutValue.vt = VT_EMPTY;
		break;
	}

	return OutValue;
}

}

#endif
