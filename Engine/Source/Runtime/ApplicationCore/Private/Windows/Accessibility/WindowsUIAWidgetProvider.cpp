// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_ACCESSIBILITY && UE_WINDOWS_USING_UIA

#include "Windows/Accessibility/WindowsUIAWidgetProvider.h"
#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"
#include "Windows/Accessibility/WindowsUIAControlProvider.h"
#include "Windows/Accessibility/WindowsUIAManager.h"
#include "Windows/Accessibility/WindowsUIAPropertyGetters.h"
#include "Windows/WindowsWindow.h"

DECLARE_CYCLE_STAT(TEXT("Windows Accessibility: Navigate"), STAT_AccessibilityWindowsNavigate, STATGROUP_Accessibility);
DECLARE_CYCLE_STAT(TEXT("Windows Accessibility: GetProperty"), STAT_AccessibilityWindowsGetProperty, STATGROUP_Accessibility);

#define LOCTEXT_NAMESPACE "SlateAccessibility"

/** Convert our accessible widget type to a Windows control ID */
ULONG WidgetTypeToControlType(TSharedRef<IAccessibleWidget> Widget)
{
	ULONG* Type = FWindowsUIAManager::WidgetTypeToWindowsTypeMap.Find(Widget->GetWidgetType());
	if (Type)
	{
		return *Type;
	}
	else
	{
		return UIA_CustomControlTypeId;
	}
}

/**
 * Convert our accessible widget type to a human-readable localized string
 * See https://docs.microsoft.com/en-us/windows/desktop/winauto/uiauto-automation-element-propids for rules.
 */
FString WidgetTypeToLocalizedString(TSharedRef<IAccessibleWidget> Widget)
{
	FText* Text = FWindowsUIAManager::WidgetTypeToTextMap.Find(Widget->GetWidgetType());
	if (Text)
	{
		return Text->ToString();
	}
	else
	{
		static FText CustomControlTypeName = LOCTEXT("ControlTypeCustom", "custom");
		return CustomControlTypeName.ToString();
	}
}

#if !UE_BUILD_SHIPPING
/**
* Used for debugging  GetPropertyValue() to see what the properties requested are 
* for each UIA  function that's called. 
* We wrap all of this in a Win10 check as  there are some PropertyIds that are only available from Win8 onwards.
* The if check ensures compilation on all versions but may exclude this implementation. 
*/
FString PropertyIdToString(PROPERTYID InPropertyId)
{
#if WINVER >= 0x0A00 // Win10
	static TMap < PROPERTYID, FString> PropertyIdToStringMap; 
	if (PropertyIdToStringMap.Num() == 0)
	{
#define ADDTOMAP(PropertyIdEnum) PropertyIdToStringMap.Add(PropertyIdEnum, TEXT(#PropertyIdEnum))
		ADDTOMAP(UIA_RuntimeIdPropertyId);
		ADDTOMAP(UIA_BoundingRectanglePropertyId);
		ADDTOMAP(UIA_ProcessIdPropertyId);
		ADDTOMAP(UIA_ControlTypePropertyId);
		ADDTOMAP(UIA_LocalizedControlTypePropertyId);
		ADDTOMAP(UIA_NamePropertyId);
		ADDTOMAP(UIA_AcceleratorKeyPropertyId);
		ADDTOMAP(UIA_AccessKeyPropertyId);
		ADDTOMAP(UIA_HasKeyboardFocusPropertyId);
		ADDTOMAP(UIA_IsKeyboardFocusablePropertyId);
		ADDTOMAP(UIA_IsEnabledPropertyId);
		ADDTOMAP(UIA_AutomationIdPropertyId);
		ADDTOMAP(UIA_ClassNamePropertyId);
		ADDTOMAP(UIA_HelpTextPropertyId);
		ADDTOMAP(UIA_ClickablePointPropertyId);
		ADDTOMAP(UIA_CulturePropertyId);
		ADDTOMAP(UIA_IsControlElementPropertyId);
		ADDTOMAP(UIA_IsContentElementPropertyId);
		ADDTOMAP(UIA_LabeledByPropertyId);
		ADDTOMAP(UIA_IsPasswordPropertyId);
		ADDTOMAP(UIA_NativeWindowHandlePropertyId);
		ADDTOMAP(UIA_ItemTypePropertyId);
		ADDTOMAP(UIA_IsOffscreenPropertyId);
		ADDTOMAP(UIA_OrientationPropertyId);
		ADDTOMAP(UIA_FrameworkIdPropertyId);
		ADDTOMAP(UIA_IsRequiredForFormPropertyId);
		ADDTOMAP(UIA_ItemStatusPropertyId);
		ADDTOMAP(UIA_IsDockPatternAvailablePropertyId);
		ADDTOMAP(UIA_IsExpandCollapsePatternAvailablePropertyId);
		ADDTOMAP(UIA_IsGridItemPatternAvailablePropertyId);
		ADDTOMAP(UIA_IsGridPatternAvailablePropertyId);
		ADDTOMAP(UIA_IsInvokePatternAvailablePropertyId);
		ADDTOMAP(UIA_IsMultipleViewPatternAvailablePropertyId);
		ADDTOMAP(UIA_IsRangeValuePatternAvailablePropertyId);
		ADDTOMAP(UIA_IsScrollPatternAvailablePropertyId);
		ADDTOMAP(UIA_IsScrollItemPatternAvailablePropertyId);
		ADDTOMAP(UIA_IsSelectionItemPatternAvailablePropertyId);
		ADDTOMAP(UIA_IsSelectionPatternAvailablePropertyId);
		ADDTOMAP(UIA_IsTablePatternAvailablePropertyId);
		ADDTOMAP(UIA_IsTableItemPatternAvailablePropertyId);
		ADDTOMAP(UIA_IsTextPatternAvailablePropertyId);
		ADDTOMAP(UIA_IsTogglePatternAvailablePropertyId);
		ADDTOMAP(UIA_IsTransformPatternAvailablePropertyId);
		ADDTOMAP(UIA_IsValuePatternAvailablePropertyId);
		ADDTOMAP(UIA_IsWindowPatternAvailablePropertyId);
		ADDTOMAP(UIA_ValueValuePropertyId);
		ADDTOMAP(UIA_ValueIsReadOnlyPropertyId);
		ADDTOMAP(UIA_RangeValueValuePropertyId);
		ADDTOMAP(UIA_RangeValueIsReadOnlyPropertyId);
		ADDTOMAP(UIA_RangeValueMinimumPropertyId);
		ADDTOMAP(UIA_RangeValueMaximumPropertyId);
		ADDTOMAP(UIA_RangeValueLargeChangePropertyId);
		ADDTOMAP(UIA_RangeValueSmallChangePropertyId);
		ADDTOMAP(UIA_ScrollHorizontalScrollPercentPropertyId);
		ADDTOMAP(UIA_ScrollHorizontalViewSizePropertyId);
		ADDTOMAP(UIA_ScrollVerticalScrollPercentPropertyId);
		ADDTOMAP(UIA_ScrollVerticalViewSizePropertyId);
		ADDTOMAP(UIA_ScrollHorizontallyScrollablePropertyId);
		ADDTOMAP(UIA_ScrollVerticallyScrollablePropertyId);
		ADDTOMAP(UIA_SelectionSelectionPropertyId);
		ADDTOMAP(UIA_SelectionCanSelectMultiplePropertyId);
		ADDTOMAP(UIA_SelectionIsSelectionRequiredPropertyId);
		ADDTOMAP(UIA_GridRowCountPropertyId);
		ADDTOMAP(UIA_GridColumnCountPropertyId);
		ADDTOMAP(UIA_GridItemRowPropertyId);
		ADDTOMAP(UIA_GridItemColumnPropertyId);
		ADDTOMAP(UIA_GridItemRowSpanPropertyId);
		ADDTOMAP(UIA_GridItemColumnSpanPropertyId);
		ADDTOMAP(UIA_GridItemContainingGridPropertyId);
		ADDTOMAP(UIA_DockDockPositionPropertyId);
		ADDTOMAP(UIA_ExpandCollapseExpandCollapseStatePropertyId);
		ADDTOMAP(UIA_MultipleViewCurrentViewPropertyId);
		ADDTOMAP(UIA_MultipleViewSupportedViewsPropertyId);
		ADDTOMAP(UIA_WindowCanMaximizePropertyId);
		ADDTOMAP(UIA_WindowCanMinimizePropertyId);
		ADDTOMAP(UIA_WindowWindowVisualStatePropertyId);
		ADDTOMAP(UIA_WindowWindowInteractionStatePropertyId);
		ADDTOMAP(UIA_WindowIsModalPropertyId);
		ADDTOMAP(UIA_WindowIsTopmostPropertyId);
		ADDTOMAP(UIA_SelectionItemIsSelectedPropertyId);
		ADDTOMAP(UIA_SelectionItemSelectionContainerPropertyId);
		ADDTOMAP(UIA_TableRowHeadersPropertyId);
		ADDTOMAP(UIA_TableColumnHeadersPropertyId);
		ADDTOMAP(UIA_TableRowOrColumnMajorPropertyId);
		ADDTOMAP(UIA_TableItemRowHeaderItemsPropertyId);
		ADDTOMAP(UIA_TableItemColumnHeaderItemsPropertyId);
		ADDTOMAP(UIA_ToggleToggleStatePropertyId);
		ADDTOMAP(UIA_TransformCanMovePropertyId);
		ADDTOMAP(UIA_TransformCanResizePropertyId);
		ADDTOMAP(UIA_TransformCanRotatePropertyId);
		ADDTOMAP(UIA_IsLegacyIAccessiblePatternAvailablePropertyId);
		ADDTOMAP(UIA_LegacyIAccessibleChildIdPropertyId);
		ADDTOMAP(UIA_LegacyIAccessibleNamePropertyId);
		ADDTOMAP(UIA_LegacyIAccessibleValuePropertyId);
		ADDTOMAP(UIA_LegacyIAccessibleDescriptionPropertyId);
		ADDTOMAP(UIA_LegacyIAccessibleRolePropertyId);
		ADDTOMAP(UIA_LegacyIAccessibleStatePropertyId);
		ADDTOMAP(UIA_LegacyIAccessibleHelpPropertyId);
		ADDTOMAP(UIA_LegacyIAccessibleKeyboardShortcutPropertyId);
		ADDTOMAP(UIA_LegacyIAccessibleSelectionPropertyId);
		ADDTOMAP(UIA_LegacyIAccessibleDefaultActionPropertyId);
		ADDTOMAP(UIA_AriaRolePropertyId);
		ADDTOMAP(UIA_AriaPropertiesPropertyId);
		ADDTOMAP(UIA_IsDataValidForFormPropertyId);
		ADDTOMAP(UIA_ControllerForPropertyId);
		ADDTOMAP(UIA_DescribedByPropertyId);
		ADDTOMAP(UIA_FlowsToPropertyId);
		ADDTOMAP(UIA_ProviderDescriptionPropertyId);
		ADDTOMAP(UIA_IsItemContainerPatternAvailablePropertyId);
		ADDTOMAP(UIA_IsVirtualizedItemPatternAvailablePropertyId);
		ADDTOMAP(UIA_IsSynchronizedInputPatternAvailablePropertyId);
		ADDTOMAP(UIA_OptimizeForVisualContentPropertyId);
		ADDTOMAP(UIA_IsObjectModelPatternAvailablePropertyId);
		ADDTOMAP(UIA_AnnotationAnnotationTypeIdPropertyId);
		ADDTOMAP(UIA_AnnotationAnnotationTypeNamePropertyId);
		ADDTOMAP(UIA_AnnotationAuthorPropertyId);
		ADDTOMAP(UIA_AnnotationDateTimePropertyId);
		ADDTOMAP(UIA_AnnotationTargetPropertyId);
		ADDTOMAP(UIA_IsAnnotationPatternAvailablePropertyId);
		ADDTOMAP(UIA_IsTextPattern2AvailablePropertyId);
		ADDTOMAP(UIA_StylesStyleIdPropertyId);
		ADDTOMAP(UIA_StylesStyleNamePropertyId);
		ADDTOMAP(UIA_StylesFillColorPropertyId);
		ADDTOMAP(UIA_StylesFillPatternStylePropertyId);
		ADDTOMAP(UIA_StylesShapePropertyId);
		ADDTOMAP(UIA_StylesFillPatternColorPropertyId);
		ADDTOMAP(UIA_StylesExtendedPropertiesPropertyId);
		ADDTOMAP(UIA_IsStylesPatternAvailablePropertyId);
		ADDTOMAP(UIA_IsSpreadsheetPatternAvailablePropertyId);
		ADDTOMAP(UIA_SpreadsheetItemFormulaPropertyId);
		ADDTOMAP(UIA_SpreadsheetItemAnnotationObjectsPropertyId);
		ADDTOMAP(UIA_SpreadsheetItemAnnotationTypesPropertyId);
		ADDTOMAP(UIA_IsSpreadsheetItemPatternAvailablePropertyId);
		ADDTOMAP(UIA_Transform2CanZoomPropertyId);
		ADDTOMAP(UIA_IsTransformPattern2AvailablePropertyId);
		ADDTOMAP(UIA_LiveSettingPropertyId);
		ADDTOMAP(UIA_IsTextChildPatternAvailablePropertyId);
		ADDTOMAP(UIA_IsDragPatternAvailablePropertyId);
		ADDTOMAP(UIA_DragIsGrabbedPropertyId);
		ADDTOMAP(UIA_DragDropEffectPropertyId);
		ADDTOMAP(UIA_DragDropEffectsPropertyId);
		ADDTOMAP(UIA_IsDropTargetPatternAvailablePropertyId);
		ADDTOMAP(UIA_DropTargetDropTargetEffectPropertyId);
		ADDTOMAP(UIA_DropTargetDropTargetEffectsPropertyId);
		ADDTOMAP(UIA_DragGrabbedItemsPropertyId);
		ADDTOMAP(UIA_Transform2ZoomLevelPropertyId);
		ADDTOMAP(UIA_Transform2ZoomMinimumPropertyId);
		ADDTOMAP(UIA_Transform2ZoomMaximumPropertyId);
		ADDTOMAP(UIA_FlowsFromPropertyId);
		ADDTOMAP(UIA_IsTextEditPatternAvailablePropertyId);
		ADDTOMAP(UIA_IsPeripheralPropertyId);
		ADDTOMAP(UIA_IsCustomNavigationPatternAvailablePropertyId);
		ADDTOMAP(UIA_PositionInSetPropertyId);
		ADDTOMAP(UIA_SizeOfSetPropertyId);
		ADDTOMAP(UIA_LevelPropertyId);
		ADDTOMAP(UIA_AnnotationTypesPropertyId);
		ADDTOMAP(UIA_AnnotationObjectsPropertyId);
		ADDTOMAP(UIA_LandmarkTypePropertyId);
		ADDTOMAP(UIA_LocalizedLandmarkTypePropertyId);
		ADDTOMAP(UIA_FullDescriptionPropertyId);
		ADDTOMAP(UIA_FillColorPropertyId);
		ADDTOMAP(UIA_OutlineColorPropertyId);
		ADDTOMAP(UIA_FillTypePropertyId);
		ADDTOMAP(UIA_VisualEffectsPropertyId);
		ADDTOMAP(UIA_OutlineThicknessPropertyId);
		ADDTOMAP(UIA_CenterPointPropertyId);
		ADDTOMAP(UIA_RotationPropertyId);
		ADDTOMAP(UIA_SizePropertyId);
		ADDTOMAP(UIA_IsSelectionPattern2AvailablePropertyId);
		ADDTOMAP(UIA_Selection2FirstSelectedItemPropertyId);
		ADDTOMAP(UIA_Selection2LastSelectedItemPropertyId);
		ADDTOMAP(UIA_Selection2CurrentSelectedItemPropertyId);
		ADDTOMAP(UIA_Selection2ItemCountPropertyId);
		ADDTOMAP(UIA_HeadingLevelPropertyId);
		ADDTOMAP(UIA_IsDialogPropertyId);

#undef  ADDTOMAP
	}
	return  PropertyIdToStringMap[InPropertyId];
#else
return FString();
#endif // #if Win10 
}
#endif // #if !UE_BUILD_SHIPPING

// FWindowsUIAWidgetProvider methods

FWindowsUIAWidgetProvider::FWindowsUIAWidgetProvider(FWindowsUIAManager& InManager, TSharedRef<IAccessibleWidget> InWidget)
	: FWindowsUIABaseProvider(InManager, InWidget)
{
	//UpdateCachedProperties();
}

FWindowsUIAWidgetProvider::~FWindowsUIAWidgetProvider()
{
	if (UIAManager)
	{
		UIAManager->OnWidgetProviderRemoved(Widget);
	}
}

//void FWindowsUIAWidgetProvider::UpdateCachedProperty(PROPERTYID PropertyId)
//{
//	FVariant& CachedValue = CachedPropertyValues.FindOrAdd(PropertyId);
//	FVariant CurrentValue = WindowsUIAPropertyGetters::GetPropertyValue(Widget, PropertyId);
//	if (CachedValue.IsEmpty())
//	{
//		CachedValue = CurrentValue;
//	}
//	else if (CachedValue != CurrentValue)
//	{
//		UE_LOGF(LogAccessibility, VeryVerbose, "UIA Property Changed: %i", PropertyId);
//		UiaRaiseAutomationPropertyChangedEvent(
//			static_cast<IRawElementProviderSimple*>(this), PropertyId,
//			WindowsUIAPropertyGetters::FVariantToWindowsVariant(CachedValue),
//			WindowsUIAPropertyGetters::FVariantToWindowsVariant(CurrentValue));
//
//		CachedValue = CurrentValue;
//	}
//}
//
//void FWindowsUIAWidgetProvider::UpdateCachedProperties()
//{
//	if (IsValid())
//	{
//		if (SupportsInterface(UIA_RangeValuePatternId))
//		{
//			UpdateCachedProperty(UIA_RangeValueIsReadOnlyPropertyId);
//			UpdateCachedProperty(UIA_RangeValueValuePropertyId);
//			UpdateCachedProperty(UIA_RangeValueIsReadOnlyPropertyId);
//			UpdateCachedProperty(UIA_RangeValueMinimumPropertyId);
//			UpdateCachedProperty(UIA_RangeValueMaximumPropertyId);
//			UpdateCachedProperty(UIA_RangeValueLargeChangePropertyId);
//			UpdateCachedProperty(UIA_RangeValueSmallChangePropertyId);
//		}
//
//		if (SupportsInterface(UIA_TogglePatternId))
//		{
//			UpdateCachedProperty(UIA_ToggleToggleStatePropertyId);
//		}
//
//		if (SupportsInterface(UIA_ValuePatternId))
//		{
//			UpdateCachedProperty(UIA_ValueIsReadOnlyPropertyId);
//			UpdateCachedProperty(UIA_ValueValuePropertyId);
//		}
//
//		if (SupportsInterface(UIA_WindowPatternId))
//		{
//			UpdateCachedProperty(UIA_WindowCanMaximizePropertyId);
//			UpdateCachedProperty(UIA_WindowCanMinimizePropertyId);
//			UpdateCachedProperty(UIA_WindowIsModalPropertyId);
//			UpdateCachedProperty(UIA_WindowIsTopmostPropertyId);
//			UpdateCachedProperty(UIA_WindowWindowInteractionStatePropertyId);
//			UpdateCachedProperty(UIA_WindowWindowVisualStatePropertyId);
//			UpdateCachedProperty(UIA_TransformCanMovePropertyId);
//			UpdateCachedProperty(UIA_TransformCanRotatePropertyId);
//			UpdateCachedProperty(UIA_TransformCanResizePropertyId);
//		}
//	}
//}

HRESULT STDCALL FWindowsUIAWidgetProvider::QueryInterface(REFIID riid, void** ppInterface)
{
	if (riid == __uuidof(IUnknown))
	{
		*ppInterface = static_cast<IRawElementProviderSimple*>(this);
	}
	else if (riid == __uuidof(IRawElementProviderSimple))
	{
		*ppInterface = static_cast<IRawElementProviderSimple*>(this);
	}
	else if (riid == __uuidof(IRawElementProviderFragment))
	{
		*ppInterface = static_cast<IRawElementProviderFragment*>(this);
	}
	else
	{
		*ppInterface = nullptr;
	}

	if (*ppInterface)
	{
		// QueryInterface is the one exception where we need to call AddRef without going through GetWidgetProvider().
		AddRef();
		return S_OK;
	}
	else
	{
		return E_NOINTERFACE;
	}
}

ULONG STDCALL FWindowsUIAWidgetProvider::AddRef()
{
	return FWindowsUIABaseProvider::IncrementRef();
}

ULONG STDCALL FWindowsUIAWidgetProvider::Release()
{
	return FWindowsUIABaseProvider::DecrementRef();
}

bool FWindowsUIAWidgetProvider::SupportsInterface(PATTERNID PatternId) const
{
	bool ReturnValue = false;
	UIAManager->RunInGameThreadBlocking(
		[this, &PatternId, &ReturnValue]() {
		switch (PatternId)
		{
		case UIA_InvokePatternId:
		{
			IAccessibleActivatable* Activatable = Widget->AsActivatable();
			// Toggle and Invoke are mutually exclusive
			ReturnValue = Activatable && !Activatable->IsCheckable();
			break;
		}
		case UIA_RangeValuePatternId:
		{
			IAccessibleProperty* Property = Widget->AsProperty();
			// Value and RangeValue are mutually exclusive
			ReturnValue = Property && Property->GetStepSize() > 0.0f;
			break;
		}
		case UIA_TextPatternId:
		{
			ReturnValue = Widget->AsText() != nullptr;
			break;
		}
		case UIA_TogglePatternId:
		{
			IAccessibleActivatable* Activatable = Widget->AsActivatable();
			ReturnValue = Activatable && Activatable->IsCheckable();
			break;
		}
		case UIA_ValuePatternId:
		{
			IAccessibleProperty* Property = Widget->AsProperty();
			ReturnValue = Property && FMath::IsNearlyZero(Property->GetStepSize());
			break;
		}
		case UIA_SelectionPatternId:
		{
			IAccessibleTable* Table = Widget->AsTable();
			ReturnValue = Table != nullptr;
			break;
		}
		case UIA_SelectionItemPatternId:
		{
			IAccessibleTableRow* TableRow = Widget->AsTableRow();
			ReturnValue = TableRow != nullptr;
			break;
		}
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAWidgetProvider::get_ProviderOptions(ProviderOptions* pRetVal)
{
	// ServerSideProvider means that we are creating the definition of the accessible widgets for Clients (eg screenreaders) to consume.
	// We turn off COM threading as passing the requests to Main Thread is too slow 
	// Instead, we allow accessibility requests to come in on any thread and pass them to the GenericAccessibilityTaskRunner 
	*pRetVal = static_cast<ProviderOptions>(ProviderOptions_ServerSideProvider);
	return S_OK;
}

HRESULT STDCALL FWindowsUIAWidgetProvider::GetPatternProvider(PATTERNID patternId, IUnknown** pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &patternId, &pRetVal]() {
		if (IsValid())
		{
			if (SupportsInterface(patternId))
			{
				// FWindowsUIAControlProvider implements all possible control providers that we support.
				FWindowsUIAControlProvider* ControlProvider = new FWindowsUIAControlProvider(*UIAManager, Widget);
				switch (patternId)
				{
				case UIA_InvokePatternId:
					*pRetVal = static_cast<IInvokeProvider*>(ControlProvider);
					break;
				case UIA_RangeValuePatternId:
					*pRetVal = static_cast<IRangeValueProvider*>(ControlProvider);
					break;
				case UIA_TextPatternId:
					*pRetVal = static_cast<ITextProvider*>(ControlProvider);
					break;
				case UIA_TogglePatternId:
					*pRetVal = static_cast<IToggleProvider*>(ControlProvider);
					break;
				case UIA_ValuePatternId:
					*pRetVal = static_cast<IValueProvider*>(ControlProvider);
					break;
				case UIA_SelectionPatternId:
					*pRetVal = static_cast<ISelectionProvider*>(ControlProvider);
					break;
				case UIA_SelectionItemPatternId:
					*pRetVal = static_cast<ISelectionItemProvider*>(ControlProvider);
					break;
				default:
					UE_LOGF(LogAccessibility, Error, "FWindowsUIAWidgetProvider::SupportsInterface() returned true, but was unhandled in GetPatternProvider(). PatternId = %i", patternId);
					*pRetVal = nullptr;
					ControlProvider->Release();
					break;
				}
			}
			ReturnValue = S_OK;
		}
		else
		{
			ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAWidgetProvider::GetPropertyValue(PROPERTYID propertyId, VARIANT* pRetVal)
{
	SCOPE_CYCLE_COUNTER(STAT_AccessibilityWindowsGetProperty);
	
	HRESULT ReturnValue = S_OK;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &propertyId, &pRetVal]() {
		if (!IsValid())
		{
			ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
			return;
		}
		bool bValid = true;

		// https://docs.microsoft.com/en-us/windows/desktop/winauto/uiauto-automation-element-propids
		// potential other properties:
		// - UIA_CenterPointPropertyId
		// - UIA_ControllerForPropertyId (eg. to make clicking on a label also click a checkbox)
		// - UIA_DescribedByPropertyId (same as LabeledBy? not sure)
		// - UIA_IsDialogPropertyId (doesn't seem to exist in our version of UIA)
		// - UIA_ItemStatusPropertyId (eg. busy, loading)
		// - UIA_LabeledByPropertyId (eg. to describe a checkbox from a separate label)
		// - UIA_NativeWindowHandlePropertyId (unclear if this is only for windows or all widgets)
		// - UIA_PositionInSetPropertyId (position in list of items, could be useful)
		// - UIA_SizeOfSetPropertyId (number of items in list, could be useful)
		switch (propertyId)
		{
			//case UIA_AcceleratorKeyPropertyId:
			//	pRetVal->vt = VT_BSTR;
			//	// todo: hotkey, used with IInvokeProvider
			//	pRetVal->bstrVal = SysAllocString(L"");
			//	break;
			//case UIA_AccessKeyPropertyId:
			//	pRetVal->vt = VT_BSTR;
			//	// todo: activates menu, like F in &File
			//	pRetVal->bstrVal = SysAllocString(L"");
			//	break;
			//case UIA_AutomationIdPropertyId:
			//	pRetVal->vt = VT_BSTR;
			//	// todo: an identifiable name for the widget for automation testing, used like App.Find("MainMenuBar")
			//	pRetVal->bstrVal = SysAllocString(L"");
			//	break;
		case UIA_BoundingRectanglePropertyId:
			pRetVal->vt = VT_R8 | VT_ARRAY;
			pRetVal->parray = SafeArrayCreateVector(VT_R8, 0, 4);
			if (pRetVal->parray)
			{
				FBox2D Box = Widget->GetBounds();
				LONG i = 0;
				bValid &= (SafeArrayPutElement(pRetVal->parray, &i, &Box.Min.X) != S_OK);
				i = 1;
				bValid &= (SafeArrayPutElement(pRetVal->parray, &i, &Box.Max.X) != S_OK);
				i = 2;
				bValid &= (SafeArrayPutElement(pRetVal->parray, &i, &Box.Min.Y) != S_OK);
				i = 3;
				bValid &= (SafeArrayPutElement(pRetVal->parray, &i, &Box.Max.Y) != S_OK);
			}
			else
			{
				bValid = false;
			}
			break;
		case UIA_ClassNamePropertyId:
			pRetVal->vt = VT_BSTR;
			pRetVal->bstrVal = SysAllocString(*Widget->GetClassName());
			break;
		case UIA_ControlTypePropertyId:
			pRetVal->vt = VT_I4;
			pRetVal->lVal = WidgetTypeToControlType(Widget);
			break;
		case UIA_CulturePropertyId:
			pRetVal->vt = VT_I4;
			pRetVal->lVal = UIAManager->GetCachedCurrentLocaleLCID();
			break;
		case UIA_FrameworkIdPropertyId:
			pRetVal->vt = VT_BSTR;
			// todo: figure out what goes here
			pRetVal->bstrVal = SysAllocString(*LOCTEXT("Slate", "Slate").ToString());
			break;
		case UIA_HasKeyboardFocusPropertyId:
		{
			pRetVal->vt = VT_BOOL;
			// UIA only recognizes 1 user, the primary accessible user
			pRetVal->boolVal = Widget->HasUserFocus(FGenericAccessibleUserRegistry::GetPrimaryUserIndex()) ? VARIANT_TRUE : VARIANT_FALSE;
		}
			break;
		case UIA_HelpTextPropertyId:
			pRetVal->vt = VT_BSTR;
			pRetVal->bstrVal = SysAllocString(*Widget->GetHelpText());
			break;
		case UIA_IsContentElementPropertyId:
			pRetVal->vt = VT_BOOL;
			// todo: https://docs.microsoft.com/en-us/windows/desktop/winauto/uiauto-treeoverview
			pRetVal->boolVal = VARIANT_TRUE;
			break;
		case UIA_IsControlElementPropertyId:
			pRetVal->vt = VT_BOOL;
			// todo: https://docs.microsoft.com/en-us/windows/desktop/winauto/uiauto-treeoverview
			pRetVal->boolVal = VARIANT_TRUE;
			break;
		case UIA_IsEnabledPropertyId:
			pRetVal->vt = VT_BOOL;
			pRetVal->boolVal = Widget->IsEnabled() ? VARIANT_TRUE : VARIANT_FALSE;
			break;
		case UIA_IsKeyboardFocusablePropertyId:
			pRetVal->vt = VT_BOOL;
			pRetVal->boolVal = Widget->SupportsFocus() ? VARIANT_TRUE : VARIANT_FALSE;
			break;
		case UIA_IsOffscreenPropertyId:
			pRetVal->vt = VT_BOOL;
			pRetVal->boolVal = Widget->IsHidden() ? VARIANT_TRUE : VARIANT_FALSE;
			break;
		case UIA_IsPasswordPropertyId:
			if (Widget->AsProperty())
			{
				pRetVal->vt = VT_BOOL;
				pRetVal->boolVal = Widget->AsProperty()->IsPassword() ? VARIANT_TRUE : VARIANT_FALSE;
			}
			break;
			//#if WINVER >= 0x0603 // Windows 8.1
			//	case UIA_IsPeripheralPropertyId:
			//		pRetVal->vt = VT_BOOL;
			//		// todo: see https://docs.microsoft.com/en-us/windows/desktop/winauto/uiauto-automation-element-propids for list of control types
			//		pRetVal->boolVal = VARIANT_FALSE;
			//		break;
			//#endif
			//	case UIA_ItemTypePropertyId:
			//		pRetVal->vt = VT_BSTR;
			//		// todo: friendly name of what's in a listview
			//		pRetVal->bstrVal = SysAllocString(L"");
			//		break;
			//#if WINVER >= 0x0602 // Windows 8
			//	case UIA_LiveSettingPropertyId:
			//		pRetVal->vt = VT_I4;
			//		// todo: "politeness" setting
			//		pRetVal->lVal = 0;
			//		break;
			//#endif
		case UIA_LocalizedControlTypePropertyId:
			pRetVal->vt = VT_BSTR;
			pRetVal->bstrVal = SysAllocString(*WidgetTypeToLocalizedString(Widget));
			break;
		case UIA_NamePropertyId:
			pRetVal->vt = VT_BSTR;
			// todo: slate widgets don't have names, screen reader may read this as accessible text
			pRetVal->bstrVal = SysAllocString(*Widget->GetWidgetName());
			break;
			//case UIA_OrientationPropertyId:
			//	pRetVal->vt = VT_I4;
			//	// todo: sliders, scroll bars, layouts
			//	pRetVal->lVal = OrientationType_None;
			//	break;
		case UIA_ProcessIdPropertyId:
			pRetVal->vt = VT_I4;
			pRetVal->lVal = ::GetCurrentProcessId();
			break;
			/*
		case UIA_NativeWindowHandlePropertyId:
		{
			pRetVal->vt = VT_I4;
			//@TODO: Null check the window 
			TSharedPtr<IAccessibleWidget> Window = Widget->GetWindow()	;
			if (Window.IsValid())
			{
				void* OSWindowHandle = Window->AsWindow()->GetNativeWindow()->GetOSWindowHandle();
				
				HWND WindowHandle = reinterpret_cast<HWND>(OSWindowHandle);
				//@TODO: Cast properly 
				// 64 bit and 32 bit Windows maintain interoperability by retaining 32 bit HWND handles 
				// Thus only lower 32 bits are considered, the top 32 bits can be truncated 
				pRetVal->lVal = ::PtrToLong(WindowHandle);
			}
			else
			{
				pRetVal->lVal = 0;
			}
			
		}
			break;
			*/
		default:
			pRetVal->vt = VT_EMPTY;
			break;
		}

		if (!bValid)
		{
			pRetVal->vt = VT_EMPTY;
			ReturnValue = E_FAIL;
			return;
		}

	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAWidgetProvider::get_HostRawElementProvider(IRawElementProviderSimple** pRetVal)
{
	// Only return host provider for native windows
	*pRetVal = nullptr;
	return S_OK;
}

HRESULT STDCALL FWindowsUIAWidgetProvider::Navigate(NavigateDirection direction, IRawElementProviderFragment** pRetVal)
{
	SCOPE_CYCLE_COUNTER(STAT_AccessibilityWindowsNavigate);
	HRESULT ReturnValue = S_OK;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &direction, &pRetVal]() {
		if (!IsValid())
		{
			ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
			return;
		}

		TSharedPtr<IAccessibleWidget> Relative = nullptr;
		switch (direction)
		{
		case NavigateDirection_Parent:
			Relative = Widget->GetParent();
			break;
		case NavigateDirection_NextSibling:
			Relative = Widget->GetNextSibling();
			break;
		case NavigateDirection_PreviousSibling:
			Relative = Widget->GetPreviousSibling();
			break;
		case NavigateDirection_FirstChild:
			if (Widget->GetNumberOfChildren() > 0)
			{
				Relative = Widget->GetChildAt(0);
			}
			break;
		case NavigateDirection_LastChild:
		{
			const int32 NumChildren = Widget->GetNumberOfChildren();
			if (NumChildren > 0)
			{
				Relative = Widget->GetChildAt(NumChildren - 1);
			}
			break;
		}
		}

		if (Relative.IsValid())
		{
			*pRetVal = static_cast<IRawElementProviderFragment*>(&UIAManager->GetWidgetProvider(Relative.ToSharedRef()));
		}
		else
		{
			*pRetVal = nullptr;
		}
	});
	
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAWidgetProvider::GetRuntimeId(SAFEARRAY** pRetVal)
{
	HRESULT ReturnValue = S_OK;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		if (IsValid())
		{
			int rtId[] = { UiaAppendRuntimeId, Widget->GetId() };
			*pRetVal = SafeArrayCreateVector(VT_I4, 0, 2);
			if (*pRetVal)
			{
				for (LONG i = 0; i < 2; ++i)
				{
					if (SafeArrayPutElement(*pRetVal, &i, &rtId[i]) != S_OK)
					{
						ReturnValue = E_FAIL;
						return;
					}
				}
			};
			ReturnValue = S_OK;
			return;
		}
		else
		{
			ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
			return;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAWidgetProvider::get_BoundingRectangle(UiaRect* pRetVal)
{
	HRESULT ReturnValue = S_OK;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		if (IsValid())
		{
			FBox2D Box = Widget->GetBounds();
			pRetVal->left = Box.Min.X;
			pRetVal->top = Box.Min.Y;
			pRetVal->width = Box.Max.X - Box.Min.X;
			pRetVal->height = Box.Max.Y - Box.Min.Y;
			ReturnValue = S_OK;
			return;
		}
		else
		{
			ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
			return;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAWidgetProvider::GetEmbeddedFragmentRoots(SAFEARRAY** pRetVal)
{
	// This would technically only be valid in our case for a window within a window
	*pRetVal = nullptr;
	return S_OK;
}

HRESULT STDCALL FWindowsUIAWidgetProvider::SetFocus()
{
	HRESULT ReturnValue = S_OK;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue]() {
		if (IsValid())
		{
			if (Widget->SupportsFocus())
			{
				// UIA only recognizes 1 user, the primary accessible user for the application
				Widget->SetUserFocus(FGenericAccessibleUserRegistry::GetPrimaryUserIndex());
				ReturnValue = S_OK;
				return;
			}
			else
			{
				ReturnValue = UIA_E_NOTSUPPORTED;
				return;
			}
		}
		else
		{
			ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
			return;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAWidgetProvider::get_FragmentRoot(IRawElementProviderFragmentRoot** pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		if (IsValid())
		{
			TSharedPtr<IAccessibleWidget> Window = Widget->GetWindow();
			if (Window.IsValid())
			{
				*pRetVal = static_cast<IRawElementProviderFragmentRoot*>(&static_cast<FWindowsUIAWindowProvider&>(UIAManager->GetWidgetProvider(Window.ToSharedRef())));
				ReturnValue = S_OK;
			}
		}
	});
	return ReturnValue;
}

// ~

// FWindowsUIAWindowProvider methods

FWindowsUIAWindowProvider::FWindowsUIAWindowProvider(FWindowsUIAManager& InManager, TSharedRef<IAccessibleWidget> InWidget)
	: FWindowsUIAWidgetProvider(InManager, InWidget)
{
	ensure(InWidget->AsWindow() != nullptr);
}

FWindowsUIAWindowProvider::~FWindowsUIAWindowProvider()
{
}

HRESULT STDCALL FWindowsUIAWindowProvider::QueryInterface(REFIID riid, void** ppInterface)
{
	if (riid == __uuidof(IRawElementProviderFragmentRoot))
	{
		*ppInterface = static_cast<IRawElementProviderFragmentRoot*>(this);
		AddRef();
		return S_OK;
	}
	else
	{
		return FWindowsUIAWidgetProvider::QueryInterface(riid, ppInterface);
	}
}

ULONG STDCALL FWindowsUIAWindowProvider::AddRef()
{
	return FWindowsUIABaseProvider::IncrementRef();
}

ULONG STDCALL FWindowsUIAWindowProvider::Release()
{
	return FWindowsUIABaseProvider::DecrementRef();
}

HRESULT STDCALL FWindowsUIAWindowProvider::get_HostRawElementProvider(IRawElementProviderSimple** pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		if (Widget->IsValid())
		{
			TSharedPtr<FGenericWindow> NativeWindow = Widget->AsWindow()->GetNativeWindow();
			if (NativeWindow.IsValid())
			{
				HWND Hwnd = static_cast<HWND>(NativeWindow->GetOSWindowHandle());
				if (Hwnd != nullptr)
				{
					ReturnValue = UiaHostProviderFromHwnd(Hwnd, pRetVal);
					return;
				}
			}
			ReturnValue = UIA_E_INVALIDOPERATION;
			return;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAWindowProvider::GetPatternProvider(PATTERNID patternId, IUnknown** pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &patternId, &pRetVal]() {
		if (IsValid())
		{
			switch (patternId)
			{
			case UIA_WindowPatternId:
				*pRetVal = static_cast<IWindowProvider*>(new FWindowsUIAControlProvider(*UIAManager, Widget));
				ReturnValue = S_OK;
				return;
			default:
				ReturnValue = FWindowsUIAWidgetProvider::GetPatternProvider(patternId, pRetVal);
				return;
			}
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAWindowProvider::ElementProviderFromPoint(double x, double y, IRawElementProviderFragment** pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal, &x, &y]() {
		if (IsValid())
		{
			TSharedPtr<IAccessibleWidget> Child = Widget->AsWindow()->GetChildAtPosition((int32)x, (int32)y);
			if (Child.IsValid())
			{
				*pRetVal = static_cast<IRawElementProviderFragment*>(&UIAManager->GetWidgetProvider(Child.ToSharedRef()));
			}
			else
			{
				*pRetVal = nullptr;
			}
			ReturnValue = S_OK;
			return;
		}
	});
	return ReturnValue;
}

HRESULT STDCALL FWindowsUIAWindowProvider::GetFocus(IRawElementProviderFragment** pRetVal)
{
	HRESULT ReturnValue = UIA_E_ELEMENTNOTAVAILABLE;
	UIAManager->RunInGameThreadBlocking(
		[this, &ReturnValue, &pRetVal]() {
		*pRetVal = nullptr;
		if (IsValid())
		{
			// UIA only assumes 1 user so we always assume that the primary user (the keyboard user) is using the accessibility services
			TSharedPtr<IAccessibleWidget> Focus = Widget->AsWindow()->GetUserFocusedWidget(0);
			if (Focus.IsValid())
			{
				*pRetVal = static_cast<IRawElementProviderFragment*>(&UIAManager->GetWidgetProvider(Focus.ToSharedRef()));
			}
			ReturnValue = S_OK;
			return;
		}
	});
	return ReturnValue;
}

// ~

#undef LOCTEXT_NAMESPACE

#endif
