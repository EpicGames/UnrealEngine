// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/TextStoreACP.h"
#include "Math/UnrealMathUtility.h"
#include "Containers/UnrealString.h"
#include "Math/Vector2D.h"
#include "Logging/LogCategory.h"
#include "GenericPlatform/GenericWindow.h"
#include "GenericPlatform/ITextInputMethodSystem.h"

DEFINE_LOG_CATEGORY_STATIC(LogTextStoreACP, Log, All);

#include "Windows/AllowWindowsPlatformTypes.h"
#include <OleCtl.h>
#include <tsattrs.h>

namespace
{
	bool IsFlaggedReadLocked(const DWORD Flags)
	{
		return (Flags & TS_LF_READ) == TS_LF_READ;
	}

	bool IsFlaggedReadWriteLocked(const DWORD Flags)
	{
		return (Flags & TS_LF_READWRITE) == TS_LF_READWRITE;
	}
}

FTextStoreACP::FTextStoreACP(const TSharedRef<ITextInputMethodContext>& Context)
	:	ReferenceCount(1)
	,	TextContext(Context)
	,	TSFDocumentManager(nullptr)
	,	TSFContext(nullptr)
	,	TSFEditCookie(0)
{
}

STDAPI FTextStoreACP::QueryInterface(REFIID riid, void **ppvObj)
{
	*ppvObj = nullptr;

	if( IsEqualIID(riid, IID_IUnknown) || IsEqualIID(riid, IID_ITextStoreACP) )
	{
		*ppvObj = static_cast<ITextStoreACP*>(this);
	}
	else if( IsEqualIID(riid, IID_ITfContextOwnerCompositionSink) )
	{
		*ppvObj = static_cast<ITfContextOwnerCompositionSink*>(this);
	}
	
	// Add a reference if we're (conceptually) returning a reference to our self.
	if (*ppvObj)
	{
		AddRef();
	}

	return *ppvObj ? S_OK : E_NOINTERFACE;
}

STDAPI_(ULONG) FTextStoreACP::AddRef()
{
	return ++ReferenceCount;
}

STDAPI_(ULONG) FTextStoreACP::Release()
{
	// Value copy necessary for return value - can't use member because the invoking object may have already been deleted when we return, causing undefined behavior.
	const ULONG LocalReferenceCount = --ReferenceCount;

	// Delete self upon having no more references.
	if(ReferenceCount == 0)
	{
		delete this;
	}

	return LocalReferenceCount;
}

STDAPI FTextStoreACP::AdviseSink(__RPC__in REFIID riid, __RPC__in_opt IUnknown *punk, DWORD dwMask)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("AdviseSink"));

	// punk must not be null.
	if(!punk)
	{
		return E_UNEXPECTED;
	}

	// Provided sink must be an ITextStoreACPSink.
	if( !IsEqualIID(riid, IID_ITextStoreACPSink) )
	{
		return E_INVALIDARG;
	}

	// Install sink object if we have none.
	if(!AdviseSinkObject.TextStoreACPSink)
	{
		// Provided sink must be an ITextStoreACPSink.
		if( FAILED( AdviseSinkObject.TextStoreACPSink.FromQueryInterface(IID_ITextStoreACPSink, punk) ) )
		{
			return E_UNEXPECTED;
		}
		// Must have a sink object.
		if(!AdviseSinkObject.TextStoreACPSink)
		{
			return E_UNEXPECTED;
		}
	}
	else
	{
		TComPtr<IUnknown> OurSinkId(nullptr);
		if( FAILED( OurSinkId.FromQueryInterface(IID_IUnknown, punk) ) )
		{
			
			return E_UNEXPECTED;
		}

		TComPtr<IUnknown> TheirSinkId(nullptr);
		if( FAILED( TheirSinkId.FromQueryInterface(IID_IUnknown, punk) ) )
		{
			return E_UNEXPECTED;
		}

		// Can not install additional sink, can only update existing sink.
		if(OurSinkId != TheirSinkId)
		{
			return CONNECT_E_ADVISELIMIT;
		}
	}

	// Update flags for what notifications we should broadcast back to TSF.
	AdviseSinkObject.SinkFlags = dwMask;

	return S_OK;
}

STDAPI FTextStoreACP::UnadviseSink(__RPC__in_opt IUnknown *punk)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("UnadviseSink"));

	// punk must not be null.
	if(!punk)
	{
		return E_INVALIDARG;
	}

	// Cannot unadvise sink if we have none.
	if(!AdviseSinkObject.TextStoreACPSink)
	{
		return CONNECT_E_NOCONNECTION;
	}

	TComPtr<IUnknown> OurSinkId(nullptr);
	if( FAILED( OurSinkId.FromQueryInterface(IID_IUnknown, punk) ) )
	{
		return E_UNEXPECTED;
	}

	TComPtr<IUnknown> TheirSinkId(nullptr);
	if( FAILED( TheirSinkId.FromQueryInterface(IID_IUnknown, punk) ) )
	{
		return E_UNEXPECTED;
	}

	// Can not unadvise a sink that was never advised.
	if(OurSinkId != TheirSinkId)
	{
		return CONNECT_E_NOCONNECTION;
	}

	AdviseSinkObject.TextStoreACPSink.Reset();

	return S_OK;
}

STDAPI FTextStoreACP::RequestLock(DWORD dwLockFlags, HRESULT *phrSession)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("RequestLock"));

	// Must have a sink object.
	if(!AdviseSinkObject.TextStoreACPSink)
	{
		return E_FAIL;
	}

	//  phrSession must not be null.
	if(!phrSession)
	{
		return E_INVALIDARG;
	}

	// If we have no lock, grant one.
	if(LockManager.LockType == 0)
	{
		// Flag as locked.
		UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("RequestLock - Locking..."));
		LockManager.LockType = dwLockFlags & (~TS_LF_SYNC);

		UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("RequestLock - Locked"));

		*phrSession = AdviseSinkObject.TextStoreACPSink->OnLockGranted(LockManager.LockType);

		UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("RequestLock - Unlocking..."));

		while(LockManager.IsPendingLockUpgrade)
		{
			UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("RequestLock - Upgrading..."));

			LockManager.LockType = TS_LF_READWRITE;
			LockManager.IsPendingLockUpgrade = false;

			UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("RequestLock - Upgraded"));

			AdviseSinkObject.TextStoreACPSink->OnLockGranted(LockManager.LockType);

			UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("RequestLock - Unlocking..."));
		}

		// Flag as unlocked.
		LockManager.LockType = 0;
		UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("RequestLock - Unlocked"));

		return S_OK;
	}
	// If we have a lock, we may upgrade the lock if we have a read-only lock and want an asynchronous read-write lock.
	else if( !IsFlaggedReadLocked(LockManager.LockType) && !!IsFlaggedReadWriteLocked(LockManager.LockType) && !IsFlaggedReadWriteLocked(dwLockFlags) && !(dwLockFlags & TS_LF_SYNC) )
	{
		*phrSession = TS_S_ASYNC;
		LockManager.IsPendingLockUpgrade = true;
	}
	// Already locked, can't relock.
	else
	{
		*phrSession = TS_E_SYNCHRONOUS;
		return E_FAIL;
	}

	return S_OK;
}

STDAPI FTextStoreACP::GetStatus(__RPC__out TS_STATUS *pdcs)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("GetStatus"));

	// pdcs cannot be null.
	if(!pdcs)
	{
		return E_INVALIDARG;
	}

	pdcs->dwDynamicFlags = TextContext->IsReadOnly() ? TS_SD_READONLY : 0;
	pdcs->dwStaticFlags = TS_SS_NOHIDDENTEXT;

	return S_OK;
}

STDAPI FTextStoreACP::GetEndACP(__RPC__out LONG *pacp)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("GetEndACP"));

	if(!IsFlaggedReadLocked(LockManager.LockType))
	{
		return TS_E_NOLOCK;
	}

	*pacp = TextContext->GetTextLength();

	return S_OK;
}

STDAPI FTextStoreACP::GetSelection(ULONG ulIndex, ULONG ulCount, __RPC__out_ecount_part(ulCount, *pcFetched) TS_SELECTION_ACP *pSelection, __RPC__out ULONG *pcFetched)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("GetSelection"));

	if(!IsFlaggedReadLocked(LockManager.LockType))
	{
		return TS_E_NOLOCK;
	}

	// Only supports one selection.
	if(ulIndex != TS_DEFAULT_SELECTION)
	{
		return TS_E_NOSELECTION;
	}

	*pcFetched = 0;
	ULONG SelectionCount = FMath::Min(1ul, ulCount);
	for(ULONG i = 0; i < SelectionCount; ++i)
	{
		uint32 SelectionBeginIndex;
		uint32 SelectionLength;
		ITextInputMethodContext::ECaretPosition CaretPosition;
		TextContext->GetSelectionRange(SelectionBeginIndex, SelectionLength, CaretPosition);

		pSelection[i].acpStart = SelectionBeginIndex;
		pSelection[i].acpEnd = SelectionBeginIndex + SelectionLength;
		switch(CaretPosition)
		{
		case ITextInputMethodContext::ECaretPosition::Beginning:	{ pSelection[i].style.ase = TS_AE_START;	} break;
		case ITextInputMethodContext::ECaretPosition::Ending:		{ pSelection[i].style.ase = TS_AE_END;		} break;
		}

		// TODO: Use fInterimChar appropriately?
		pSelection[i].style.fInterimChar = FALSE;

		++(*pcFetched);
	}

	return S_OK;
}

STDAPI FTextStoreACP::SetSelection(ULONG ulCount, __RPC__in_ecount_full(ulCount) const TS_SELECTION_ACP *pSelection)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("SetSelection - From %d to %d"), pSelection->acpStart, pSelection->acpEnd);

	if(!IsFlaggedReadWriteLocked(LockManager.LockType))
	{
		return TS_E_NOLOCK;
	}

	const LONG StringLength = TextContext->GetTextLength();

	unsigned int SelectionCount = FMath::Min(1ul, ulCount);

	// Validate selections.
	for(unsigned int i = 0; i < SelectionCount; ++i)
	{
		if(pSelection[i].acpStart < 0 || pSelection[i].acpStart > StringLength || pSelection[i].acpEnd < 0 || pSelection[i].acpEnd > StringLength)
		{
			return TF_E_INVALIDPOS;
		}
	}

	for(unsigned int i = 0; i < SelectionCount; ++i)
	{
		uint32 SelectionBeginIndex = pSelection[i].acpStart;
		uint32 SelectionLength = pSelection[i].acpEnd - pSelection[i].acpStart;
		ITextInputMethodContext::ECaretPosition CaretPosition = ITextInputMethodContext::ECaretPosition::Ending;
		// TODO: Use fInterimChar appropriately - means we were composing a single character?

		TextContext->SetSelectionRange(SelectionBeginIndex, SelectionLength, CaretPosition);
	}

	return S_OK;
}

STDAPI FTextStoreACP::RequestSupportedAttrs(DWORD dwFlags, ULONG cFilterAttrs, __RPC__in_ecount_full_opt(cFilterAttrs) const TS_ATTRID *paFilterAttrs)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("RequestSupportedAttrs"));

	for(ULONG i = 0; i < cFilterAttrs; ++i)
	{
		auto Predicate = [&](const FSupportedAttribute& Attribute) -> bool
		{
			return IsEqualGUID(*Attribute.Id, paFilterAttrs[i]) == TRUE;
		};

		int32 Index = SupportedAttributes.IndexOfByPredicate(Predicate);
		if(Index != INDEX_NONE)
		{
			SupportedAttributes[Index].WantsDefault = true;
		}
	}

	return S_OK;
}

STDAPI FTextStoreACP::RequestAttrsAtPosition(LONG acpPos, ULONG cFilterAttrs, __RPC__in_ecount_full_opt(cFilterAttrs) const TS_ATTRID *paFilterAttrs, DWORD dwFlags)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("RequestAttrsAtPosition"));

	for(ULONG i = 0; i < cFilterAttrs; ++i)
	{
		auto Predicate = [&](const FSupportedAttribute& Attribute) -> bool
		{
			return IsEqualGUID(*Attribute.Id, paFilterAttrs[i]) == TRUE;
		};

		int32 Index = SupportedAttributes.IndexOfByPredicate(Predicate);
		if(Index != INDEX_NONE)
		{
			SupportedAttributes[Index].WantsDefault = true;
		}
	}

	return S_OK;
}

STDAPI FTextStoreACP::RequestAttrsTransitioningAtPosition(LONG acpPos, ULONG cFilterAttrs, __RPC__in_ecount_full_opt(cFilterAttrs) const TS_ATTRID *paFilterAttrs, DWORD dwFlags)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("RequestAttrsTransitioningAtPosition"));
	return E_NOTIMPL; // TODO: No meaningful documentation to this method, not implemented.
}

STDAPI FTextStoreACP::FindNextAttrTransition(LONG acpStart, LONG acpHalt, ULONG cFilterAttrs, __RPC__in_ecount_full_opt(cFilterAttrs) const TS_ATTRID *paFilterAttrs, DWORD dwFlags, __RPC__out LONG *pacpNext, __RPC__out BOOL *pfFound, __RPC__out LONG *plFoundOffset)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("FindNextAttrTransition"));
	return E_NOTIMPL; // TODO: No meaningful documentation to this method, not implemented.
}

CA_SUPPRESS(6101)
STDAPI FTextStoreACP::RetrieveRequestedAttrs(ULONG ulCount, __RPC__out_ecount_part(ulCount, *pcFetched) TS_ATTRVAL *paAttrVals, __RPC__out ULONG *pcFetched)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("RetrieveRequestedAttrs"));

	*pcFetched = 0;

	for(int32 i = 0; i < SupportedAttributes.Num() && *pcFetched < ulCount; ++i)
	{
		if(SupportedAttributes[i].WantsDefault)
		{
			paAttrVals[i].idAttr = *(SupportedAttributes[i].Id);
			CA_SUPPRESS(6031)
			VariantCopy(&(paAttrVals[*pcFetched].varValue), &(SupportedAttributes[i].DefaultValue));
			paAttrVals[i].dwOverlapId = 0;
		}

		++(*pcFetched);
	}

	return S_OK;
}

STDAPI FTextStoreACP::GetActiveView(__RPC__out TsViewCookie *pvcView)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("GetActiveView"));
	*pvcView = 0;

	return S_OK;
}

STDAPI FTextStoreACP::GetACPFromPoint(TsViewCookie vcView, __RPC__in const POINT *pt, DWORD dwFlags, __RPC__out LONG *pacp)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("GetACPFromPoint - At (%d, %d)"), pt->x, pt->y);

	if(!IsFlaggedReadLocked(LockManager.LockType))
	{
		return TS_E_NOLOCK;
	}

	const FVector2D Point(pt->x, pt->y);

	// TODO: Implement logic based on dwFlags.
	*pacp = TextContext->GetCharacterIndexFromPoint(Point);

	return S_OK;
}

STDAPI FTextStoreACP::GetTextExt(TsViewCookie vcView, LONG acpStart, LONG acpEnd, __RPC__out RECT *prc, __RPC__out BOOL *pfClipped)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("GetTextExt - From %d to %d"), acpStart, acpEnd);

	if(!IsFlaggedReadLocked(LockManager.LockType))
	{
		return TS_E_NOLOCK;
	}

	const LONG StringLength = TextContext->GetTextLength();

	// Validate range.
	if( acpStart < 0 || acpStart > StringLength || ( acpEnd != -1 && ( acpEnd < 0 || acpEnd > StringLength) ) )
	{
		return TS_E_INVALIDPOS;
	}

	FVector2D Position;
	FVector2D Size;
	*pfClipped	=	TextContext->GetTextBounds(acpStart, acpEnd - acpStart, Position, Size);
	prc->left	=	(LONG)Position.X;
	prc->top	=	(LONG)Position.Y;
	prc->right	=	(LONG)(Position.X + Size.X);
	prc->bottom	=	(LONG)(Position.Y + Size.Y);

	return S_OK;
}

STDAPI FTextStoreACP::GetScreenExt(TsViewCookie vcView, __RPC__out RECT *prc)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("GetScreenExt"));

	// View cookie must be valid.
	if(vcView != 0)
	{
		return E_INVALIDARG;
	}

	FVector2D Position;
	FVector2D Size;
	TextContext->GetScreenBounds(Position, Size);
	prc->left	=	(LONG)Position.X;
	prc->top	=	(LONG)Position.Y;
	prc->right	=	(LONG)(Position.X + Size.X);
	prc->bottom	=	(LONG)(Position.Y + Size.Y);

	return S_OK;
}

CA_SUPPRESS(28196)
CA_SUPPRESS(6387)
STDAPI FTextStoreACP::GetWnd(TsViewCookie vcView, __RPC__deref_out_opt HWND *phwnd)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("GetWnd"));

	const TSharedPtr<FGenericWindow> GenericWindow = TextContext->GetWindow();
	*phwnd = GenericWindow.IsValid() ? reinterpret_cast<HWND>(GenericWindow->GetOSWindowHandle()) : nullptr;

	return S_OK;
}

STDAPI FTextStoreACP::GetText(LONG acpStart, LONG acpEnd, __RPC__out_ecount_part(cchPlainReq, *pcchPlainOut) WCHAR* pchPlain, ULONG cchPlainReq, __RPC__out ULONG* pcchPlainOut, __RPC__out_ecount_part(ulRunInfoReq, *pulRunInfoOut) TS_RUNINFO* prgRunInfo, ULONG ulRunInfoReq, __RPC__out ULONG* pulRunInfoOut, __RPC__out LONG* pacpNext)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("GetText - From %d to %d"), acpStart, acpEnd);

	if (!IsFlaggedReadLocked(LockManager.LockType))
	{
		return TS_E_NOLOCK;
	}

	const LONG StringLength = TextContext->GetTextLength();
	if (acpEnd == -1)
	{
		acpEnd = StringLength;
	}

	// Validate range. acpStart must be a valid position, acpEnd must be within the document, and the range must
	// not be inverted: the unsigned span arithmetic below would underflow if acpStart > acpEnd.
	if (acpStart < 0 || acpStart > StringLength || acpEnd < 0 || acpEnd > StringLength || acpStart > acpEnd)
	{
		return TF_E_INVALIDPOS;
	}

	const uint32 BeginIndex = static_cast<uint32>(acpStart);
	const uint32 RangeLength = static_cast<uint32>(acpEnd - acpStart);

	// Plain text output is bounded ONLY by the plain-text buffer: cchPlainReq is a character count, whereas
	// ulRunInfoReq counts TS_RUNINFO *structures*, not characters (see ITextStoreACP::GetText / TS_RUNINFO).
	// Capping the returned character span by ulRunInfoReq mixes those units; instead we emit a single TS_RT_PLAIN
	// run whose uCount is the character count we advance over, which is valid for any ulRunInfoReq >= 1.
	const bool bWantsPlainText = (pchPlain != nullptr && cchPlainReq > 0);

	uint32 PlainLength = 0;
	*pcchPlainOut = 0;
	if (bWantsPlainText)
	{
		FString StringInRange;
		TextContext->GetTextInRange(BeginIndex, FMath::Min<uint32>(cchPlainReq, RangeLength), StringInRange);
		PlainLength = static_cast<uint32>(StringInRange.Len());
		FMemory::Memcpy(pchPlain, *StringInRange, PlainLength * sizeof(WCHAR));
		*pcchPlainOut = PlainLength;
	}

	// The run/advance span is decoupled from the plain buffer so that a run-only request (cchPlainReq == 0,
	// pchPlain == NULL - a documented mode) still describes a run and advances pacpNext to make forward progress.
	const uint32 RunLength = bWantsPlainText ? PlainLength : RangeLength;

	*pulRunInfoOut = 0;
	if (prgRunInfo && ulRunInfoReq > 0 && RunLength > 0)
	{
		// We only edit text stripped of markup, so a single plain run suffices. TS_RUNINFO::uCount is a CHARACTER
		// count and must equal the span this call accounts for; composition IMEs (Korean, UE-372861) desync if it
		// disagrees with the returned/advanced characters.
		prgRunInfo[0].uCount = RunLength;
		prgRunInfo[0].type = TS_RT_PLAIN;
		*pulRunInfoOut = 1;
	}

	*pacpNext = static_cast<LONG>(BeginIndex + FMath::Max<uint32>(PlainLength, RunLength));

	return S_OK;
}

STDAPI FTextStoreACP::QueryInsert(LONG acpInsertStart, LONG acpInsertEnd, ULONG cch, __RPC__out LONG *pacpResultStart, __RPC__out LONG *pacpResultEnd)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("QueryInsert"));

	// Can't successfully query if there's nowhere to write a result.
	if(!pacpResultStart || !pacpResultEnd)
	{
		return E_INVALIDARG;
	}

	uint32 BeginIndex;
	uint32 Length;
	ITextInputMethodContext::ECaretPosition CaretPosition;
	TextContext->GetSelectionRange(BeginIndex, Length, CaretPosition);

	// Workaround for Microsoft IMEs that expect QueryInsert to return the current selection range (since they omit a call to GetSelection).
	*pacpResultStart = static_cast<LONG>(BeginIndex);
	*pacpResultEnd = static_cast<LONG>(BeginIndex + Length);

	return S_OK;
}

CA_SUPPRESS(6101)
STDAPI FTextStoreACP::InsertTextAtSelection(DWORD dwFlags, __RPC__in_ecount_full(cch) const WCHAR *pchText, ULONG cch, __RPC__out LONG *pacpStart, __RPC__out LONG *pacpEnd, __RPC__out TS_TEXTCHANGE *pChange)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("InsertTextAtSelection - %s"), pchText);

	// Must have a read lock if querying.
	if(dwFlags == TS_IAS_QUERYONLY && !IsFlaggedReadLocked(LockManager.LockType))
	{
		return TS_E_NOLOCK;
	}
	// Must have a read-write lock if inserting.
	else if(!IsFlaggedReadWriteLocked(LockManager.LockType))
	{
		return TS_E_NOLOCK;
	}

	// pchText must not be null if a string is expected.
	if(cch && !pchText)
	{
		return E_INVALIDARG;
	}

	FString NewString = FString::ConstructFromPtrSize(pchText, cch);

	uint32 BeginIndex;
	uint32 Length;
	ITextInputMethodContext::ECaretPosition CaretPosition;
	TextContext->GetSelectionRange(BeginIndex, Length, CaretPosition);

	if(dwFlags == TS_IAS_QUERYONLY)
	{
		// pacpStart and pacpEnd must be valid.
		if(!pacpStart || !pacpEnd)
		{
			return E_INVALIDARG;
		}

		*pacpStart = BeginIndex;
		*pacpEnd = BeginIndex + Length;

		if(pChange)
		{
			pChange->acpStart = BeginIndex;
			pChange->acpOldEnd = BeginIndex + Length;
			pChange->acpNewEnd = BeginIndex + NewString.Len();
		}
	}
	else
	{
		if(dwFlags != TS_IAS_NOQUERY && (!pacpStart || !pacpEnd))
		{
			return E_INVALIDARG;
		}

		if(!pChange)
		{
			return E_INVALIDARG;
		}

		// Some IMEs call InsertTextAtSelection text before OnStartComposition; in that case we need to call BeginComposition ourselves here to make sure things are notified in the correct order
		if(!TextContext->IsComposing())
		{
			TextContext->BeginComposition();
		}

		TextContext->SetTextInRange(BeginIndex, Length, NewString);
		TextContext->SetSelectionRange(BeginIndex + NewString.Len(), 0, ITextInputMethodContext::ECaretPosition::Ending);

		pChange->acpStart = BeginIndex;
		pChange->acpOldEnd = BeginIndex + Length;
		pChange->acpNewEnd = BeginIndex + NewString.Len();

		if(dwFlags != TS_IAS_NOQUERY)
		{
			*pacpStart = pChange->acpStart;
			*pacpEnd = pChange->acpNewEnd;
		}
	}

	return S_OK;
}

CA_SUPPRESS(6103)
STDAPI FTextStoreACP::SetText(DWORD dwFlags, LONG acpStart, LONG acpEnd, __RPC__in_ecount_full(cch) const WCHAR *pchText, ULONG cch, __RPC__out TS_TEXTCHANGE *pChange)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("SetText - From %d to %d, set %s"), acpStart, acpEnd, pchText);

	if(!IsFlaggedReadWriteLocked(LockManager.LockType))
	{
		return TS_E_NOLOCK;
	}

	const LONG StringLength = TextContext->GetTextLength();

	// Validate range.
	if(acpStart < 0 || acpStart > StringLength || acpEnd < 0 || acpEnd > StringLength)
	{
		return TF_E_INVALIDPOS;
	}

	TS_SELECTION_ACP TSSelectionACP;
	TSSelectionACP.acpStart = acpStart;
	TSSelectionACP.acpEnd = acpEnd;
	TSSelectionACP.style.fInterimChar = false; // TODO: Is this appropriate?
	TSSelectionACP.style.ase = TS_AE_END; // TODO: Is this appropriate?
	SetSelection(1, &(TSSelectionACP));

	LONG InsertionResultBegin;
	LONG InsertionResultEnd;
	InsertTextAtSelection(0, pchText, cch, &InsertionResultBegin, &InsertionResultEnd, pChange);

	return S_OK;
}

STDAPI FTextStoreACP::GetEmbedded(LONG acpPos, __RPC__in REFGUID rguidService, __RPC__in REFIID riid, __RPC__deref_out_opt IUnknown **ppunk)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("GetEmbedded"));

	if(!IsFlaggedReadLocked(LockManager.LockType))
	{
		return TS_E_NOLOCK;
	}

	return E_NOTIMPL;
}

STDAPI FTextStoreACP::GetFormattedText(LONG acpStart, LONG acpEnd, __RPC__deref_out_opt IDataObject **ppDataObject)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("GetFormattedText"));

	if(!IsFlaggedReadLocked(LockManager.LockType))
	{
		return TS_E_NOLOCK;
	}

	return E_NOTIMPL;
}

STDAPI FTextStoreACP::QueryInsertEmbedded(__RPC__in const GUID *pguidService, __RPC__in const FORMATETC *pFormatEtc, __RPC__out BOOL *pfInsertable)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("QueryInsertEmbedded"));

	return E_NOTIMPL;
}

STDAPI FTextStoreACP::InsertEmbedded(DWORD dwFlags, LONG acpStart, LONG acpEnd, __RPC__in_opt IDataObject *pDataObject, __RPC__out TS_TEXTCHANGE *pChange)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("InsertEmbedded"));

	if(!IsFlaggedReadWriteLocked(LockManager.LockType))
	{
		return TS_E_NOLOCK;
	}

	return E_NOTIMPL;
}

STDAPI FTextStoreACP::InsertEmbeddedAtSelection(DWORD dwFlags, __RPC__in_opt IDataObject *pDataObject, __RPC__out LONG *pacpStart, __RPC__out LONG *pacpEnd, __RPC__out TS_TEXTCHANGE *pChange)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("InsertEmbeddedAtSelection"));

	if(!IsFlaggedReadLocked(LockManager.LockType))
	{
		return TS_E_NOLOCK;
	}

	return E_NOTIMPL;
}

STDAPI FTextStoreACP::OnStartComposition(__RPC__in_opt ITfCompositionView *pComposition, __RPC__out BOOL *pfOk)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("OnStartComposition"));

	// Can only handle 1 composition. This is not an error or failure, however.
	if(!Composition.TSFCompositionView)
	{
		Composition.TSFCompositionView = pComposition;

		// Some IMEs call InsertTextAtSelection text before OnStartComposition; in that case we need to skip calling BeginComposition ourselves here as things things were already notified in the correct order
		if (!TextContext->IsComposing())
		{
			TextContext->BeginComposition();
		}

		*pfOk = TRUE;
	}
	else
	{
		*pfOk = FALSE;
	}

	return S_OK;
}

STDAPI FTextStoreACP::OnUpdateComposition(__RPC__in_opt ITfCompositionView *pComposition, __RPC__in_opt ITfRange *pRangeNew)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("OnUpdateComposition"));

	// Can not update without an active composition.
	if(!Composition.TSFCompositionView)
	{
		return E_UNEXPECTED;
	}

	// Specified composition must be our composition.
	if(pComposition != Composition.TSFCompositionView)
	{
		return E_UNEXPECTED;
	}

	if(pRangeNew)
	{
		TComPtr<ITfRangeACP> RangeACP;
		RangeACP.FromQueryInterface(IID_ITfRangeACP, pRangeNew);

		LONG BeginIndex;
		LONG Length;
		const auto Result = RangeACP->GetExtent(&(BeginIndex), &(Length));

		if(FAILED(Result))
		{
			return E_FAIL;
		}

		UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("OnUpdateComposition - From %d to %d"), BeginIndex, BeginIndex + Length);

		TextContext->UpdateCompositionRange(BeginIndex, Length);
	}

	return S_OK;
}

STDAPI FTextStoreACP::OnEndComposition(__RPC__in_opt ITfCompositionView *pComposition)
{
	UE_LOG_CLINKAGE(LogTextStoreACP, Verbose, TEXT("OnEndComposition"));

	// Can not update without an active composition.
	if(!Composition.TSFCompositionView)
	{
		return E_UNEXPECTED;
	}

	// Specified composition must be our composition.
	if(pComposition != Composition.TSFCompositionView)
	{
		return E_UNEXPECTED;
	}

	Composition.TSFCompositionView.Reset();

	TextContext->EndComposition();

	return S_OK;
}

#include "Windows/HideWindowsPlatformTypes.h"
