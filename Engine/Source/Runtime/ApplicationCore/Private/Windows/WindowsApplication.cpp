// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsApplication.h"
#include "Null/NullApplication.h"
#include "Containers/Queue.h"
#include "Containers/StringConv.h"
#include "CoreGlobals.h"
#include "Internationalization/Text.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/App.h"
#include "Windows/WindowsWindow.h"
#include "Windows/WindowsCursor.h"
#include "Features/IModularFeatures.h"
#include "IInputDeviceModule.h"
#include "IInputDevice.h"
#include "IHapticDevice.h"
#include "HAL/ThreadHeartBeat.h"
#include "Templates/UniquePtr.h"
#include "Windows/WindowsPlatformApplicationMisc.h"
#include "Null/NullPlatformApplicationMisc.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "Tasks/Task.h"
#include "Tasks/Pipe.h"
#include "Containers/LockFreeList.h"
#include "HAL/RunnableThread.h"
#include "GenericPlatform/InputDeviceRegistry.h"

#if WITH_EDITOR
#include "Modules/ModuleManager.h"
#endif

#if WITH_ACCESSIBILITY && UE_WINDOWS_USING_UIA
#include "Windows/Accessibility/WindowsUIAManager.h"
#include "Windows/Accessibility/WindowsUIAWidgetProvider.h"
#include <UIAutomation.h>
#endif

// Allow Windows Platform types in the entire file.
#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <ShlObj.h>
#include <objbase.h>
#include <SetupAPI.h>
#include <devguid.h>
#include <dwmapi.h>
#include <cfgmgr32.h>
#include <windowsx.h>
#include <VersionHelpers.h>
#include "winuser.h"
THIRD_PARTY_INCLUDES_END

// Platform code uses IsMaximized which is defined to IsZoomed by windowsx.h
#pragma push_macro("IsMaximized")
#undef IsMaximized

DEFINE_LOG_CATEGORY(LogWindowsDesktop);

DECLARE_CYCLE_STAT(TEXT("WindowsApplication Custom Message Handler"), STAT_CustomMessageHandler_ProcessMessage, STATGROUP_Quick);

/**
 * Hack to get around multiple mouse events being triggered for touch events.
 * Enabling this will prevent pen tablets from working since until we switch to the windows 8 sdk (and can use WM_POINTER*) events we cannot detect the difference
 */
static int32 bPreventDuplicateMouseEventsForTouch = false;

FAutoConsoleVariableRef	CVarPreventDuplicateMouseEventsForTouch(
	TEXT("Slate.PreventDuplicateMouseEventsForTouchForWindows7"),
	bPreventDuplicateMouseEventsForTouch,
	TEXT("Hack to get around multiple mouse events being triggered for touch events on Windows 7 and lower.  Enabling this will prevent pen tablets from working on windows 7 since until we switch to the windows 8 sdk (and can use WM_POINTER* events) we cannot detect the difference")
);

static bool bUsePointerMessagesForTouch = true;
FAutoConsoleVariableRef CVarUsePointerMessagesForTouch(
	TEXT("WindowsApplication.UsePointerMessagesForTouch"),
	bUsePointerMessagesForTouch,
	TEXT("Uses the WM_POINTER messages for touch input instead of WM_TOUCH. This avoids touch input generating mouse messages.")
);

static bool bUsePointerMessagesForPen = true;
FAutoConsoleVariableRef CVarUsePointerMessagesForPen(
	TEXT("WindowsApplication.UsePointerMessagesForPen"),
	bUsePointerMessagesForPen,
	TEXT("Uses the WM_POINTER messages for pen input instead of WM_TOUCH. This avoids pen input generating mouse messages.")
);

namespace UE::WindowsApplication::TouchpadSensitivity
{
	DECLARE_LOG_CATEGORY_EXTERN(LogWindowsTouchpad, Display, VeryVerbose);
	DEFINE_LOG_CATEGORY(LogWindowsTouchpad);

	static bool bForceMaxTouchpadSensitivity = false;
	FAutoConsoleVariableRef	CVarForceMaxTouchpadSensitivity(
		TEXT("WindowsApplication.ForceMaxTouchpadSensitivity"),
		bForceMaxTouchpadSensitivity,
		TEXT("Force Touchpad sensitivity to maximum in game when the app is active (aka has focus)."
			" The sensitivity setting can reject touchpad touches while typing on the theory that they are accidental, but that is not a good assumption for games."
			" The setting defaults to 'Medium'. The 'Most sensitive' setting does not reject any touches, so that is what we want.."
			" On Windows10 22H2 we force the global system setting to 'Most Sensitive', but do not write that to the registry persistently."
			" We will set it back when the app loses focus and on exit, but it may not happen in the case of a crash leaving the setting altered until a reboot."
			" On Window11 24H2 (excepting very early installs of 24H2) a new api SetMaxTouchpadSensitivity exists and we use that."
			" SetMaxTouchpadSensitivity changes the setting only for this app, so we do not need to handle clean-up."
			" When the cvar is changed a CVarSinkHandler will attempt to apply or remove max sensitivity.")
	);

	static void CVarSinkHandler();
	static FAutoConsoleVariableSink CVarSink(FConsoleCommandDelegate::CreateStatic(&CVarSinkHandler));

	static bool bDisableForceMaxTouchpadSensitivity_SetTouchpadParameters = false;
	FAutoConsoleVariableRef	CVarDisableForceMaxTouchpadSensitivity_SetTouchpadParameters(
		TEXT("WindowsApplication.DisableForceMaxTouchpadSensitivity_SetTouchpadParameters"),
		bDisableForceMaxTouchpadSensitivity_SetTouchpadParameters,
		TEXT("Disable use of the legacy SetTouchpadParameters API for ForceMaxTouchpadSensitivity.")
	);

	static bool bDisableForceMaxTouchpadSensitivity_SetMaxTouchpadSensitivity = false;
	FAutoConsoleVariableRef	CVarDisableForceMaxTouchpadSensitivity_SetMaxTouchpadSensitivity(
		TEXT("WindowsApplication.DisableForceMaxTouchpadSensitivity_SetMaxTouchpadSensitivity"),
		bDisableForceMaxTouchpadSensitivity_SetMaxTouchpadSensitivity,
		TEXT("Disable use of the new SetMaxTouchpadSensitivity API for ForceMaxTouchpadSensitivity.")
	);

	static bool bWriteSavedTouchpadSensitivityToRegistry = false;
	FAutoConsoleVariableRef	CVarWriteSavedTouchpadSensitivityToRegistry(
		TEXT("WindowsApplication.WriteSavedTouchpadSensitivityToRegistry"),
		bWriteSavedTouchpadSensitivityToRegistry,
		TEXT("If true and we are using the SetTouchpadParameters API for ForceMaxTouchpadSensitivity the saved touchpad sensitivity value will be written to the registry so that we can attempt to restore it even if the application crashes.")

	);

	static bool bRestoreTouchpadSensitivityFromRegistry = false;
	FAutoConsoleVariableRef	CVarRestoreTouchpadSensitivityFromRegistry(
		TEXT("WindowsApplication.RestoreTouchpadSensitivityFromRegistry"),
		bRestoreTouchpadSensitivityFromRegistry,
		TEXT("If true and we are using the SetTouchpadParameters API for ForceMaxTouchpadSensitivity and we find that the system is currently set to MOST_SENSITIVE and there is saved sensitivity setting previously written to the registry behave as if that is the setting we need to restore.")
	);

	enum class EForceMaxTouchpadSensitivityMode
	{
		NotCached,
		DoNothing,
		SetTouchpadParameters,
		SetMaxTouchpadSensitivity
	};

	static EForceMaxTouchpadSensitivityMode CachedForceMaxTouchpadSensitivityMode = EForceMaxTouchpadSensitivityMode::NotCached;

	typedef BOOL (*SetMaxTouchpadSensitivityProc)(_In_ BOOL enable);
	static SetMaxTouchpadSensitivityProc SetMaxTouchpadSensitivityProcCached = nullptr;

	bool SetMaxTouchpadSensitivity(bool Enable)
	{
		check(SetMaxTouchpadSensitivityProcCached);
		if (SetMaxTouchpadSensitivityProcCached)
		{
			return SetMaxTouchpadSensitivityProcCached(Enable);
		}
		return false;
	}

	bool TryCacheSetMaxTouchpadSensitivityProc()
	{
		if (void* User32Dll = GetModuleHandle(L"user32.dll"))
		{
			// We are using the very new SetMaxTouchpadSensitivity(BOOL) api function which was added to windows 11 24H2, but after its release.  So some early installs of 24H2 will be missing the function.
			// It will also be added to some version of 23H2.
			// We will need to check for existance and then check for function to be sure we can use the new API.
			// Info from Microsoft:
			//		The function was renamed to SetMaxTouchpadSensitivity(BOOL) in API review. You can check for its existence by probing user32.dll for export #2806
			//		If the API is present but disabled (due to gradual rollout over months), calling the API will fail with an error of ERROR_INVALID_STATE.
			// Notes:
			//		We are looking the function up by export ordinal because it is possible that when the name is officially exposed the signature will be different, but the ordinal function
			//		version would be maintained.
			//SetMaxTouchpadSensitivityProcCached = static_cast<SetMaxTouchpadSensitivityProc>(FPlatformProcess::GetDllExport(User32Dll, TEXT("SetMaxTouchpadSensitivity"))); // For reference, see comment above.
			if (SetMaxTouchpadSensitivityProcCached == nullptr)
			{
				// Calling GetProcAddress directly here because MAKEINTRESOURCEW (or MAKEINTRESOURCEA) gives us what is essentially an opaque handle that can be passed into certain winapi
				// functions.  We can't use it as an array of characters and convert it to ANSI as FPlatformProcess::GetDllExport would do.
				SetMaxTouchpadSensitivityProcCached = static_cast<SetMaxTouchpadSensitivityProc>((void*)::GetProcAddress((HMODULE)User32Dll, MAKEINTRESOURCEA(2806)));
			}
			if (SetMaxTouchpadSensitivityProcCached)
			{
				UE_LOGF(LogWindowsTouchpad, Log, "TryCacheSetMaxTouchpadSensitivityProc calling SetMaxTouchpadSensitivityProcCached.");
				BOOL bSuccess = SetMaxTouchpadSensitivityProcCached(false);
				if (!bSuccess)
				{
					DWORD LastError = GetLastError();
					if (LastError == ERROR_INVALID_STATE)
					{
						UE_LOGF(LogWindowsTouchpad, Display, "TryCacheSetMaxTouchpadSensitivityProc: SetMaxTouchpadSensitivity failed with Code %d ERROR_INVALID_STATE, which means it is not available in this version of windows.  We will try to use the fallback path for touchpad sensitivity, if available.", LastError);
					}
					else
					{
						UE_LOGF(LogWindowsTouchpad, Warning, "TryCacheSetMaxTouchpadSensitivityProc: SetMaxTouchpadSensitivity failed with Code %d, which is unexpected.  We will try to use the fallback path for touchpad sensitivity, if available.", LastError);
					}

					SetMaxTouchpadSensitivityProcCached = nullptr;
					return false;
				}
				else
				{
					return true;
				}
			}
		}

		return false;
	}

	void CacheForceMaxTouchpadSensitivityMode()
	{
		// Do nothing if we already cached it.
		if (CachedForceMaxTouchpadSensitivityMode != EForceMaxTouchpadSensitivityMode::NotCached)
		{
			return;
		}

		if (!bDisableForceMaxTouchpadSensitivity_SetMaxTouchpadSensitivity && FPlatformMisc::VerifyWindowsVersion(10, 0, 22631)) //WINDOWS 11 23H2+
		{
			// Windows 11 23H2 and early installs of 24H2 might be missing the new api, because it was added after the release
			// so we also have to test that the function is actually available.
			if (TryCacheSetMaxTouchpadSensitivityProc())
			{
				CachedForceMaxTouchpadSensitivityMode = EForceMaxTouchpadSensitivityMode::SetMaxTouchpadSensitivity;
				UE_LOGF(LogWindowsTouchpad, Display, "CacheForceMaxTouchpadSensitivityMode SetMaxTouchpadSensitivity");
				return;
			}
		}

		if (!bDisableForceMaxTouchpadSensitivity_SetTouchpadParameters && FPlatformMisc::VerifyWindowsVersion(10, 0, 19045)) // WINDOWS 10 22H2+
		{
			CachedForceMaxTouchpadSensitivityMode = EForceMaxTouchpadSensitivityMode::SetTouchpadParameters;
			UE_LOGF(LogWindowsTouchpad, Display, "CacheForceMaxTouchpadSensitivityMode SetTouchpadParameters");
		}
		else
		{
			CachedForceMaxTouchpadSensitivityMode = EForceMaxTouchpadSensitivityMode::DoNothing;
			UE_LOGF(LogWindowsTouchpad, Display, "CacheForceMaxTouchpadSensitivityMode DoNothing");
		}
	}

	EForceMaxTouchpadSensitivityMode GetForceMaxTouchpadSensitivityMode()
	{
		check(CachedForceMaxTouchpadSensitivityMode != EForceMaxTouchpadSensitivityMode::NotCached);

		UE_LOGF(LogWindowsTouchpad, VeryVerbose, "GetForceMaxTouchpadSensitivityMode %i", int32{EnumToUnderlyingType(CachedForceMaxTouchpadSensitivityMode)});
		return CachedForceMaxTouchpadSensitivityMode;
	}

	bool SupportsForceMaxTouchpadSensitivity()
	{
		CacheForceMaxTouchpadSensitivityMode();
		EForceMaxTouchpadSensitivityMode Mode = GetForceMaxTouchpadSensitivityMode();
		return Mode != EForceMaxTouchpadSensitivityMode::DoNothing;
	}

	// Define constants, enums, and structs so that we can use the SetTouchpadParameters api on versions of windows
	// where it was not yet public.

	// In Windows 10, version 22H2, and later versions, "version 0" of the TOUCHPAD_PARAMETERS
	// settings structure can be used to query and update the user's touchpad settings.

	// IMPORTANT: In some later versions, new settings were introduced to this structure. These
	// settings are omitted from the below definition, since there is no way to distinguish when
	// retrieving the settings whether they are supported by that OS version. To ensure that these
	// settings are preserved when updating other settings, the structure  passed with
	// SPI_SETTOUCHPADPARAMETERS should first be populated using SPI_GETTOUCHPADPARAMETERS, with the
	// desired settings updates then applied to that structure.

	// The following are from the 24H2 winuser.h public header.

	const UINT SPI_GETTOUCHPADPARAMETERS = 0x00AE;
	const UINT SPI_SETTOUCHPADPARAMETERS = 0x00AF;

	typedef enum LEGACY_TOUCHPAD_FEATURES {
		LEGACY_TOUCHPAD_FEATURE_NONE = 0x00000000,
		LEGACY_TOUCHPAD_FEATURE_ENABLE_DISABLE = 0x00000001,
		LEGACY_TOUCHPAD_FEATURE_REVERSE_SCROLL_DIRECTION = 0x00000004,
	} LEGACY_TOUCHPAD_FEATURES;

	typedef enum TOUCHPAD_SENSITIVITY_LEVEL {
		TOUCHPAD_SENSITIVITY_LEVEL_MOST_SENSITIVE = 0x00000000,
		TOUCHPAD_SENSITIVITY_LEVEL_HIGH_SENSITIVITY = 0x00000001,
		TOUCHPAD_SENSITIVITY_LEVEL_MEDIUM_SENSITIVITY = 0x00000002,
		TOUCHPAD_SENSITIVITY_LEVEL_LOW_SENSITIVITY = 0x00000003,
		TOUCHPAD_SENSITIVITY_LEVEL_LEAST_SENSITIVE = 0x00000004,
		TOUCHPAD_SENSITIVITY_LEVEL_NOT_SET = 0x0000000F, // Special value used to indicate no value has been set.
	} TOUCHPAD_SENSITIVITY_LEVEL;
	static_assert((int32)FWindowsApplication::EForceMaxTouchpadSensitivityRestorationValues::TOUCHPAD_SENSITIVITY_LEVEL_MOST_SENSITIVE == (int32)TOUCHPAD_SENSITIVITY_LEVEL::TOUCHPAD_SENSITIVITY_LEVEL_MOST_SENSITIVE);
	static_assert((int32)FWindowsApplication::EForceMaxTouchpadSensitivityRestorationValues::TOUCHPAD_SENSITIVITY_LEVEL_LEAST_SENSITIVE == (int32)TOUCHPAD_SENSITIVITY_LEVEL::TOUCHPAD_SENSITIVITY_LEVEL_LEAST_SENSITIVE);

	// This is the v0 structure for getting/setting touchpad settings prior to Win11 24H2.
	typedef struct tagTOUCHPAD_PARAMETERS_V0 {
		UINT versionNumber;
		LEGACY_TOUCHPAD_FEATURES legacyTouchpadFeatures;
		TOUCHPAD_SENSITIVITY_LEVEL sensitivityLevel;
		BOOL touchpadEnabled : 1;
		BOOL Reserved1 : 1;
		BOOL Reserved2 : 1;
		BOOL scrollDirectionReversed : 1;
		BOOL legacyTouchpadPresent : 1;
		BOOL Reserved3 : 1;
		BOOL touchpadPresent : 1;
		BOOL panEnabled : 1;
		BOOL zoomEnabled : 1;
		BOOL tapEnabled : 1;
		BOOL twoFingerTapEnabled : 1;
		BOOL Reserved4 : 21;
		UINT cursorSpeed;
		BOOL allowActiveWhenMousePresent : 1;
		BOOL rightClickZoneEnabled : 1;
		BOOL tapAndDragEnabled : 1;
		BOOL touchpadActive : 1;
		BOOL externalMousePresent : 1;
		BOOL fourFingersSupported : 1;
		BOOL mouseAccelSettingHonored : 1;
		BOOL Reserved5 : 25;
	} TOUCHPAD_PARAMETERS_V0, * PTOUCHPAD_PARAMETERS_V0;
	static_assert(sizeof(TOUCHPAD_PARAMETERS_V0) == 0x18);

	void GetRegistryStoreIdAndSection(FString& OutStoreId, FString& OutSectionName)
	{
		{
			static FString ProjectName;
			if (ProjectName.IsEmpty())
			{
				// First try to use the ProjectName field.
				if (!GConfig->GetString(TEXT("/Script/EngineSettings.GeneralProjectSettings"), TEXT("ProjectName"), ProjectName, GGameIni))
				{
					// Fallback to FApp and then just the executable name.
					// These are pretty likely to not work with a separate process crash recovery, but they may work if this executable ends up trying
					// to restore the system setting after a crash or process kill.
					ProjectName = FApp::HasProjectName() ? FApp::GetProjectName() : FPlatformProcess::ExecutableName();
				}
			}
			OutSectionName = ProjectName;
		}

		{
			static FString CompanyName;
			if (CompanyName.IsEmpty())
			{
				// First try to use the CompanyDistinguishedName CN field.
				if (GConfig->GetString(TEXT("/Script/EngineSettings.GeneralProjectSettings"), TEXT("CompanyDistinguishedName"), CompanyName, GGameIni))
				{
					const int32 CNIndex = CompanyName.Find(TEXT("CN="));
					if (CNIndex != -1)
					{
						const int32 StartIndex = CNIndex + 3;
						const int32 EndIndex = CompanyName.Find(TEXT(","), ESearchCase::IgnoreCase, ESearchDir::FromStart, StartIndex);
						const int32 Count = EndIndex - StartIndex;
						CompanyName.MidInline(StartIndex, Count);
						CompanyName.TrimEndInline();
					}
				}

				// Next try to use the CompanyName field.
				if (CompanyName.IsEmpty() && !GConfig->GetString(TEXT("/Script/EngineSettings.GeneralProjectSettings"), TEXT("CompanyName"), CompanyName, GGameIni))
				{
					CompanyName = OutSectionName;
				}

				// Finally re-use the project name we found before.
				if (CompanyName.IsEmpty())
				{
					CompanyName = OutSectionName;
				}
			}
			OutStoreId = CompanyName;
		}
	}

	void WriteSavedSensitivityToRegistry(TOUCHPAD_SENSITIVITY_LEVEL SavedSensitivity)
	{
		if (bWriteSavedTouchpadSensitivityToRegistry)
		{
			FString Value = FString::FromInt(SavedSensitivity);
			FString StoreId;
			FString SectionName;
			GetRegistryStoreIdAndSection(StoreId, SectionName);
			FPlatformMisc::SetStoredValue(
				StoreId,
				SectionName,
				TEXT("SavedTouchpadSensitivitySetting"),
				Value
			);
		}
	}

	void ReadSavedSensitivityFromRegistry(TOUCHPAD_SENSITIVITY_LEVEL& OutSavedSensitivity, bool& OutbSuccess)
	{
		if (bWriteSavedTouchpadSensitivityToRegistry)
		{
			FString OutValue;
			FString StoreId;
			FString SectionName;
			GetRegistryStoreIdAndSection(StoreId, SectionName);
			OutbSuccess = FPlatformMisc::GetStoredValue(
				StoreId,
				SectionName,
				TEXT("SavedTouchpadSensitivitySetting"),
				OutValue
			);
			int32 ReadValue = FCString::Atoi(*OutValue);
			if (ReadValue < TOUCHPAD_SENSITIVITY_LEVEL_MOST_SENSITIVE || ReadValue > TOUCHPAD_SENSITIVITY_LEVEL_LEAST_SENSITIVE)
			{
				OutbSuccess = false;
				return;
			}
			OutSavedSensitivity = static_cast<TOUCHPAD_SENSITIVITY_LEVEL>(ReadValue);
		}
	}

	void ClearSavedSensitivityFromRegistry()
	{
		if (bWriteSavedTouchpadSensitivityToRegistry)
		{
			FString StoreId;
			FString SectionName;
			GetRegistryStoreIdAndSection(StoreId, SectionName);
			FPlatformMisc::DeleteStoredValue(
				StoreId,
				SectionName,
				TEXT("SavedTouchpadSensitivitySetting")
			);
		}
	}

	static std::atomic<TOUCHPAD_SENSITIVITY_LEVEL> ForceMaxTouchpadSensitivity_SavedSensitivity = TOUCHPAD_SENSITIVITY_LEVEL_NOT_SET;
	static bool bRegistrySavedValueConsumed = false;

	FWindowsApplication::EForceMaxTouchpadSensitivityRestorationValues GetForceMaxTouchpadSensitivityRestorationValue()
	{
		CacheForceMaxTouchpadSensitivityMode();

		if (GetForceMaxTouchpadSensitivityMode() == EForceMaxTouchpadSensitivityMode::SetTouchpadParameters)
		{
			// If we have a cached value return that.
			TOUCHPAD_SENSITIVITY_LEVEL LocalSavedSensitivity = ForceMaxTouchpadSensitivity_SavedSensitivity.load();
			if (LocalSavedSensitivity != TOUCHPAD_SENSITIVITY_LEVEL_NOT_SET)
			{
				UE_LOGF(LogWindowsTouchpad, Verbose, "GetForceMaxTouchpadSensitivityRestorationValue returning SavedSensitivity %i", LocalSavedSensitivity);
				return static_cast<FWindowsApplication::EForceMaxTouchpadSensitivityRestorationValues>(LocalSavedSensitivity);
			}
			else
			{
				// If no cached value return the current system setting.
				// Get the user's touchpad settings.
				TOUCHPAD_PARAMETERS_V0 params = {};
				params.versionNumber = 0;
				BOOL bSuccess = ::SystemParametersInfo(SPI_GETTOUCHPADPARAMETERS, sizeof(params), &params, 0 /*fWinIni*/);
				if (!bSuccess)
				{
					UE_LOGF(LogWindowsTouchpad, Warning, "GetForceMaxTouchpadSensitivityRestorationValue SystemParametersInfo SPI_GETTOUCHPADPARAMETERS failed");
					return FWindowsApplication::EForceMaxTouchpadSensitivityRestorationValues::SystemCallFailed;
				}

				if (params.sensitivityLevel == TOUCHPAD_SENSITIVITY_LEVEL_MOST_SENSITIVE)
				{
					UE_LOGF(LogWindowsTouchpad, Verbose, "GetForceMaxTouchpadSensitivityRestorationValue returning RestorationNotNeeded because the system setting was already MostSensitive.");
					return FWindowsApplication::EForceMaxTouchpadSensitivityRestorationValues::RestorationNotNeeded;
				}
				else
				{
					UE_LOGF(LogWindowsTouchpad, Verbose, "GetPreExistingTouchpadSensitivity returning Current System Sensitivity %i", params.sensitivityLevel);
					return static_cast<FWindowsApplication::EForceMaxTouchpadSensitivityRestorationValues>(params.sensitivityLevel);
				}
			}
		}
		else
		{
			UE_LOGF(LogWindowsTouchpad, Verbose, "GetForceMaxTouchpadSensitivityRestorationValue returning RestorationNotNeeded");
			return FWindowsApplication::EForceMaxTouchpadSensitivityRestorationValues::RestorationNotNeeded;
		}
	}

	void OnWMSettingChangeSetTouchpadParameters()
	{
		// Note: we do not use GetForceMaxTouchpadSensitivityMode() here because in this case it is not an error if the mode has not yet been cached.
		if (bForceMaxTouchpadSensitivity && CachedForceMaxTouchpadSensitivityMode == EForceMaxTouchpadSensitivityMode::SetTouchpadParameters)
		{
			// We got the windows message telling us that the touchpad settings have changed
			// We don't know which settings have changed, but we can go ahead and use whatever the value is right now
			// as our new restore value, and also ensure that our registry saved value matches.
			// This relies on the fact that we never send the settings change message, so we are not talking to ourselves here.
			UE_LOGF(LogWindowsTouchpad, Verbose, "OnWMSettingChangeSetTouchpadParameters touchpad parameters have changed.");

			// Get the user's touchpad settings.
			TOUCHPAD_PARAMETERS_V0 params = {};
			params.versionNumber = 0;
			BOOL bSuccess = ::SystemParametersInfo(SPI_GETTOUCHPADPARAMETERS, sizeof(params), &params, 0);
			if (!bSuccess)
			{
				UE_LOGF(LogWindowsTouchpad, Warning, "OnWMSettingChangeSetTouchpadParameters SystemParametersInfo SPI_GETTOUCHPADPARAMETERS failed");
				return;
			}

			UE_LOGF(LogWindowsTouchpad, Log, "OnWMSettingChangeSetTouchpadParameters changing saved senstivity to %i", params.sensitivityLevel);
			ForceMaxTouchpadSensitivity_SavedSensitivity.store(params.sensitivityLevel);
			if (params.sensitivityLevel == TOUCHPAD_SENSITIVITY_LEVEL_MOST_SENSITIVE)
			{
				ClearSavedSensitivityFromRegistry();
			}
			else
			{
				WriteSavedSensitivityToRegistry(params.sensitivityLevel);
			}
		}
	}

	void ApplyForceMaxTouchpadSensitivity()
	{
		UE_LOGF(LogWindowsTouchpad, Verbose, "ApplyForceMaxTouchpadSensitivity Starting.");
		check(IsInGameThread());

		switch (GetForceMaxTouchpadSensitivityMode())
		{
		case EForceMaxTouchpadSensitivityMode::NotCached:
			UE_LOGF(LogWindowsTouchpad, Warning, "ApplyForceMaxTouchpadSensitivity NotCached, this is a bug");
			check(false);
			break;
		case EForceMaxTouchpadSensitivityMode::DoNothing:
			UE_LOGF(LogWindowsTouchpad, Verbose, "ApplyForceMaxTouchpadSensitivity DoNothing");
			break;
		case EForceMaxTouchpadSensitivityMode::SetTouchpadParameters:
		{
			// Note: we will get the params, change just the one, then write them back so that the reserved param block is preserved.
			UE_LOGF(LogWindowsTouchpad, Verbose, "ApplyForceMaxTouchpadSensitivity setting sensitivity to MOST_SENSITIVE");

			// Get the user's touchpad settings.
			TOUCHPAD_PARAMETERS_V0 params = {};
			params.versionNumber = 0;
			BOOL bSuccess = ::SystemParametersInfo(SPI_GETTOUCHPADPARAMETERS, sizeof(params), &params, 0);
			if (!bSuccess)
			{
				UE_LOGF(LogWindowsTouchpad, Warning, "ApplyForceMaxTouchpadSensitivity SystemParametersInfo SPI_GETTOUCHPADPARAMETERS failed");
				return;
			}

			// If we previously exited without restoring (crashed, killed in debugger, etc) the sensitivity the system's current sensitivity will already be MOST, but we may
			// have saved some other value into the registry so that we can restore it.
			// We only want to read that registry value once per run of the app.
			// So first we will mark that we did that, then we might apply it.
			if (bRestoreTouchpadSensitivityFromRegistry && !bRegistrySavedValueConsumed)
			{
				// We only want to attempt to read the registry value once.
				bRegistrySavedValueConsumed = true;

				// Only if our current sensitivity is already MOST we suspect we might have crashed and want to use the registry value as our saved sensitivity.
				if (params.sensitivityLevel == TOUCHPAD_SENSITIVITY_LEVEL_MOST_SENSITIVE)
				{
					TOUCHPAD_SENSITIVITY_LEVEL RegistrySensitivity = TOUCHPAD_SENSITIVITY_LEVEL_MOST_SENSITIVE;
					bool bReadSuccess = false;
					ReadSavedSensitivityFromRegistry(RegistrySensitivity, bReadSuccess);
					if (bReadSuccess && RegistrySensitivity != TOUCHPAD_SENSITIVITY_LEVEL_MOST_SENSITIVE)
					{
						UE_LOGF(LogWindowsTouchpad, Log, "ApplyForceMaxTouchpadSensitivity restoring saved sensitivity %i from registry, we are already in MostSensitive and do not need to change the setting now.", (int32)RegistrySensitivity);
						ForceMaxTouchpadSensitivity_SavedSensitivity.store(RegistrySensitivity);
						// The sensitivity setting is already MOST, so we are done now.
						return;
					}
				}
				else
				{
					// We are not already in MOST, so we can ignore the registry saved value and restore to the current value.
					// We already marked the registry value consumed, so we can proceed as normal.
				}
			}

			if (params.sensitivityLevel == TOUCHPAD_SENSITIVITY_LEVEL_MOST_SENSITIVE)
			{
				UE_LOGF(LogWindowsTouchpad, Log, "ApplyForceMaxTouchpadSensitivity found that sensitivity is already MOST_SENSITIVE. Doing nothing.");
				return;
			}

			// When we first try to change the sensitivity we will store the value we want to restore to.
			if (ForceMaxTouchpadSensitivity_SavedSensitivity.load() == TOUCHPAD_SENSITIVITY_LEVEL_NOT_SET)
			{
				UE_LOGF(LogWindowsTouchpad, Verbose, "ApplyForceMaxTouchpadSensitivity setting ForceMaxTouchpadSensitivity_SavedSensitivity to %i", params.sensitivityLevel);
				ForceMaxTouchpadSensitivity_SavedSensitivity.store(params.sensitivityLevel);
				WriteSavedSensitivityToRegistry(params.sensitivityLevel);
			}

			// Change the sensitivity level.
			// Leave all other bytes in the struct untouched.
			params.sensitivityLevel = TOUCHPAD_SENSITIVITY_LEVEL_MOST_SENSITIVE;

			// Do not specify SPIF_UPDATEINIFILE so that the registry is not updated.
			// We will revert this update later, and not writing to the registry is a
			// failsafe in case we fail to do so (e.g. due to a crash).
			// Do not specify SPIF_SENDCHANGE because the message will be sent to every window and then this
			// function will block until the other windows handle it.
			bSuccess = ::SystemParametersInfo(SPI_SETTOUCHPADPARAMETERS, sizeof(params), &params, 0);
			if (!bSuccess)
			{
				UE_LOGF(LogWindowsTouchpad, Warning, "ApplyForceMaxTouchpadSensitivity SystemParametersInfo SPI_SETTOUCHPADPARAMETERS failed");
				return;
			}

			// If we want to send the WM_SETTINGCHANGE message without blocking this should do it:
			//SendNotifyMessage(HWND_BROADCAST, WM_SETTINGCHANGE, SPI_SETTOUCHPADPARAMETERS, 0);

			UE_LOGF(LogWindowsTouchpad, Log, "ApplyForceMaxTouchpadSensitivity sucessfully set sensitivity to MOST_SENSITIVE");
		}
		break;
		case EForceMaxTouchpadSensitivityMode::SetMaxTouchpadSensitivity:
		{
			if (!SetMaxTouchpadSensitivity(true))
			{
				UE_LOGF(LogWindowsTouchpad, Warning, "ApplyForceMaxTouchpadSensitivity SetMaxTouchpadSensitivity Failed");
			}
			else
			{
				UE_LOGF(LogWindowsTouchpad, Log, "ApplyForceMaxTouchpadSensitivity SetMaxTouchpadSensitivity set to true");
			}
		}
		break;
		default:
			check(false);
			break;
		}
		UE_LOGF(LogWindowsTouchpad, Verbose, "ApplyForceMaxTouchpadSensitivity Completed.");
	}

	void RemoveForceMaxTouchpadSensitivity()
	{
		UE_LOGF(LogWindowsTouchpad, Verbose, "RemoveForceMaxTouchpadSensitivity Starting.");
		check(IsInGameThread());

		switch (GetForceMaxTouchpadSensitivityMode())
		{
		case EForceMaxTouchpadSensitivityMode::NotCached:
			check(false);
			break;
		case EForceMaxTouchpadSensitivityMode::DoNothing:
			UE_LOGF(LogWindowsTouchpad, Verbose, "RemoveForceMaxTouchpadSensitivity DoNothing");
			break;
		case EForceMaxTouchpadSensitivityMode::SetTouchpadParameters:
		{
			TOUCHPAD_SENSITIVITY_LEVEL LocalSavedSensitivity = ForceMaxTouchpadSensitivity_SavedSensitivity.load();

			if (LocalSavedSensitivity <= TOUCHPAD_SENSITIVITY_LEVEL_MOST_SENSITIVE || LocalSavedSensitivity > TOUCHPAD_SENSITIVITY_LEVEL_LEAST_SENSITIVE)
			{
				// No restore needed (saved value is MOST_SENSITIVE), NOT_SET, or somehow an invalid value.
				return;
			}

			// Note: we will get the params, change just the one, then write them back so that the reserved param block is preserved.
			UE_LOGF(LogWindowsTouchpad, Verbose, "RemoveForceMaxTouchpadSensitivity reverting to SavedSensitivity %i", LocalSavedSensitivity);

			// Get the user's touchpad settings.
			TOUCHPAD_PARAMETERS_V0 params = {};
			params.versionNumber = 0;
			BOOL bSuccess = ::SystemParametersInfo(SPI_GETTOUCHPADPARAMETERS, sizeof(params), &params, 0);
			if (!bSuccess)
			{
				UE_LOGF(LogWindowsTouchpad, Warning, "RemoveForceMaxTouchpadSensitivity SystemParametersInfo SPI_GETTOUCHPADPARAMETERS failed");
				return;
			}

			// Restore the saved sensitivity level.
			// Leave all other bytes in the struct untouched.
			params.sensitivityLevel = LocalSavedSensitivity;

			// Do not specify SPIF_UPDATEINIFILE so that the registry is not updated.
			// We will revert this update later, and not writing to the registry is a
			// failsafe in case we fail to do so (e.g. due to a crash).
			// Do not specify SPIF_SENDCHANGE because the message will be sent to every window and then this
			// function will block until the other windows handle it.
			bSuccess = ::SystemParametersInfo(SPI_SETTOUCHPADPARAMETERS, sizeof(params), &params, 0);
			if (!bSuccess)
			{
				UE_LOGF(LogWindowsTouchpad, Warning, "RemoveForceMaxTouchpadSensitivity SystemParametersInfo SPI_SETTOUCHPADPARAMETERS failed");
				return;
			}

			// If we want to send the WM_SETTINGCHANGE message without blocking this should do it:
			//SendNotifyMessage(HWND_BROADCAST, WM_SETTINGCHANGE, SPI_SETTOUCHPADPARAMETERS, 0);

			// We have removed our sensitivity change, so we can clear the registry saved value.
			ClearSavedSensitivityFromRegistry();

			UE_LOGF(LogWindowsTouchpad, Log, "RemoveForceMaxTouchpadSensitivity sucessfully reverted to SavedSensitivity %i", LocalSavedSensitivity);

		}
		break;
		case EForceMaxTouchpadSensitivityMode::SetMaxTouchpadSensitivity:
		{
			if (!SetMaxTouchpadSensitivity(false))
			{
				UE_LOGF(LogWindowsTouchpad, Warning, "RemoveForceMaxTouchpadSensitivity SetMaxTouchpadSensitivity Failed");
			}
			else
			{
				UE_LOGF(LogWindowsTouchpad, Log, "ApplyForceMaxTouchpadSensitivity SetMaxTouchpadSensitivity set to false");
			}		
		}
		break;
		default:
			check(false);
			break;
		}
		UE_LOGF(LogWindowsTouchpad, Verbose, "RemoveForceMaxTouchpadSensitivity Completed.");
	}

	// Note: this function could be called from threads other than the main game thread.
	// If an async change task is already running it is possible for us to end up in a bad state, but I don't want to add locks or waits to this case
	// should still be available.
	void RemoveForceMaxTouchpadSensitivityOnErrorShutdown()
	{
		if (bForceMaxTouchpadSensitivity)
		{
			UE_LOGF(LogWindowsTouchpad, Verbose, "RemoveForceMaxTouchpadSensitivityOnErrorShutdown Starting.");

			TOUCHPAD_SENSITIVITY_LEVEL LocalSavedSensitivity = ForceMaxTouchpadSensitivity_SavedSensitivity.load();

			if (LocalSavedSensitivity <= TOUCHPAD_SENSITIVITY_LEVEL_MOST_SENSITIVE || LocalSavedSensitivity > TOUCHPAD_SENSITIVITY_LEVEL_LEAST_SENSITIVE)
			{
				// No restore needed, NOT_SET, or somehow an invalid value.
				return;
			}

			// At this point we have a saved sensitivity value that it makes sense to restore.

			// Note: we will get the params, change just the one, then write them back so that the reserved param block is preserved.
			UE_LOGF(LogWindowsTouchpad, Verbose, "RemoveForceMaxTouchpadSensitivityOnErrorShutdown reverting to SavedSensitivity %i", LocalSavedSensitivity);

			// Get the user's touchpad settings.
			TOUCHPAD_PARAMETERS_V0 params = {};
			params.versionNumber = 0;
			BOOL bSuccess = ::SystemParametersInfo(SPI_GETTOUCHPADPARAMETERS, sizeof(params), &params, 0 /*fWinIni*/);
			if (!bSuccess)
			{
				UE_LOGF(LogWindowsTouchpad, Warning, "RemoveForceMaxTouchpadSensitivityOnErrorShutdown SystemParametersInfo SPI_GETTOUCHPADPARAMETERS failed");
				return;
			}

			// Restore the saved sensitivity level.
			// Leave all other bytes in the struct untouched.
			params.sensitivityLevel = LocalSavedSensitivity;

			// Do not specify SPIF_UPDATEINIFILE so that the registry is not updated.
			// We will revert this update later, and not writing to the registry is a
			// failsafe in case we fail to do so (e.g. due to a crash).
			bSuccess = ::SystemParametersInfo(SPI_SETTOUCHPADPARAMETERS, sizeof(params), &params, 0);
			if (!bSuccess)
			{
				UE_LOGF(LogWindowsTouchpad, Warning, "RemoveForceMaxTouchpadSensitivityOnErrorShutdown SystemParametersInfo SPI_SETTOUCHPADPARAMETERS failed");
				return;
			}
			UE_LOGF(LogWindowsTouchpad, Log, "RemoveForceMaxTouchpadSensitivityOnErrorShutdown sucessfully reverted to SavedSensitivity %i", LocalSavedSensitivity);
		}
		UE_LOGF(LogWindowsTouchpad, Verbose, "RemoveForceMaxTouchpadSensitivityOnErrorShutdown Completed.");
	}

	// Note that the other thee app event hooks only need to do anything in EForceMaxTouchpadSensitivityMode::SetTouchpadParameters
	void HandleStartupApp()
	{
		UE_LOGF(LogWindowsTouchpad, Verbose, "HandleStartupApp");
		CacheForceMaxTouchpadSensitivityMode();
		if (bForceMaxTouchpadSensitivity)
		{
			ApplyForceMaxTouchpadSensitivity();
		}
	}

	void HandleActivateApp( bool bActivate)
	{
		if (bForceMaxTouchpadSensitivity && (GetForceMaxTouchpadSensitivityMode() == EForceMaxTouchpadSensitivityMode::SetTouchpadParameters))
		{
			UE_LOGF(LogWindowsTouchpad, Verbose, "HandleActivateApp %i", bActivate);
			if (bActivate)
			{
				ApplyForceMaxTouchpadSensitivity();
			}
			else
			{
				RemoveForceMaxTouchpadSensitivity();
			}
		}
	}

	void HandleDestroyApplication()
	{
		if (bForceMaxTouchpadSensitivity && (GetForceMaxTouchpadSensitivityMode() == EForceMaxTouchpadSensitivityMode::SetTouchpadParameters))
		{
			UE_LOGF(LogWindowsTouchpad, Verbose, "HandleDestroyApplication");
			RemoveForceMaxTouchpadSensitivity();
		}
	}

	void HandleShutDownAfterError()
	{
		if (bForceMaxTouchpadSensitivity && (GetForceMaxTouchpadSensitivityMode() == EForceMaxTouchpadSensitivityMode::SetTouchpadParameters))
		{
			UE_LOGF(LogWindowsTouchpad, Verbose, "HandleShutDownAfterError");
			RemoveForceMaxTouchpadSensitivityOnErrorShutdown();
		}
	}

	void CVarSinkHandler()
	{
		check(IsInGameThread());

		static bool CachedbForceMaxTouchpadSensitivity = false;
		bool ForceMaxTouchpadSensitivity = bForceMaxTouchpadSensitivity;

		if (bForceMaxTouchpadSensitivity != CachedbForceMaxTouchpadSensitivity)
		{
			CachedbForceMaxTouchpadSensitivity = bForceMaxTouchpadSensitivity;

			// If the cvar value has been changed we need to put it into effect.
			if (bForceMaxTouchpadSensitivity)
			{
				UE_LOGF(LogWindowsTouchpad, Verbose, "WindowsApplication.ForceMaxTouchpadSensitivity CVarSinkHandler calling ApplyForceMaxTouchpadSensitivity");
				CacheForceMaxTouchpadSensitivityMode();
				ApplyForceMaxTouchpadSensitivity();
			}
			else
			{
				UE_LOGF(LogWindowsTouchpad, Verbose, "WindowsApplication.ForceMaxTouchpadSensitivity CVarSinkHandler calling RemoveForceMaxTouchpadSensitivity");
				RemoveForceMaxTouchpadSensitivity();
			}
		}
	}
}

#if !UE_BUILD_SHIPPING
static int32 EnableRawInputSimulationOverRDP = true;
FAutoConsoleVariableRef	CVarEnableRawInputSimulationOverRDP(
	TEXT("Slate.EnableRawInputSimulationOverRDP"),
	EnableRawInputSimulationOverRDP,
	TEXT("")
);

static int32 ForceRawInputSimulation = false;
FAutoConsoleVariableRef	CVarForceRawInputSimulation(
	TEXT("Slate.ForceRawInputSimulation"),
	ForceRawInputSimulation,
	TEXT("")
);
#else
static int32 ForceRawInputSimulation = false;
static int32 EnableRawInputSimulationOverRDP = false;
#endif

static bool bEnableWindowMessageProfilerScopes = false;
FAutoConsoleVariableRef CVarEnableWindowMessageProfilerScopes(
	TEXT("WindowsApplication.EnableWindowMessageProfilerScopes"),
	bEnableWindowMessageProfilerScopes,
	TEXT("Enable profiler scopes for window message handling on a per-message basis and for custom message handlers.")
);

/* 
* Enabling first touch event will prevent small pop on some touch input devices.
*/
static bool bEnableFirstTouchEvent = false;
FAutoConsoleVariableRef CVarEnableFirstTouchEvent(
	TEXT("WindowsApplication.EnableFirstTouchEvent"),
	bEnableFirstTouchEvent,
	TEXT("Enable FirstTouch Event which prevents small pop on some touch input devices")
);

const FIntPoint FWindowsApplication::MinimizedWindowPosition(-32000,-32000);

FWindowsApplication* WindowsApplication = nullptr;

namespace UE::WindowsApplication::RawInputThread
{
	DECLARE_LOG_CATEGORY_EXTERN(LogWindowsRawInputThread, Display, VeryVerbose);
	DEFINE_LOG_CATEGORY(LogWindowsRawInputThread);

	constexpr uint32 WM_UE_RAWINPUT_QUIT = WM_APP;
	constexpr uint32 WM_UE_RAWINPUT_REGISTER = WM_APP + 1;
	constexpr uint32 WM_UE_RAWINPUT_UNREGISTER = WM_APP + 2;

	class FWindowsRawInputRunnable : public FRunnable
	{
		// Somewhat arbitrary
		static constexpr int32 RawInputBlockCount = 16;

		static FRunnableThread* Thread;
		static bool bInitialized;
		static HINSTANCE InstanceHandle;
		static HANDLE CreateWindowEvent;
		static HANDLE UnregisterEvent;

		HWND WindowHandle = NULL;

		TQueue<RAWMOUSE> MouseInputs;
		std::atomic<uint32> InputCount = 0;

		bool bRegistered = false;

		FWindowsRawInputRunnable() = default;

		virtual uint32 Run() override;
		virtual void Stop() override;

		static LRESULT CALLBACK WndProc(HWND hwnd, uint32 msg, WPARAM wParam, LPARAM lParam);
		void HandleRegister();
		void HandleUnregister();
		void HandleInputMessage(HRAWINPUT InputHandle);

	public:
		static void Initialize(HINSTANCE HInstance);
		static void Shutdown();

		static void Register();
		static void Unregister();
		static uint32 TakeMouseInputCount();
		static void PopMouseInput(RAWMOUSE& Mouse);

	private:
		static FWindowsRawInputRunnable& GetInstance()
		{
			static FWindowsRawInputRunnable Singleton;
			return Singleton;
		}
	};

	static bool bUseWorkerThreadForRawInput = false;
	FAutoConsoleVariableRef	CVarUseWorkerThreadForRawInput(
		TEXT("WindowsApplication.UseWorkerThreadForRawInput"),
		bUseWorkerThreadForRawInput,
		TEXT("Use a worker thread for raw input messages to remove processing from game thread."
			" Mice with high polling rates can take multiple milliseconds per frame even on fast CPUs."
			" When the cvar is changed a CVarSinkHandler will attempt to switch the thread used for handling inputs.")
	);

	static bool ShouldUseWorkerThread()
	{
		return bUseWorkerThreadForRawInput && FPlatformProcess::SupportsMultithreading();
	}

	FRunnableThread* FWindowsRawInputRunnable::Thread;
	bool FWindowsRawInputRunnable::bInitialized = false;
	HINSTANCE FWindowsRawInputRunnable::InstanceHandle = NULL;
	HANDLE FWindowsRawInputRunnable::CreateWindowEvent = NULL;
	HANDLE FWindowsRawInputRunnable::UnregisterEvent = NULL;

	uint32 FWindowsRawInputRunnable::Run()
	{
		WNDCLASS wc{};
		wc.style = 0;
		wc.lpfnWndProc = &WndProc;
		wc.cbClsExtra = 0;
		wc.cbWndExtra = 0;
		wc.hInstance = InstanceHandle;
		wc.hIcon = NULL;
		wc.hCursor = NULL;
		wc.hbrBackground = NULL;
		wc.lpszMenuName = NULL;
		wc.lpszClassName = L"UnrealRawInputWindow";
		if (RegisterClassW(&wc))
		{
			WindowHandle = CreateWindowW(wc.lpszClassName, wc.lpszClassName, 0, 0, 0, 0, 0, HWND_MESSAGE, NULL, InstanceHandle, NULL);
			if (WindowHandle)
			{
				SetEvent(CreateWindowEvent);

				MSG Msg;
				BOOL bRet;
				while ((bRet = GetMessageW(&Msg, NULL, 0, 0)) != 0)
				{
					if (bRet != -1)
					{
						TranslateMessage(&Msg);
						DispatchMessage(&Msg);
					}
				}

				HandleUnregister();

				DestroyWindow(WindowHandle);
				WindowHandle = NULL;
			}
			else
			{
				UE_LOGF(LogWindowsDesktop, Error, "Could not CreateWindowW for the raw input thread. Error code: %u", GetLastError());
			}

			UnregisterClassW(wc.lpszClassName, InstanceHandle);
		}
		else
		{
			UE_LOGF(LogWindowsDesktop, Error, "Could not RegisterClassW for the raw input thread. Error code: %u", GetLastError());
		}

		// Unblocks game thread on failure, redundant on success
		SetEvent(CreateWindowEvent);

		return 0;
	}

	void FWindowsRawInputRunnable::Stop()
	{
		SendMessageW(WindowHandle, WM_UE_RAWINPUT_QUIT, 0, 0);
	}

	LRESULT CALLBACK FWindowsRawInputRunnable::WndProc(HWND hWnd, uint32 Msg, WPARAM wParam, LPARAM lParam)
	{
		switch (Msg)
		{
		case WM_INPUT:
			FWindowsRawInputRunnable::GetInstance().HandleInputMessage((HRAWINPUT)lParam);
			// RIM_INPUTSINK should not appear since we don't use RIDEV_INPUTSINK
			// Call DefWindowProc conditionally for RIM_INPUT if this assert trips somehow though
			check(GET_RAWINPUT_CODE_WPARAM(wParam) == RIM_INPUT);
			// THe documentation says DefWindowProc is needed for RIM_INPUT cleanup
			// The OS may be forgiving but don't rely on that
			DefWindowProc(hWnd, Msg, wParam, lParam);
			// Ignore DefWindowProc return value since the documentation says to return 0 if the application handles the message
			return 0;
		case WM_UE_RAWINPUT_QUIT:
			PostQuitMessage(0);
			return 0;
		case WM_UE_RAWINPUT_REGISTER:
			FWindowsRawInputRunnable::GetInstance().HandleRegister();
			return 0;
		case WM_UE_RAWINPUT_UNREGISTER:
			FWindowsRawInputRunnable::GetInstance().HandleUnregister();
			return 0;
		default:;
		}

		return DefWindowProc(hWnd, Msg, wParam, lParam);
	}

	void FWindowsRawInputRunnable::HandleRegister()
	{
		if (!bRegistered)
		{
			RAWINPUTDEVICE RawInputDevice;
			RawInputDevice.usUsagePage = 0x01; // HID_USAGE_PAGE_GENERIC
			RawInputDevice.usUsage = 0x02; // HID_USAGE_GENERIC_MOUSE
			RawInputDevice.dwFlags = RIDEV_NOLEGACY;
			RawInputDevice.hwndTarget = WindowHandle;
			bRegistered = ::RegisterRawInputDevices(&RawInputDevice, 1, sizeof(RAWINPUTDEVICE)) == TRUE;
			UE_LOGF(LogWindowsRawInputThread, Verbose, "WindowsApplication.UseWorkerThreadForRawInput CVarSinkHandler registering worker thread for input: bRegistered=%d", (int)bRegistered);
		}
	}

	void FWindowsRawInputRunnable::HandleUnregister()
	{
		if (bRegistered)
		{
			RAWINPUTDEVICE RawInputDevice;
			RawInputDevice.usUsagePage = 0x01; // HID_USAGE_PAGE_GENERIC
			RawInputDevice.usUsage = 0x02; // HID_USAGE_GENERIC_MOUSE
			RawInputDevice.dwFlags = RIDEV_REMOVE;
			RawInputDevice.hwndTarget = NULL;
			bRegistered = ::RegisterRawInputDevices(&RawInputDevice, 1, sizeof(RAWINPUTDEVICE)) != TRUE;
			UE_LOGF(LogWindowsRawInputThread, Verbose, "WindowsApplication.UseWorkerThreadForRawInput CVarSinkHandler unregistering worker thread for input: bRegistered=%d", (int)bRegistered);
		}

		SetEvent(UnregisterEvent);
	}

	void FWindowsRawInputRunnable::HandleInputMessage(HRAWINPUT InputHandle)
	{
		uint32 Size = 0;
		if (::GetRawInputData(InputHandle, RID_INPUT, NULL, &Size, sizeof(RAWINPUTHEADER)) == 0)
		{
			void* const RawData = FMemory_Alloca(Size);

			if (::GetRawInputData(InputHandle, RID_INPUT, RawData, &Size, sizeof(RAWINPUTHEADER)) == Size)
			{
				const RAWINPUT* const Raw = static_cast<RAWINPUT*>(RawData);
				MouseInputs.Enqueue(Raw->data.mouse);
				++InputCount;
			}
		}
	}

	void FWindowsRawInputRunnable::Initialize(HINSTANCE HInstance)
	{
		ensure(IsInGameThread());

		if (!bInitialized)
		{
			check(Thread == nullptr);

			CreateWindowEvent = CreateEventW(NULL, TRUE, FALSE, NULL);
			UnregisterEvent = CreateEventW(NULL, FALSE, FALSE, NULL);
			if (CreateWindowEvent && UnregisterEvent)
			{
				InstanceHandle = HInstance;

				// Imitate DirectInput which also uses THREAD_PRIORITY_HIGHEST for its input thread
				FWindowsRawInputRunnable& Runnable = GetInstance();
				Thread = FRunnableThread::Create(&Runnable, TEXT("WindowsRawInputThread"), 0, TPri_Highest);

				// Don't read WindowHandle until this event clears
				WaitForSingleObject(CreateWindowEvent, INFINITE);

				if (Runnable.WindowHandle)
				{
					bInitialized = true;
				}
				else
				{
					CloseHandle(UnregisterEvent);
					UnregisterEvent = NULL;
					CloseHandle(CreateWindowEvent);
					CreateWindowEvent = NULL;
				}
			}
			else
			{
				UE_LOGF(LogWindowsDesktop, Error, "Could not events for the raw input thread. Error code: %u", GetLastError());

				if (UnregisterEvent)
				{
					CloseHandle(UnregisterEvent);
					UnregisterEvent = NULL;
				}

				if (CreateWindowEvent)
				{
					CloseHandle(CreateWindowEvent);
					CreateWindowEvent = NULL;
				}
			}
		}
	}

	void FWindowsRawInputRunnable::Shutdown()
	{
		ensure(IsInGameThread());

		if (bInitialized)
		{
			bInitialized = false;

			if (Thread)
			{
				Thread->Kill(true);
				delete Thread;
				Thread = nullptr;
			}

			CloseHandle(CreateWindowEvent);
			CreateWindowEvent = NULL;

			InstanceHandle = NULL;
		}
	}

	void FWindowsRawInputRunnable::Register()
	{
		ensure(IsInGameThread());

		SendMessageW(GetInstance().WindowHandle, WM_UE_RAWINPUT_REGISTER, 0, 0);
	}

	void FWindowsRawInputRunnable::Unregister()
	{
		ensure(IsInGameThread());

		SendMessageW(GetInstance().WindowHandle, WM_UE_RAWINPUT_UNREGISTER, 0, 0);
		WaitForSingleObject(UnregisterEvent, INFINITE);
	}

	uint32 FWindowsRawInputRunnable::TakeMouseInputCount()
	{
		return Thread ? GetInstance().InputCount.exchange(0) : 0;
	}

	void FWindowsRawInputRunnable::PopMouseInput(RAWMOUSE& Mouse)
	{
		ensure(IsInGameThread());
		GetInstance().MouseInputs.Dequeue(Mouse);
	}
}

namespace UE::WindowsInput
{
	static const FName InputClassName = "WindowsApplication";
	static const FString KBMInputHardwareName = "KBM";
	static const FString TouchInputHardwareName = "MobileTouch";
};

static bool ShouldSimulateRawInput()
{
	return ForceRawInputSimulation || (EnableRawInputSimulationOverRDP && FPlatformMisc::IsRemoteSession());
}

FWindowsApplication* FWindowsApplication::CreateWindowsApplication( const HINSTANCE InstanceHandle, const HICON IconHandle )
{
	WindowsApplication = new FWindowsApplication( InstanceHandle, IconHandle );
	return WindowsApplication;
}


FWindowsApplication::FWindowsApplication( const HINSTANCE HInstance, const HICON IconHandle )
	: GenericApplication( MakeShareable( new FWindowsCursor() ) )
	, InstanceHandle( HInstance )
	, bMinimized( false )
	, bUsingHighPrecisionMouseInput( false )
	, bCachedUseWorkerThreadForRawInput( false )
	, HighPrecisionMouseInputWindowHandle( NULL )
	, bIsMouseAttached( false )
	, bForceActivateByMouse( false )
	, bForceNoGamepads( false )
	, bConsumeAltSpace( false )
	, bHasLoadedInputPlugins( false )
	, bAllowedToDeferMessageProcessing(true)
	, CVarDeferMessageProcessing( 
		TEXT( "Slate.DeferWindowsMessageProcessing" ),
		bAllowedToDeferMessageProcessing,
		TEXT( "Whether windows message processing is deferred until tick or if they are processed immediately" ) )
#if WITH_ACCESSIBILITY && UE_WINDOWS_USING_UIA
	, UIAManager(new FWindowsUIAManager(*this))
#endif
	, bSimulatingHighPrecisionMouseInputForRDP(false)
	, LastCursorPoint(FIntPoint::ZeroValue)
	, LastCursorPointPreWrap(FIntPoint::ZeroValue)
	, NumPreWrapMsgsToRespect(0)
	, ClipCursorRect()
	, RepeatTime{}
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWindowsApplication::FWindowsApplication);

	FMemory::Memzero(ModifierKeyState, EModifierKey::Count);

	// Disable the process from being showing "ghosted" not responding messages during slow tasks
	// This is a hack.  A more permanent solution is to make our slow tasks not block the editor for so long
	// that message pumping doesn't occur (which causes these messages).
	::DisableProcessWindowsGhosting();

	// Register the Win32 class for Slate windows and assign the application instance and icon
	const bool bClassRegistered = RegisterClass( InstanceHandle, IconHandle );

	// Initialize OLE for Drag and Drop support.
	CA_SUPPRESS(6031);
	OleInitialize( NULL );

	TaskbarList = FTaskbarList::Create();

	// Get initial display metrics. (display information for existing desktop, before we start changing resolutions)
	FDisplayMetrics::RebuildDisplayMetrics(InitialDisplayMetrics);

	if (!GIsEditor)
	{
		GConfig->GetBool(TEXT("WindowsApplication"), TEXT("bConsumeAltSpace"), bConsumeAltSpace, GEngineIni);
	}

	// Save the current sticky/toggle/filter key settings so they can be restored them later
	// If there are .ini settings, use them instead of the current system settings.
	// NOTE: Whenever we exit and restore these settings gracefully, the .ini settings are removed.
	FMemory::Memzero(StartupStickyKeys);
	FMemory::Memzero(StartupToggleKeys);
	FMemory::Memzero(StartupFilterKeys);
	
	StartupStickyKeys.cbSize = sizeof(StartupStickyKeys);
	StartupToggleKeys.cbSize = sizeof(StartupToggleKeys);
	StartupFilterKeys.cbSize = sizeof(StartupFilterKeys);

	SystemParametersInfo(SPI_GETSTICKYKEYS, sizeof(STICKYKEYS), &StartupStickyKeys, 0);
	SystemParametersInfo(SPI_GETTOGGLEKEYS, sizeof(TOGGLEKEYS), &StartupToggleKeys, 0);
	SystemParametersInfo(SPI_GETFILTERKEYS, sizeof(FILTERKEYS), &StartupFilterKeys, 0);

	bool bSKHotkey = (StartupStickyKeys.dwFlags & SKF_HOTKEYACTIVE) ? true : false;
	bool bTKHotkey = (StartupToggleKeys.dwFlags & TKF_HOTKEYACTIVE) ? true : false;
	bool bFKHotkey = (StartupFilterKeys.dwFlags & FKF_HOTKEYACTIVE) ? true : false;
	bool bSKConfirmation = (StartupStickyKeys.dwFlags & SKF_CONFIRMHOTKEY) ? true : false;
	bool bTKConfirmation = (StartupToggleKeys.dwFlags & TKF_CONFIRMHOTKEY) ? true : false;
	bool bFKConfirmation = (StartupFilterKeys.dwFlags & FKF_CONFIRMHOTKEY) ? true : false;

	GConfig->GetBool(TEXT("WindowsApplication.Accessibility"), TEXT("StickyKeysHotkey"), bSKHotkey, GEngineIni);
	GConfig->GetBool(TEXT("WindowsApplication.Accessibility"), TEXT("ToggleKeysHotkey"), bTKHotkey, GEngineIni);
	GConfig->GetBool(TEXT("WindowsApplication.Accessibility"), TEXT("FilterKeysHotkey"), bFKHotkey, GEngineIni);
	GConfig->GetBool(TEXT("WindowsApplication.Accessibility"), TEXT("StickyKeysConfirmation"), bSKConfirmation, GEngineIni);
	GConfig->GetBool(TEXT("WindowsApplication.Accessibility"), TEXT("ToggleKeysConfirmation"), bTKConfirmation, GEngineIni);
	GConfig->GetBool(TEXT("WindowsApplication.Accessibility"), TEXT("FilterKeysConfirmation"), bFKConfirmation, GEngineIni);

	StartupStickyKeys.dwFlags = bSKHotkey ? (StartupStickyKeys.dwFlags | SKF_HOTKEYACTIVE) : (StartupStickyKeys.dwFlags & ~SKF_HOTKEYACTIVE);
	StartupToggleKeys.dwFlags = bTKHotkey ? (StartupToggleKeys.dwFlags | TKF_HOTKEYACTIVE) : (StartupToggleKeys.dwFlags & ~TKF_HOTKEYACTIVE);
	StartupFilterKeys.dwFlags = bFKHotkey ? (StartupFilterKeys.dwFlags | FKF_HOTKEYACTIVE) : (StartupFilterKeys.dwFlags & ~FKF_HOTKEYACTIVE);
	StartupStickyKeys.dwFlags = bSKConfirmation ? (StartupStickyKeys.dwFlags | SKF_CONFIRMHOTKEY) : (StartupStickyKeys.dwFlags & ~SKF_CONFIRMHOTKEY);
	StartupToggleKeys.dwFlags = bTKConfirmation ? (StartupToggleKeys.dwFlags | TKF_CONFIRMHOTKEY) : (StartupToggleKeys.dwFlags & ~TKF_CONFIRMHOTKEY);
	StartupFilterKeys.dwFlags = bFKConfirmation ? (StartupFilterKeys.dwFlags | FKF_CONFIRMHOTKEY) : (StartupFilterKeys.dwFlags & ~FKF_CONFIRMHOTKEY);

	GConfig->SetBool(TEXT("WindowsApplication.Accessibility"), TEXT("StickyKeysHotkey"), bSKHotkey, GEngineIni);
	GConfig->SetBool(TEXT("WindowsApplication.Accessibility"), TEXT("ToggleKeysHotkey"), bTKHotkey, GEngineIni);
	GConfig->SetBool(TEXT("WindowsApplication.Accessibility"), TEXT("FilterKeysHotkey"), bFKHotkey, GEngineIni);
	GConfig->SetBool(TEXT("WindowsApplication.Accessibility"), TEXT("StickyKeysConfirmation"), bSKConfirmation, GEngineIni);
	GConfig->SetBool(TEXT("WindowsApplication.Accessibility"), TEXT("ToggleKeysConfirmation"), bTKConfirmation, GEngineIni);
	GConfig->SetBool(TEXT("WindowsApplication.Accessibility"), TEXT("FilterKeysConfirmation"), bFKConfirmation, GEngineIni);

	GConfig->Flush(false, GEngineIni);

	FCoreDelegates::OnShutdownAfterError.AddRaw(this, &FWindowsApplication::ShutDownAfterError);

	// Disable accessibility shortcuts
	AllowAccessibilityShortcutKeys(false);

	QueryConnectedMice();

	if (FParse::Param(FCommandLine::Get(), TEXT("NoGamepad")))
	{
		bForceNoGamepads = true;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("FilterLowLevelMouse")))
	{
		ApplyLowLevelMouseFilter();
	}	
	IConsoleManager::Get().RegisterConsoleCommand(TEXT("WindowsApplication.ApplyLowLevelMouseFilter"), TEXT("Applies Low Level mouse filter that filters out mouse inputs that act like touch inputs"), FConsoleCommandDelegate::CreateRaw(this, &FWindowsApplication::ApplyLowLevelMouseFilter));
	IConsoleManager::Get().RegisterConsoleCommand(TEXT("WindowsApplication.RemoveLowLevelMouseFilter"), TEXT("Removes Low Level mouse filter that filters out mouse inputs that act like touch inputs"), FConsoleCommandDelegate::CreateRaw(this, &FWindowsApplication::RemoveLowLevelMouseFilter));

	UE::WindowsApplication::TouchpadSensitivity::HandleStartupApp();

	if (FPlatformProcess::SupportsMultithreading())
	{
		UE::WindowsApplication::RawInputThread::FWindowsRawInputRunnable::Initialize(HInstance);
	}

	// Register the keyboard and mouse device immediately so callers can look up
	// hardware metadata from FInputDeviceRegistry before the first tick.
	{
		const FInputDeviceId KBMDeviceId = IPlatformInputDeviceMapper::Get().GetDefaultInputDevice();
		FInputDeviceDescriptor Desc;
		Desc.InputDeviceName          = UE::WindowsInput::InputClassName;
		Desc.HardwareDeviceHandle     = KBMDeviceId;
		Desc.HardwareDeviceIdentifier = FName(*UE::WindowsInput::KBMInputHardwareName);
		FInputDeviceRegistry::RegisterDevice(Desc);
	}
}

bool FWindowsApplication::SupportsForceMaxTouchpadSensitivity()
{
	return UE::WindowsApplication::TouchpadSensitivity::SupportsForceMaxTouchpadSensitivity();
}

void FWindowsApplication::ApplyForceMaxTouchpadSensitivity()
{
	UE::WindowsApplication::TouchpadSensitivity::ApplyForceMaxTouchpadSensitivity();
}

void FWindowsApplication::RemoveForceMaxTouchpadSensitivity()
{
	UE::WindowsApplication::TouchpadSensitivity::RemoveForceMaxTouchpadSensitivity();
}

FWindowsApplication::EForceMaxTouchpadSensitivityRestorationValues FWindowsApplication::GetForceMaxTouchpadSensitivityRestorationValue()
{
	return UE::WindowsApplication::TouchpadSensitivity::GetForceMaxTouchpadSensitivityRestorationValue();
}

void FWindowsApplication::ApplyLowLevelMouseFilter()
{
	//Only check fake mouse inputs in game or digitizer devices will not work in the editor.
	if (FApp::IsGame() && !bLowLevelMouseFilterIsApplied)
	{
		LowLevelMouseFilterHook = ::SetWindowsHookEx(WH_MOUSE_LL, HandleLowLevelMouseFilterHook, NULL, 0);
		bLowLevelMouseFilterIsApplied = true;
	}
}

void FWindowsApplication::RemoveLowLevelMouseFilter()
{
	if (FApp::IsGame() && bLowLevelMouseFilterIsApplied)
	{
		::UnhookWindowsHookEx(LowLevelMouseFilterHook);
		bLowLevelMouseFilterIsApplied = false;
	}
}

void FWindowsApplication::AllowAccessibilityShortcutKeys(const bool bAllowKeys)
{
	if (bAllowKeys)
	{
		// Restore StickyKeys/etc to original state and enable Windows key      
		SystemParametersInfo(SPI_SETSTICKYKEYS, sizeof(STICKYKEYS), &StartupStickyKeys, 0);
		SystemParametersInfo(SPI_SETTOGGLEKEYS, sizeof(TOGGLEKEYS), &StartupToggleKeys, 0);
		SystemParametersInfo(SPI_SETFILTERKEYS, sizeof(FILTERKEYS), &StartupFilterKeys, 0);
	}
	else
	{
		// Disable StickyKeys/etc shortcuts but if the accessibility feature is on, 
		// then leave the settings alone as its probably being usefully used

		STICKYKEYS skOff = StartupStickyKeys;
		if ((skOff.dwFlags & SKF_STICKYKEYSON) == 0)
		{
			// Disable the hotkey and the confirmation
			skOff.dwFlags &= ~SKF_HOTKEYACTIVE;
			skOff.dwFlags &= ~SKF_CONFIRMHOTKEY;

			SystemParametersInfo(SPI_SETSTICKYKEYS, sizeof(STICKYKEYS), &skOff, 0);
		}

		TOGGLEKEYS tkOff = StartupToggleKeys;
		if ((tkOff.dwFlags & TKF_TOGGLEKEYSON) == 0)
		{
			// Disable the hotkey and the confirmation
			tkOff.dwFlags &= ~TKF_HOTKEYACTIVE;
			tkOff.dwFlags &= ~TKF_CONFIRMHOTKEY;

			SystemParametersInfo(SPI_SETTOGGLEKEYS, sizeof(TOGGLEKEYS), &tkOff, 0);
		}

		FILTERKEYS fkOff = StartupFilterKeys;
		if ((fkOff.dwFlags & FKF_FILTERKEYSON) == 0)
		{
			// Disable the hotkey and the confirmation
			fkOff.dwFlags &= ~FKF_HOTKEYACTIVE;
			fkOff.dwFlags &= ~FKF_CONFIRMHOTKEY;

			SystemParametersInfo(SPI_SETFILTERKEYS, sizeof(FILTERKEYS), &fkOff, 0);
		}
	}
}

void FWindowsApplication::DestroyApplication()
{
	// Restore accessibility shortcuts and remove the saved state from the .ini
	AllowAccessibilityShortcutKeys(true);
	GConfig->EmptySection(TEXT("WindowsApplication.Accessibility"), GEngineIni);

	TaskbarList = nullptr;

	RemoveLowLevelMouseFilter();
	IConsoleManager::Get().UnregisterConsoleObject(TEXT("WindowsApplication.ApplyLowLevelMouseFilter"));
	IConsoleManager::Get().UnregisterConsoleObject(TEXT("WindowsApplication.RemoveLowLevelMouseFilter"));
	UE::WindowsApplication::TouchpadSensitivity::HandleDestroyApplication();
}

void FWindowsApplication::ShutDownAfterError()
{
	// Restore accessibility shortcuts and remove the saved state from the .ini
	AllowAccessibilityShortcutKeys(true);
	GConfig->EmptySection(TEXT("WindowsApplication.Accessibility"), GEngineIni);

	TaskbarList = nullptr;

	RemoveLowLevelMouseFilter();
	IConsoleManager::Get().UnregisterConsoleObject(TEXT("WindowsApplication.ApplyLowLevelMouseFilter"));
	IConsoleManager::Get().UnregisterConsoleObject(TEXT("WindowsApplication.RemoveLowLevelMouseFilter"));
	UE::WindowsApplication::TouchpadSensitivity::HandleShutDownAfterError();
}

bool FWindowsApplication::RegisterClass( const HINSTANCE HInstance, const HICON HIcon )
{
	WNDCLASS wc;
	FMemory::Memzero( &wc, sizeof(wc) );
	wc.style = CS_DBLCLKS; // We want to receive double clicks
	wc.lpfnWndProc = AppWndProc;
	wc.cbClsExtra = 0;
	wc.cbWndExtra = 0;
	wc.hInstance = HInstance;
	wc.hIcon = HIcon;
	wc.hCursor = NULL;	// We manage the cursor ourselves
	wc.hbrBackground = NULL; // Transparent
	wc.lpszMenuName = NULL;
	wc.lpszClassName = FWindowsWindow::AppWindowClass;

	if( !::RegisterClass( &wc ) )
	{
		//ShowLastError();

		// @todo Slate: Error message should be localized!
		FSlowHeartBeatScope SuspendHeartBeat;
		MessageBox(NULL, TEXT("Window Registration Failed!"), TEXT("Error!"), MB_ICONEXCLAMATION | MB_OK);

		return false;
	}

	return true;
}

FWindowsApplication::~FWindowsApplication()
{
	if (FPlatformProcess::SupportsMultithreading())
	{
		UE::WindowsApplication::RawInputThread::FWindowsRawInputRunnable::Shutdown();
	}

	IModularFeatures::Get().OnModularFeatureRegistered().RemoveAll(this);

	FCoreDelegates::OnShutdownAfterError.RemoveAll(this);

	if (TextInputMethodSystem.IsValid())
	{
		TextInputMethodSystem->Terminate();
	}

	::CoUninitialize();
	OleUninitialize();
}

TSharedRef< FGenericWindow > FWindowsApplication::MakeWindow() 
{ 
	return FWindowsWindow::Make();
}

void FWindowsApplication::InitializeWindow( const TSharedRef< FGenericWindow >& InWindow, const TSharedRef< FGenericWindowDefinition >& InDefinition, const TSharedPtr< FGenericWindow >& InParent, const bool bShowImmediately )
{
	const TSharedRef< FWindowsWindow > Window = StaticCastSharedRef< FWindowsWindow >( InWindow );
	const TSharedPtr< FWindowsWindow > ParentWindow = StaticCastSharedPtr< FWindowsWindow >( InParent );

	Windows.Add( Window );
	Window->Initialize( this, InDefinition, InstanceHandle, ParentWindow, bShowImmediately );

#if !USING_ADDRESS_SANITISER
	// Disabled under AddressSanitizer as otherwise AddresSanitizer catches an issue in the cleanup code
	// of one of the input method Microsoft DLLs during shutdown
	static const bool InitInputMethodSystem = [this] {
		TextInputMethodSystem = MakeShareable(new FWindowsTextInputMethodSystem);
		if (!TextInputMethodSystem->Initialize())
		{
			TextInputMethodSystem.Reset();
		}
		return true;
	}();
#endif
}

void FWindowsApplication::SetMessageHandler( const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler )
{
	GenericApplication::SetMessageHandler(InMessageHandler);

	TArray<IInputDeviceModule*> PluginImplementations = IModularFeatures::Get().GetModularFeatureImplementations<IInputDeviceModule>( IInputDeviceModule::GetModularFeatureName() );
	for( auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt )
	{
		(*DeviceIt)->SetMessageHandler(InMessageHandler);
	}

}

#if WITH_ACCESSIBILITY
void FWindowsApplication::SetAccessibleMessageHandler(const TSharedRef<FGenericAccessibleMessageHandler>& InAccessibleMessageHandler)
{
	GenericApplication::SetAccessibleMessageHandler(InAccessibleMessageHandler);
#if UE_WINDOWS_USING_UIA
	UIAManager->OnAccessibleMessageHandlerChanged();
#endif
}
#endif

bool FWindowsApplication::IsGamepadAttached() const
{
	if (bForceNoGamepads)
	{
		return false;
	}

	for( auto DeviceIt = ExternalInputDevices.CreateConstIterator(); DeviceIt; ++DeviceIt )
	{
		if ((*DeviceIt)->IsGamepadAttached())
		{
			return true;
		}
	}

	return false;
}

FModifierKeysState FWindowsApplication::GetModifierKeys() const
{
	return FModifierKeysState(
		ModifierKeyState[EModifierKey::LeftShift], ModifierKeyState[EModifierKey::RightShift], 
		ModifierKeyState[EModifierKey::LeftControl], ModifierKeyState[EModifierKey::RightControl], 
		ModifierKeyState[EModifierKey::LeftAlt], ModifierKeyState[EModifierKey::RightAlt], 
		false, false, 
		ModifierKeyState[EModifierKey::CapsLock]
		); // Win key is ignored
}

void FWindowsApplication::UpdateAllModifierKeyStates()
{
	ModifierKeyState[EModifierKey::LeftShift]		= (::GetAsyncKeyState(VK_LSHIFT) & 0x8000) != 0;
	ModifierKeyState[EModifierKey::RightShift]		= (::GetAsyncKeyState(VK_RSHIFT) & 0x8000) != 0;
	ModifierKeyState[EModifierKey::LeftControl]		= (::GetAsyncKeyState(VK_LCONTROL) & 0x8000) != 0;
	ModifierKeyState[EModifierKey::RightControl]	= (::GetAsyncKeyState(VK_RCONTROL) & 0x8000) != 0;
	ModifierKeyState[EModifierKey::LeftAlt]			= (::GetAsyncKeyState(VK_LMENU) & 0x8000) != 0;
	ModifierKeyState[EModifierKey::RightAlt]		= (::GetAsyncKeyState(VK_RMENU) & 0x8000) != 0;
	ModifierKeyState[EModifierKey::CapsLock]		= (::GetKeyState(VK_CAPITAL) & 0x0001) != 0;
}

static TSharedPtr< FWindowsWindow > FindWindowByHWND(const TArray< TSharedRef< FWindowsWindow > >& WindowsToSearch, HWND HandleToFind)
{
	for (int32 WindowIndex = 0; WindowIndex < WindowsToSearch.Num(); ++WindowIndex)
	{
		TSharedRef< FWindowsWindow > Window = WindowsToSearch[WindowIndex];
		if (Window->GetHWnd() == HandleToFind)
		{
			return Window;
		}
	}

	return TSharedPtr< FWindowsWindow >(nullptr);
}


bool FWindowsApplication::IsCursorDirectlyOverSlateWindow() const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_IsCursorDirectlyOverSlateWindow);

	POINT CursorPos;
	if (::GetCursorPos(&CursorPos))
	{
		const HWND hWnd = ::WindowFromPoint(CursorPos);
		const TSharedPtr<FWindowsWindow> SlateWindowUnderCursor = FindWindowByHWND(Windows, hWnd);
		return SlateWindowUnderCursor.IsValid();
	}

	return false;
}

void FWindowsApplication::SetCapture( const TSharedPtr< FGenericWindow >& InWindow )
{
	HWND DesiredCaptureHandle = (InWindow.IsValid() ? (HWND)InWindow->GetOSWindowHandle() : NULL);

	if (DesiredCaptureHandle != CaptureWindowHandle)
	{
		if (DesiredCaptureHandle)
		{
			::SetCapture(DesiredCaptureHandle);
		}
		else
		{
			::ReleaseCapture();
		}
	}
	CaptureWindowHandle = DesiredCaptureHandle;
}

void* FWindowsApplication::GetCapture( void ) const
{
	return ::GetCapture();
}

void FWindowsApplication::SetHighPrecisionMouseMode( const bool Enable, const TSharedPtr< FGenericWindow >& InWindow )
{
	if (ShouldSimulateRawInput())
	{
		if(Enable)
		{
			bUsingHighPrecisionMouseInput = true;
			bSimulatingHighPrecisionMouseInputForRDP = true;

			POINT CursorPos;
			BOOL bGotPoint = ::GetCursorPos(&CursorPos);

			::GetClipCursor(&ClipCursorRect);

			//UE_LOGF(LogWindowsDesktop, Log, "Entering High Precision to Top: %d Bottom: %d Left: %d Right: %d ---- Starting x: %d y: %d", ClipCursorRect.top, ClipCursorRect.bottom, ClipCursorRect.left, ClipCursorRect.right, CursorPos.x, CursorPos.y);

			LastCursorPoint = FIntPoint(CursorPos.x, CursorPos.y);

			LastCursorPointPreWrap = FIntPoint::ZeroValue;
			NumPreWrapMsgsToRespect = 0;
		}
		else
		{
			bSimulatingHighPrecisionMouseInputForRDP = false;
		}
	}

	bUsingHighPrecisionMouseInput = Enable;
	HighPrecisionMouseInputWindowHandle = (Enable && InWindow.IsValid()) ? (HWND)InWindow->GetOSWindowHandle() : NULL;

	// NOTE: Currently has to be created every time due to conflicts with Direct8 Input used by the wx unrealed
	if (!Enable || (bCachedUseWorkerThreadForRawInput != UE::WindowsApplication::RawInputThread::ShouldUseWorkerThread()))
	{
		UnregisterHighPrecisionMouseMode();
	}
	if (Enable)
	{
		RegisterHighPrecisionMouseMode();
	}
}

void FWindowsApplication::RegisterHighPrecisionMouseMode()
{
	const bool bUseWorkerThreadForRawInputLocal = UE::WindowsApplication::RawInputThread::ShouldUseWorkerThread();
	if (bUseWorkerThreadForRawInputLocal)
	{
		UE::WindowsApplication::RawInputThread::FWindowsRawInputRunnable::Register();
	}
	else
	{
		RAWINPUTDEVICE RawInputDevice;
		RawInputDevice.usUsagePage = 0x01; // HID_USAGE_PAGE_GENERIC
		RawInputDevice.usUsage = 0x02; // HID_USAGE_GENERIC_MOUSE
		RawInputDevice.dwFlags = 0;
		RawInputDevice.hwndTarget = HighPrecisionMouseInputWindowHandle;
		const BOOL bRegistered = ::RegisterRawInputDevices(&RawInputDevice, 1, sizeof(RAWINPUTDEVICE));
		UE_LOGF(LogWindowsDesktop, Verbose, "Registering game thread for raw input: %d", bRegistered);
	}

	bCachedUseWorkerThreadForRawInput = bUseWorkerThreadForRawInputLocal;
}

void FWindowsApplication::UnregisterHighPrecisionMouseMode()
{
	if (bCachedUseWorkerThreadForRawInput)
	{
		UE::WindowsApplication::RawInputThread::FWindowsRawInputRunnable::Unregister();

		// Flush inputs from worker thread
		ProcessWorkerInputs();
	}
	else
	{
		RAWINPUTDEVICE RawInputDevice;
		RawInputDevice.usUsagePage = 0x01; // HID_USAGE_PAGE_GENERIC
		RawInputDevice.usUsage = 0x02; // HID_USAGE_GENERIC_MOUSE
		RawInputDevice.dwFlags = RIDEV_REMOVE;
		RawInputDevice.hwndTarget = NULL;
		const BOOL bUnregistered = ::RegisterRawInputDevices(&RawInputDevice, 1, sizeof(RAWINPUTDEVICE));
		UE_LOGF(LogWindowsDesktop, Verbose, "Unregistering game thread for raw input: %d", bUnregistered);

		// Flush inputs on window thread (and other Windows messages)
		MSG Message;
		while (PeekMessage(&Message, NULL, 0, 0, PM_REMOVE))
		{
			TranslateMessage(&Message);
			DispatchMessage(&Message);
		}
	}
}

void FWindowsApplication::UpdateHighPrecisionMouseMode()
{
	if (bUsingHighPrecisionMouseInput &&
		(bCachedUseWorkerThreadForRawInput != UE::WindowsApplication::RawInputThread::ShouldUseWorkerThread()))
	{
		UnregisterHighPrecisionMouseMode();
		RegisterHighPrecisionMouseMode();
	}
}

FPlatformRect FWindowsApplication::GetWorkArea(const FPlatformRect& CurrentWindow) const
{
	return GetWorkAreaFromOS(CurrentWindow);
}

FPlatformRect FWindowsApplication::GetWorkAreaFromOS( const FPlatformRect& CurrentWindow ) const
{
	RECT WindowsWindowDim;
	WindowsWindowDim.left = CurrentWindow.Left;
	WindowsWindowDim.top = CurrentWindow.Top;
	WindowsWindowDim.right = CurrentWindow.Right;
	WindowsWindowDim.bottom = CurrentWindow.Bottom;

	// ... figure out the best monitor for that window.
	HMONITOR hBestMonitor = MonitorFromRect( &WindowsWindowDim, MONITOR_DEFAULTTONEAREST );

	// Get information about that monitor...
	MONITORINFO MonitorInfo;
	MonitorInfo.cbSize = sizeof(MonitorInfo);
	GetMonitorInfo( hBestMonitor, &MonitorInfo);

	// ... so that we can figure out the work area (are not covered by taskbar)

	FPlatformRect WorkArea;
	WorkArea.Left = MonitorInfo.rcWork.left;
	WorkArea.Top = MonitorInfo.rcWork.top;
	WorkArea.Right = MonitorInfo.rcWork.right;
	WorkArea.Bottom = MonitorInfo.rcWork.bottom;

	return WorkArea;
}

/**
 * Extracts EDID data from the given registry key and reads out native display with and height
 * @param hDevRegKey - Registry key where EDID is stored
 * @param OutWidth - Reference to output variable for monitor native width
 * @param OutHeight - Reference to output variable for monitor native height
 * @returns 'true' if data was extracted successfully, 'false' otherwise
 **/
static bool GetMonitorSizeFromEDID(const HKEY hDevRegKey, int32& OutWidth, int32& OutHeight)
{	
	static const uint32 NameSize = 512;
	static TCHAR ValueName[NameSize];

	DWORD Type;

	BYTE EDIDData[1024];

	for (LONG i = 0, RetValue = ERROR_SUCCESS; RetValue != ERROR_NO_MORE_ITEMS; ++i)
	{
		DWORD ActualValueNameLength = NameSize;
		DWORD EDIDSize = sizeof(EDIDData);
		RetValue = RegEnumValue ( hDevRegKey, 
			i, 
			&ValueName[0],
			&ActualValueNameLength, NULL, &Type,
			EDIDData,
			&EDIDSize);

		if (RetValue != ERROR_SUCCESS || (FCString::Strcmp(ValueName, TEXT("EDID")) != 0) || (EDIDSize < 62))
		{
			continue;
		}

		// EDID data format documented here:
		// http://en.wikipedia.org/wiki/EDID

		int DetailTimingDescriptorStartIndex = 54;
		OutWidth = ((EDIDData[DetailTimingDescriptorStartIndex+4] >> 4) << 8) | EDIDData[DetailTimingDescriptorStartIndex+2];
		OutHeight = ((EDIDData[DetailTimingDescriptorStartIndex+7] >> 4) << 8) | EDIDData[DetailTimingDescriptorStartIndex+5];

		return true; // valid EDID found
	}

	return false; // EDID not found
}

static void GetMonitorInfoSetupAPI(const FString& HardwareID, TOptional<FIntPoint>& NativeResolution, FString& FriendlyName)
{
	static constexpr GUID ClassMonitorGuid = {0x4d36e96e, 0xe325, 0x11ce, {0xbf, 0xc1, 0x08, 0x00, 0x2b, 0xe1, 0x03, 0x18}};

	// Could include <Devpkey.h> instead of copying here, but INITGUID behavior seems to diverge between editor and client, and would need a solve.
	static constexpr DEVPROPKEY DEVPKEY_Device_HardwareIds = {{0xa45c254e, 0xdf1c, 0x4efd, {0x80, 0x20, 0x67, 0xd1, 0x46, 0xa8, 0x50, 0xe0}}, 3};
	static constexpr DEVPROPKEY DEVPKEY_Device_FriendlyName = {{0xa45c254e, 0xdf1c, 0x4efd, {0x80, 0x20, 0x67, 0xd1, 0x46, 0xa8, 0x50, 0xe0}}, 14};

	HDEVINFO DevInfo = SetupDiGetClassDevsEx(
		&ClassMonitorGuid, //class GUID
		NULL,
		NULL,
		DIGCF_PRESENT,
		NULL,
		NULL,
		NULL);

	if (DevInfo != INVALID_HANDLE_VALUE)
	{
		bool bFound = false;
		SP_DEVINFO_DATA DevInfoData{};
		DevInfoData.cbSize = sizeof(DevInfoData);
		for (ULONG MonitorIndex = 0; SetupDiEnumDeviceInfo(DevInfo, MonitorIndex, &DevInfoData); ++MonitorIndex)
		{
			DEVPROPTYPE PropertyType;
			DWORD RequiredSize;
			if (!SetupDiGetDeviceProperty(DevInfo, &DevInfoData, &DEVPKEY_Device_HardwareIds, &PropertyType, nullptr, 0, &RequiredSize, 0) && (GetLastError() == ERROR_INSUFFICIENT_BUFFER))
			{
				TArray<WCHAR> HardwareIDsBuffer;
				HardwareIDsBuffer.SetNum((RequiredSize + 1) / 2);
				if (SetupDiGetDeviceProperty(DevInfo, &DevInfoData, &DEVPKEY_Device_HardwareIds, &PropertyType, (PBYTE)HardwareIDsBuffer.GetData(), RequiredSize, nullptr, 0))
				{
					int32 Cursor = 0;
					while ((Cursor < HardwareIDsBuffer.Num()) && HardwareIDsBuffer[Cursor])
					{
						if (HardwareID == &HardwareIDsBuffer[Cursor])
						{
							bFound = true;
							break;
						}

						// Move cursor to null terminator
						do
						{
							++Cursor;
						}
						while ((Cursor < HardwareIDsBuffer.Num()) && HardwareIDsBuffer[Cursor]);

						// Move cursor to next string
						++Cursor;
					}

					if (bFound)
					{
						HKEY hDevRegKey = SetupDiOpenDevRegKey(DevInfo, &DevInfoData, DICS_FLAG_GLOBAL, 0, DIREG_DEV, KEY_READ);
						if (hDevRegKey && hDevRegKey != INVALID_HANDLE_VALUE)
						{
							int32 NativeWidth, NativeHeight;
							if (GetMonitorSizeFromEDID(hDevRegKey, NativeWidth, NativeHeight))
							{
								NativeResolution = FIntPoint(NativeWidth, NativeHeight);
							}

							if (!SetupDiGetDeviceProperty(DevInfo, &DevInfoData, &DEVPKEY_Device_FriendlyName, &PropertyType, nullptr, 0, &RequiredSize, 0) && (GetLastError() == ERROR_INSUFFICIENT_BUFFER))
							{
								TArray<WCHAR> FriendlyNameBuffer;
								FriendlyNameBuffer.SetNum((RequiredSize + 1) / 2);
								if (SetupDiGetDeviceProperty(DevInfo, &DevInfoData, &DEVPKEY_Device_FriendlyName, &PropertyType, (PBYTE)FriendlyNameBuffer.GetData(), RequiredSize, nullptr, 0))
								{
									FriendlyName = FriendlyNameBuffer.GetData();
								}
							}

							RegCloseKey(hDevRegKey);
						}

						break;
					}
				}
			}

			// Just in case SetupDiEnumDeviceInfo could modify this value.
			DevInfoData.cbSize = sizeof(DevInfoData);
		}
		ensure(bFound || (GetLastError() == ERROR_NO_MORE_ITEMS));

		SetupDiDestroyDeviceInfoList(DevInfo);
	}
}

static void GetFriendlyNameDisplayDevices(const WCHAR DeviceName[CCHDEVICENAME], FString& FriendlyName)
{
	UINT32 PathCount;
	TArray<DISPLAYCONFIG_PATH_INFO> Paths;
	LONG Result;

	do
	{
		constexpr UINT32 Flags = QDC_ONLY_ACTIVE_PATHS | QDC_VIRTUAL_MODE_AWARE;
		UINT32 ModeCount;
		Result = GetDisplayConfigBufferSizes(Flags, &PathCount, &ModeCount);
		if (Result != ERROR_SUCCESS)
		{
			return;
		}

		TArray<DISPLAYCONFIG_MODE_INFO> Modes;
		Paths.SetNum(PathCount);
		Modes.SetNum(ModeCount);
		Result = QueryDisplayConfig(Flags, &PathCount, Paths.GetData(), &ModeCount, Modes.GetData(), NULL);
	} while (Result == ERROR_INSUFFICIENT_BUFFER);

	if (Result == ERROR_SUCCESS)
	{
		for (size_t PathIndex = 0; PathIndex < PathCount; ++PathIndex)
		{
			const DISPLAYCONFIG_PATH_INFO& Path = Paths[PathIndex];
			DISPLAYCONFIG_SOURCE_DEVICE_NAME SourceName;
			SourceName.header.type = DISPLAYCONFIG_DEVICE_INFO_GET_SOURCE_NAME;
			SourceName.header.size = sizeof(SourceName);
			SourceName.header.adapterId = Path.sourceInfo.adapterId;
			SourceName.header.id = Path.sourceInfo.id;
			if ((DisplayConfigGetDeviceInfo(&SourceName.header) == ERROR_SUCCESS) &&
				(FCString::Strcmp(DeviceName, SourceName.viewGdiDeviceName) == 0)) //-V614
			{
				DISPLAYCONFIG_TARGET_DEVICE_NAME TargetName;
				TargetName.header.type = DISPLAYCONFIG_DEVICE_INFO_GET_TARGET_NAME;
				TargetName.header.size = sizeof(TargetName);
				TargetName.header.adapterId = Path.targetInfo.adapterId;
				TargetName.header.id = Path.targetInfo.id;
				if (DisplayConfigGetDeviceInfo(&TargetName.header) == ERROR_SUCCESS)
				{
					if (TargetName.flags.friendlyNameFromEdid)
					{
						FriendlyName = TargetName.monitorFriendlyDeviceName;
						break;
					}
				}
			}
		}
	}
}

static bool GetMonitorDetails(HMONITOR Monitor, const FString& DeviceID, FString& FriendlyName, int32& NativeWidth, int32& NativeHeight, int32& DPI)
{
	MONITORINFOEX MonitorInfoEx;
	MonitorInfoEx.cbSize = sizeof(MonitorInfoEx);
	if (GetMonitorInfo(Monitor, &MonitorInfoEx))
	{
		TOptional<FIntPoint> NativeResolution;
		FString FriendlyNameFallback;
		const int32 BackslashIndex = DeviceID.Find(TEXT("\\"), ESearchCase::CaseSensitive, ESearchDir::FromStart, 9);
		GetMonitorInfoSetupAPI((BackslashIndex != INDEX_NONE) ? DeviceID.Left(BackslashIndex) : DeviceID, NativeResolution, FriendlyNameFallback);

		if (NativeResolution.IsSet())
		{
			NativeWidth = NativeResolution->X;
			NativeHeight = NativeResolution->Y;
		}
		else
		{
			NativeWidth = MonitorInfoEx.rcMonitor.right - MonitorInfoEx.rcMonitor.left;
			NativeHeight = MonitorInfoEx.rcMonitor.bottom - MonitorInfoEx.rcMonitor.top;
		}

		GetFriendlyNameDisplayDevices(MonitorInfoEx.szDevice, FriendlyName);
		if (FriendlyName.IsEmpty())
		{
			FriendlyName = FriendlyNameFallback;
		}

		DPI = FWindowsPlatformApplicationMisc::GetMonitorDPI(Monitor);

		return true;
	}

	return false;
}

struct MonitorUserData
{
	FMonitorInfo* Info = nullptr;
	HMONITOR NativeHandle = NULL;
};
static BOOL CALLBACK MonitorEnumProc(HMONITOR Monitor, HDC MonitorDC, LPRECT Rect, LPARAM UserData)
{
	MONITORINFOEX MonitorInfoEx;
	MonitorInfoEx.cbSize = sizeof(MonitorInfoEx);	
	if (GetMonitorInfo(Monitor, &MonitorInfoEx))
	{
		MonitorUserData* Data = (MonitorUserData*)UserData;
		FMonitorInfo* Info = Data->Info;
		if (Info->Name == MonitorInfoEx.szDevice)
		{
			Info->DisplayRect.Bottom = MonitorInfoEx.rcMonitor.bottom;
			Info->DisplayRect.Left = MonitorInfoEx.rcMonitor.left;
			Info->DisplayRect.Right = MonitorInfoEx.rcMonitor.right;
			Info->DisplayRect.Top = MonitorInfoEx.rcMonitor.top;

			Info->WorkArea.Bottom = MonitorInfoEx.rcWork.bottom;
			Info->WorkArea.Left = MonitorInfoEx.rcWork.left;
			Info->WorkArea.Right = MonitorInfoEx.rcWork.right;
			Info->WorkArea.Top = MonitorInfoEx.rcWork.top;

			Data->NativeHandle = Monitor;

			return FALSE;
		}
	}

	return TRUE;
}

static FIntPoint GetMaxResolutionForDisplay(const DISPLAY_DEVICE& DisplayDevice, int32 NativeWidth, int32 NativeHeight)
{
	uint32 MaxWidth = NativeWidth;
	uint32 MaxHeight = NativeHeight;

	uint32 ModeIndex = 0;
	DEVMODE DisplayMode;
	FMemory::Memzero(DisplayMode);

	while (EnumDisplaySettings(DisplayDevice.DeviceName, ModeIndex++, &DisplayMode))
	{
		if (DisplayMode.dmPelsWidth > MaxWidth && DisplayMode.dmPelsHeight > MaxHeight)
		{
			MaxWidth = DisplayMode.dmPelsWidth;
			MaxHeight = DisplayMode.dmPelsHeight;
		}

		FMemory::Memzero(DisplayMode);
	}

	return FIntPoint(MaxWidth, MaxHeight);
}

/**
 * Extract hardware information about connect monitors
 * @param OutMonitorInfo - Reference to an array for holding records about each detected monitor
 **/
static void GetMonitorsInfo(TArray<FMonitorInfo>& OutMonitorInfo)
{
	DISPLAY_DEVICE DisplayDevice;
	DisplayDevice.cb = sizeof(DisplayDevice);
	DWORD DeviceIndex = 0; // device index

	OutMonitorInfo.Empty(2); // Reserve two slots, as that will be the most common maximum

	FString DeviceID;
	while (EnumDisplayDevices(0, DeviceIndex, &DisplayDevice, 0))
	{
		if ((DisplayDevice.StateFlags & DISPLAY_DEVICE_ATTACHED_TO_DESKTOP) > 0)
		{
			DISPLAY_DEVICE Monitor;
			ZeroMemory(&Monitor, sizeof(Monitor));
			Monitor.cb = sizeof(Monitor);
			DWORD MonitorIndex = 0;

			while (EnumDisplayDevices(DisplayDevice.DeviceName, MonitorIndex, &Monitor, 0))
			{
				if (Monitor.StateFlags & DISPLAY_DEVICE_ACTIVE &&
					!(Monitor.StateFlags & DISPLAY_DEVICE_MIRRORING_DRIVER))
				{
					FMonitorInfo Info;
					Info.Name = DisplayDevice.DeviceName;

					MonitorUserData Data{};
					Data.Info = &Info;
					EnumDisplayMonitors(nullptr, nullptr, MonitorEnumProc, (LPARAM)&Data);

					Info.ID = Monitor.DeviceID;
					const int32 BackslashIndex = Info.ID.Find(TEXT("\\"), ESearchCase::CaseSensitive, ESearchDir::FromStart, 9);
					Info.Name = (BackslashIndex != INDEX_NONE) ? Info.ID.Mid(8, BackslashIndex - 8) : Monitor.DeviceID;

					if (GetMonitorDetails(Data.NativeHandle, Info.ID, Info.FriendlyName, Info.NativeWidth, Info.NativeHeight, Info.DPI))
					{
						// usually expected: 4:3=1.333, 16:9=1.777, 16:10=1.6, multi-monitor-wide: >2
						bool bIsPortrait = Info.NativeWidth < Info.NativeHeight;
						// special case to detect specific platform display possibly rotated by the OS but the rotation is not reported in the EDID data
						if (bIsPortrait && (Info.DisplayRect.Bottom == Info.NativeWidth) && (Info.DisplayRect.Right == Info.NativeHeight))
						{
							Swap(Info.NativeWidth, Info.NativeHeight);
						}

						Info.bIsPrimary = (DisplayDevice.StateFlags & DISPLAY_DEVICE_PRIMARY_DEVICE) > 0;

						// The editor shouldn't care about lower level display resolutions. This is only necessary for fullscreen exclusive situations.
						if (!GIsEditor)
						{
							Info.MaxResolution = GetMaxResolutionForDisplay(DisplayDevice, Info.NativeWidth, Info.NativeHeight);
						}

						Info.NativeHandle = Data.NativeHandle;

						OutMonitorInfo.Add(Info);
					}
				}
				MonitorIndex++;

				ZeroMemory(&Monitor, sizeof(Monitor));
				Monitor.cb = sizeof(Monitor);
			}
		}

		ZeroMemory(&DisplayDevice, sizeof(DisplayDevice));
		DisplayDevice.cb = sizeof(DisplayDevice);
		DeviceIndex++;
	}
}

void FDisplayMetrics::RebuildDisplayMetrics(struct FDisplayMetrics& OutDisplayMetrics)
{
	if (FNullPlatformApplicationMisc::IsUsingNullApplication())
	{
		FNullPlatformDisplayMetrics::RebuildDisplayMetrics(OutDisplayMetrics);
	}
	else
	{
		// Total screen size of the primary monitor
		OutDisplayMetrics.PrimaryDisplayWidth = ::GetSystemMetrics( SM_CXSCREEN );
		OutDisplayMetrics.PrimaryDisplayHeight = ::GetSystemMetrics( SM_CYSCREEN );

		// Get the screen rect of the primary monitor, excluding taskbar etc.
		RECT WorkAreaRect;
		if(!SystemParametersInfo(SPI_GETWORKAREA, 0, &WorkAreaRect, 0))
		{
			WorkAreaRect.top = WorkAreaRect.bottom = WorkAreaRect.left = WorkAreaRect.right = 0;
		}

		OutDisplayMetrics.PrimaryDisplayWorkAreaRect.Left = WorkAreaRect.left;
		OutDisplayMetrics.PrimaryDisplayWorkAreaRect.Top = WorkAreaRect.top;
		OutDisplayMetrics.PrimaryDisplayWorkAreaRect.Right = WorkAreaRect.right;
		OutDisplayMetrics.PrimaryDisplayWorkAreaRect.Bottom = WorkAreaRect.bottom;
		
		// Virtual desktop area
		OutDisplayMetrics.VirtualDisplayRect.Left = ::GetSystemMetrics( SM_XVIRTUALSCREEN );
		OutDisplayMetrics.VirtualDisplayRect.Top = ::GetSystemMetrics( SM_YVIRTUALSCREEN );
		OutDisplayMetrics.VirtualDisplayRect.Right = OutDisplayMetrics.VirtualDisplayRect.Left + ::GetSystemMetrics( SM_CXVIRTUALSCREEN );
		OutDisplayMetrics.VirtualDisplayRect.Bottom = OutDisplayMetrics.VirtualDisplayRect.Top + ::GetSystemMetrics( SM_CYVIRTUALSCREEN );

		// Get connected monitor information
		GetMonitorsInfo(OutDisplayMetrics.MonitorInfo);
	}

	// Apply the debug safe zones
	OutDisplayMetrics.ApplyDefaultSafeZones();
}

void FWindowsApplication::GetInitialDisplayMetrics( FDisplayMetrics& OutDisplayMetrics ) const
{
	OutDisplayMetrics = InitialDisplayMetrics;
}

EWindowTitleAlignment::Type FWindowsApplication::GetWindowTitleAlignment() const
{
	OSVERSIONINFOEX VersionInfo;
	FMemory::Memzero(VersionInfo);
	VersionInfo.dwMajorVersion = 6;
	VersionInfo.dwMinorVersion = 2;
	VersionInfo.dwOSVersionInfoSize = sizeof(VersionInfo);

	DWORDLONG LongConditionMask = 0;
	BYTE ConditionMask = VER_GREATER_EQUAL;
	VER_SET_CONDITION(LongConditionMask, VER_MAJORVERSION, ConditionMask);
	VER_SET_CONDITION(LongConditionMask, VER_MINORVERSION, ConditionMask);

	if (::VerifyVersionInfo(&VersionInfo, VER_MAJORVERSION | VER_MINORVERSION, LongConditionMask) != 0)
	{
		return EWindowTitleAlignment::Center;
	}		

	return EWindowTitleAlignment::Left;
}
	
EWindowTransparency FWindowsApplication::GetWindowTransparencySupport() const
{
#if ALPHA_BLENDED_WINDOWS
	BOOL bIsCompositionEnabled = FALSE;
	::DwmIsCompositionEnabled(&bIsCompositionEnabled);

	return bIsCompositionEnabled ? EWindowTransparency::PerPixel : EWindowTransparency::PerWindow;
#else
	return EWindowTransparency::PerWindow;
#endif
}

// Defined as a global so that it can be extern'd by UELibrary
LRESULT WindowsApplication_WndProc(HWND hwnd, uint32 msg, WPARAM wParam, LPARAM lParam)
{
	ensure( IsInGameThread() );

	return WindowsApplication->ProcessMessage( hwnd, msg, wParam, lParam );
}

LRESULT CALLBACK FWindowsApplication::AppWndProc(HWND hwnd, uint32 msg, WPARAM wParam, LPARAM lParam)
{
	return WindowsApplication_WndProc( hwnd, msg, wParam, lParam );
}

int32 FWindowsApplication::ProcessMessage( HWND hwnd, uint32 msg, WPARAM wParam, LPARAM lParam )
{
	TSharedPtr< FWindowsWindow > CurrentNativeEventWindowPtr = FindWindowByHWND( Windows, hwnd );

	if( Windows.Num() && CurrentNativeEventWindowPtr.IsValid() )
	{
		TSharedRef< FWindowsWindow > CurrentNativeEventWindow = CurrentNativeEventWindowPtr.ToSharedRef();

		static const TMap<uint32, FString> WindowsMessageStrings = []()
		{
			TMap<uint32, FString> Result;
#define ADD_WINDOWS_MESSAGE_STRING(WMCode) Result.Add(WMCode, TEXT(#WMCode))
			ADD_WINDOWS_MESSAGE_STRING(WM_INPUTLANGCHANGEREQUEST);
			ADD_WINDOWS_MESSAGE_STRING(WM_INPUTLANGCHANGE);
			ADD_WINDOWS_MESSAGE_STRING(WM_IME_SETCONTEXT);
			ADD_WINDOWS_MESSAGE_STRING(WM_IME_NOTIFY);
			ADD_WINDOWS_MESSAGE_STRING(WM_IME_REQUEST);
			ADD_WINDOWS_MESSAGE_STRING(WM_IME_STARTCOMPOSITION);
			ADD_WINDOWS_MESSAGE_STRING(WM_IME_COMPOSITION);
			ADD_WINDOWS_MESSAGE_STRING(WM_IME_ENDCOMPOSITION);
			ADD_WINDOWS_MESSAGE_STRING(WM_IME_CHAR);
#undef ADD_WINDOWS_MESSAGE_STRING
			return Result;
		}();

		static const TMap<uint32, FString> IMNStrings = []()
		{
			TMap<uint32, FString> Result;
#define ADD_IMN_STRING(IMNCode) Result.Add(IMNCode, TEXT(#IMNCode))
			ADD_IMN_STRING(IMN_CLOSESTATUSWINDOW);
			ADD_IMN_STRING(IMN_OPENSTATUSWINDOW);
			ADD_IMN_STRING(IMN_CHANGECANDIDATE);
			ADD_IMN_STRING(IMN_CLOSECANDIDATE);
			ADD_IMN_STRING(IMN_OPENCANDIDATE);
			ADD_IMN_STRING(IMN_SETCONVERSIONMODE);
			ADD_IMN_STRING(IMN_SETSENTENCEMODE);
			ADD_IMN_STRING(IMN_SETOPENSTATUS);
			ADD_IMN_STRING(IMN_SETCANDIDATEPOS);
			ADD_IMN_STRING(IMN_SETCOMPOSITIONFONT);
			ADD_IMN_STRING(IMN_SETCOMPOSITIONWINDOW);
			ADD_IMN_STRING(IMN_SETSTATUSWINDOWPOS);
			ADD_IMN_STRING(IMN_GUIDELINE);
			ADD_IMN_STRING(IMN_PRIVATE);
#undef ADD_IMN_STRING
			return Result;
		}();

		static const TMap<uint32, FString> IMRStrings = []()
		{
			TMap<uint32, FString> Result;
#define ADD_IMR_STRING(IMRCode) Result.Add(IMRCode, TEXT(#IMRCode))
	ADD_IMR_STRING(IMR_CANDIDATEWINDOW);
	ADD_IMR_STRING(IMR_COMPOSITIONFONT);
	ADD_IMR_STRING(IMR_COMPOSITIONWINDOW);
	ADD_IMR_STRING(IMR_CONFIRMRECONVERTSTRING);
	ADD_IMR_STRING(IMR_DOCUMENTFEED);
	ADD_IMR_STRING(IMR_QUERYCHARPOSITION);
	ADD_IMR_STRING(IMR_RECONVERTSTRING);
#undef ADD_IMR_STRING
			return Result;
		}();

		bool bMessageExternallyHandled = false;
		int32 ExternalMessageHandlerResult = 0;

		// give others a chance to handle messages
		for (IWindowsMessageHandler* Handler : MessageHandlers)
		{
			CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_CustomMessageHandler_ProcessMessage, bEnableWindowMessageProfilerScopes);
			int32 HandlerResult = 0;
			if (Handler->ProcessMessage(hwnd, msg, wParam, lParam, HandlerResult))
			{
				if (!bMessageExternallyHandled)
				{
					bMessageExternallyHandled = true;
					ExternalMessageHandlerResult = HandlerResult;
				}
			}
		}

		//Only check fake mouse inputs in game or digitizer devices will not work in the editor.
		if (FApp::IsGame() && IsFakeMouseInputMessage(msg))
		{
			return 0;
		}


		FString MessageScopeName;
		if (bEnableWindowMessageProfilerScopes)
		{
			MessageScopeName = FString::Printf(TEXT("Window Message %u"), msg);
		}

		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_CONDITIONAL(*MessageScopeName, !MessageScopeName.IsEmpty());
		switch(msg)
		{
		case WM_INPUTLANGCHANGEREQUEST:
		case WM_INPUTLANGCHANGE:
		case WM_IME_SETCONTEXT:
		case WM_IME_STARTCOMPOSITION:
		case WM_IME_COMPOSITION:
		case WM_IME_ENDCOMPOSITION:
		case WM_IME_CHAR:
			UE_LOGF(LogWindowsDesktop, Verbose, "%ls", *(WindowsMessageStrings[msg]));
			DeferMessage( CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam );
			return 0;
		case WM_IME_NOTIFY:
			UE_LOGF(LogWindowsDesktop, Verbose, "WM_IME_NOTIFY - %ls", IMNStrings.Find(IntCastChecked<uint32>(wParam)) ? *(IMNStrings[(uint32)wParam]) : nullptr);
			DeferMessage( CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam );
			return 0;
		case WM_IME_REQUEST:
			UE_LOGF(LogWindowsDesktop, Verbose, "WM_IME_REQUEST - %ls", IMRStrings.Find(IntCastChecked<uint32>(wParam)) ? *(IMRStrings[(uint32)wParam]) : nullptr);
			DeferMessage( CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam );
			return 0;
			// Character
		case WM_CHAR:
			DeferMessage( CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam );
			return 0;
		case WM_SYSCHAR:
			{
				if (!bConsumeAltSpace && (HIWORD(lParam) & 0x2000) != 0 && wParam == VK_SPACE)
				{
					// Do not handle Alt+Space so that it passes through and opens the window system menu
				}
				else
				{
					return 0;
				}
			}
			
			break;

		case WM_SYSKEYDOWN:
			{
				// Alt-F4 or Alt+Space was pressed.
				if (wParam == VK_F4)
				{
					// Allow alt+f4 to close the window, but write a log warning
					UE_LOGF(LogWindowsDesktop, Log, "Alt-F4 pressed!");
				}
				// If we're consuming alt+space, pass it along
				else if (bConsumeAltSpace || wParam != VK_SPACE)
				{
					DeferMessage( CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam );
				}
			}
			break;

		case WM_CLIPBOARDUPDATE:
			{
				OnClipboardContentChangedEvent.Broadcast();
			}
			break;
		
		case WM_KEYDOWN:
		case WM_SYSKEYUP:
		case WM_KEYUP:
		case WM_LBUTTONDBLCLK:
		case WM_LBUTTONDOWN:
		case WM_MBUTTONDBLCLK:
		case WM_MBUTTONDOWN:
		case WM_RBUTTONDBLCLK:
		case WM_RBUTTONDOWN:
		case WM_XBUTTONDBLCLK:
		case WM_XBUTTONDOWN:
		case WM_XBUTTONUP:
		case WM_LBUTTONUP:
		case WM_MBUTTONUP:
		case WM_RBUTTONUP:
		case WM_NCMOUSEMOVE:
		case WM_MOUSEMOVE:
		case WM_MOUSEWHEEL:
		case WM_TOUCH:
			{
				DeferMessage( CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam );
				// Handled
				return 0;
			}

		case WM_POINTERDOWN:
		case WM_POINTERUPDATE:
		case WM_POINTERUP:
			{
				UINT32 PointerId = GET_POINTERID_WPARAM(wParam);
				POINTER_INPUT_TYPE Type;

				if (GetPointerType(PointerId, &Type))
				{
					// Touch input is handled using WM_POINTER instead of WM_TOUCH. WM_TOUCH generates mouse messages
					// for each touch, resulting in double input and flickering between touch and mouse input modes.
					// While you can filter some of the mouse events, there is always one additional WM_MOUSEMOVE that
					// can't be filtered out when using WM_TOUCH. The newer (Windows 8) WM_POINTER messages do not have
					// this issue.

					if (bUsePointerMessagesForTouch && Type == PT_TOUCH)
					{
						DeferMessage( CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam );
						// Handled
						return 0;
					}

					if (bUsePointerMessagesForPen && Type == PT_PEN)
					{
						DeferMessage( CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam );
						// Handled
						return 0;
					}
				}

				// Let Windows handle all other cases by forwarding to DefWindowProc where it generally translates to
				// other message types.
			}
			break;

		case WM_SETCURSOR:
		{
			DeferMessage(CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam);

			// If we're rendering our own window border, we'll "handle" this event so that Windows doesn't try to manage the cursor
			// appearance for us in the non-client area.  However, for OS window borders we need to fall through to DefWindowProc to
			// allow Windows to draw the resize cursor
			if (!CurrentNativeEventWindow->GetDefinition().HasOSWindowBorder)
			{
				// Handled
				return 0;
			}
		}
		break;

		// Mouse Movement
		case WM_INPUT:
			{
				uint32 Size = 0;
				if (::GetRawInputData((HRAWINPUT)lParam, RID_INPUT, NULL, &Size, sizeof(RAWINPUTHEADER)) == 0)
				{
					void* const RawData = FMemory_Alloca(Size);

					if (::GetRawInputData((HRAWINPUT)lParam, RID_INPUT, RawData, &Size, sizeof(RAWINPUTHEADER)) == Size)
					{
						const RAWINPUT* const Raw = static_cast<RAWINPUT*>(RawData);

						if (Raw->header.dwType == RIM_TYPEMOUSE)
						{
							RawMouseInputResult Result;
							ProcessRawMouseInput(Raw->data.mouse, Result);
							if (Result.bIsAbsoluteInput)
							{
								if (bSimulatingHighPrecisionMouseInputForRDP)
								{
									if (Result.bSimulateRelativeMovement)
									{
										DeferMessage(CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam, Result.DeltaX, Result.DeltaY, MOUSE_MOVE_RELATIVE);
									}
								}
								else
								{
									// Since the raw input is coming in as absolute it is likely the user is using a tablet
									// or perhaps is interacting through a virtual desktop
									DeferMessage(CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam, 0, 0, MOUSE_MOVE_ABSOLUTE);
								}
								return 1;
							}

							// Since raw input is coming in as relative it is likely a traditional mouse device
							const int xPosRelative = Result.DeltaX;
							const int yPosRelative = Result.DeltaY;

							DeferMessage(CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam, xPosRelative, yPosRelative, MOUSE_MOVE_RELATIVE);
							return 1;
						}
					}
				}
			}
			break;


		case WM_NCCALCSIZE:
			{
				// Let windows absorb this message if using the standard border
				if ( wParam && !CurrentNativeEventWindow->GetDefinition().HasOSWindowBorder )
				{
					// Borderless windows are not actually borderless: they have a thick border that we simply draw content over by making the client
					// rect contain that border. When maximized, however, Windows forces the border to bleed over the edges of the monitor. To avoid
					// drawing content outside of the viewable area, we counter this behavior by adjusting the size back inwards by the bleed amount,
					// making the window and client rects again align exactly with the viewable area. The size adjustments here will be sent to
					// WM_MOVE and WM_SIZE, but the window will still be considered maximized.
					if (CurrentNativeEventWindow->IsMaximized())
					{
						// Ask the system for the window border size as this is the amount that Windows will bleed our window over the edge
						// of our desired space. The value returned by CurrentNativeEventWindow will be incorrect for our usage here as it
						// refers to the border of the window that Slate should consider.
						WINDOWINFO WindowInfo;
						FMemory::Memzero(WindowInfo);
						WindowInfo.cbSize = sizeof(WindowInfo);
						::GetWindowInfo(hwnd, &WindowInfo);

						// A pointer to the window size data that Windows will use is passed to us in lParam
						LPNCCALCSIZE_PARAMS ResizingRects = (LPNCCALCSIZE_PARAMS)lParam;

						// The first rectangle initially contains the new window size; we are expected to overwrite this with the desired client size.
						// Make the client inset on all sides by the bleed amount.
						ResizingRects->rgrc[0].left += WindowInfo.cxWindowBorders;
						ResizingRects->rgrc[0].top += WindowInfo.cxWindowBorders;
						ResizingRects->rgrc[0].right -= WindowInfo.cxWindowBorders;
						ResizingRects->rgrc[0].bottom -= WindowInfo.cxWindowBorders;
						// The second rectangle contains the destination rectangle for the content currently displayed in the window's
						// client rect. Windows will blit the previous client content into this new location to simulate the move of
						// the window until the window can repaint itself. This should also be adjusted to our new window size.
						ResizingRects->rgrc[1].left = ResizingRects->rgrc[0].left;
						ResizingRects->rgrc[1].top = ResizingRects->rgrc[0].top;
						ResizingRects->rgrc[1].right = ResizingRects->rgrc[0].right;
						ResizingRects->rgrc[1].bottom = ResizingRects->rgrc[0].bottom;
						// A third rectangle is passed in that contains the source rectangle (client area from window pre-maximize).
						// It's value should not be changed.

						// Informs Windows to use the values as we altered them.
						return WVR_VALIDRECTS;
					}
					else
					{
						//////////////////////////////////////////////////////
						// Find out the schedule of when composition takes place 
						// so rendering events are slotted into the queue at appropriate moments.
						// This fixes bad flicker problems when resizing.
						//////////////////////////////////////////////////////

						LARGE_INTEGER Freq;
						QueryPerformanceFrequency(&Freq);

						// Ask DWM where the vertical blank falls
						DWM_TIMING_INFO TimingInfo;
						memset(&TimingInfo, 0, sizeof(TimingInfo));
						TimingInfo.cbSize = sizeof(TimingInfo);
						if (FAILED(DwmGetCompositionTimingInfo(NULL, &TimingInfo)))
						{
							return 0;
						}

						LARGE_INTEGER Now;
						QueryPerformanceCounter(&Now);

						// DWM told us about SOME vertical blank,
						// past or future, possibly many frames away.
						// Convert that into the NEXT vertical blank

						int64 Period = (int64)TimingInfo.qpcRefreshPeriod;

						int64 TimeToVSync = (int64)TimingInfo.qpcVBlank - (int64)Now.QuadPart;

						int64 FrameAdjustMultiplier;

						if (TimeToVSync >= 0)
						{
							FrameAdjustMultiplier = TimeToVSync / Period;
						}
						else
						{
							// Reach back to previous period
							// so WaitTime represents consistent position within phase
							FrameAdjustMultiplier = -1 + TimeToVSync / Period;
						}

						int64 WaitTime = TimeToVSync - (Period * FrameAdjustMultiplier);
						if (WaitTime == Period)
						{
							// this can happen when TimeToVSync is negative and a multiple of Period. 
							// It means that the Vsync is about to happen in the next 1 ms.
							// In this case, we wait 0 instead of waiting for the next Vsync after that one
							WaitTime = 0;
						}

						check(WaitTime >= 0);
						check(WaitTime < Period); 

						// Wait for the indicated time using a waitable timer as it 
						// is more accurate than a simple sleep.
						HANDLE hTimer = CreateWaitableTimer(NULL, TRUE, NULL);
						if (NULL != hTimer)
						{
							double WaitTimeMilliseconds = 0.0;
							if (Freq.QuadPart > 0)
							{
								WaitTimeMilliseconds = 1000.0 * (double)WaitTime / (double)Freq.QuadPart;
							}

							// Due time for WaitForSingleObject is in 100 nanosecond units.							
							double WaitTime100NanoSeconds = (1000.0f * 10.0f * WaitTimeMilliseconds);

							LARGE_INTEGER DueTime;

							// Negative value indicates relative time in the future.
							DueTime.QuadPart = (LONGLONG)(WaitTime100NanoSeconds) * -1;

							if (SetWaitableTimer(hTimer, &DueTime, 0, NULL, NULL, 0))
							{
								// Timeout time (second param) is in milliseconds. Set it to a bit longer than
								// the timer due time. If everything is working it won't be used.
								WaitForSingleObject(hTimer, (DWORD)(WaitTimeMilliseconds) + 10);
							}

							CloseHandle(hTimer);
						}

						return 0;
					}
				}
			}
			break;

		case WM_SHOWWINDOW:
			{
				DeferMessage( CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam );
			}
			break;

		case WM_SIZE:
			{
				DeferMessage( CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam );

				const bool bWasMaximized = (wParam == SIZE_MAXIMIZED);
				const bool bWasRestored = (wParam == SIZE_RESTORED);

				if (bWasMaximized || bWasRestored)
				{
					MessageHandler->OnWindowAction(CurrentNativeEventWindow, bWasMaximized ? EWindowAction::Maximize : EWindowAction::Restore);
				}
				if (CurrentNativeEventWindowPtr->IsInModalSizeLoop())
				{
					MessageHandler->OnRegularWindowMovedOrResized(CurrentNativeEventWindowPtr.ToSharedRef());
				}

				return 0;
			}

		case WM_SIZING:
			{
				DeferMessage( CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam, 0, 0 );
		
				if (CurrentNativeEventWindowPtr->GetDefinition().ShouldPreserveAspectRatio)
				{
					// The rect we get in lParam is window rect, but we need to preserve client's aspect ratio,
					// so we need to find what the border and title bar sizes are, if window has them and adjust the rect.
					WINDOWINFO WindowInfo;
					FMemory::Memzero(WindowInfo);
					WindowInfo.cbSize = sizeof(WindowInfo);
					::GetWindowInfo(hwnd, &WindowInfo);

					RECT* Rect = (RECT*)lParam;
					if (CurrentNativeEventWindow->GetDefinition().HasOSWindowBorder)
					{
						RECT TestRect;
						TestRect.left = TestRect.right = TestRect.top = TestRect.bottom = 0;
						AdjustWindowRectEx(&TestRect, WindowInfo.dwStyle, false, WindowInfo.dwExStyle);

					
						Rect->left -= TestRect.left;
						Rect->right -= TestRect.right;
						Rect->top -= TestRect.top;
						Rect->bottom -= TestRect.bottom;
					}

					const float AspectRatio = CurrentNativeEventWindowPtr->GetAspectRatio();
					int32 NewWidth = Rect->right - Rect->left;
					int32 NewHeight = Rect->bottom - Rect->top;

					FWindowSizeLimits SizeLimits = MessageHandler->GetSizeLimitsForWindow(CurrentNativeEventWindow);

					switch (wParam)
					{
					case WMSZ_LEFT:
					case WMSZ_RIGHT:
					case WMSZ_BOTTOMLEFT:
					case WMSZ_BOTTOMRIGHT:
					case WMSZ_TOPLEFT:
					case WMSZ_TOPRIGHT:
					{
						int32 MinWidth = (int32)SizeLimits.GetMinWidth().GetValue();
						if (SizeLimits.GetMinHeight().GetValue() < SizeLimits.GetMinWidth().GetValue())
						{
							MinWidth = (int32)(SizeLimits.GetMinHeight().GetValue() * AspectRatio);
						}

						if (NewWidth < MinWidth)
						{
							if (wParam == WMSZ_LEFT || wParam == WMSZ_BOTTOMLEFT || wParam == WMSZ_TOPLEFT)
							{
								Rect->left -= (MinWidth - NewWidth);
							}
							else if (wParam == WMSZ_RIGHT || wParam == WMSZ_BOTTOMRIGHT || wParam == WMSZ_TOPRIGHT)
							{
								Rect->right += (MinWidth - NewWidth);
							}

							NewWidth = MinWidth;
						}

						break;
					}
					case WMSZ_TOP:
					case WMSZ_BOTTOM:
					{
						int32 MinHeight = (int32)SizeLimits.GetMinHeight().GetValue();
						if (SizeLimits.GetMinWidth().GetValue() < SizeLimits.GetMinHeight().GetValue())
						{
							MinHeight = (int32)(SizeLimits.GetMinWidth().GetValue() / AspectRatio);
						}

						if (NewHeight < MinHeight)
						{
							if (wParam == WMSZ_TOP)
							{
								Rect->top -= (MinHeight - NewHeight);
							}
							else
							{
								Rect->bottom += (MinHeight - NewHeight);
							}

							NewHeight = MinHeight;
						}
						break;
					}
					}

					switch (wParam)
					{
					case WMSZ_LEFT:
					case WMSZ_RIGHT:
						{
							int32 AdjustedHeight = (int32)((float)NewWidth / AspectRatio);
							Rect->top -= (AdjustedHeight - NewHeight) / 2;
							Rect->bottom += (AdjustedHeight - NewHeight) / 2;
							break;
						}
					case WMSZ_TOP:
					case WMSZ_BOTTOM:
						{
							int32 AdjustedWidth = (int32)((float)NewHeight * AspectRatio);
							Rect->left -= (AdjustedWidth - NewWidth) / 2;
							Rect->right += (AdjustedWidth - NewWidth) / 2;
							break;
						}
					case WMSZ_TOPLEFT:
						{
							int32 AdjustedHeight = (int32)((float)NewWidth / AspectRatio);
							Rect->top -= AdjustedHeight - NewHeight;
							break;
						}
					case WMSZ_TOPRIGHT:
						{
							int32 AdjustedHeight = (int32)((float)NewWidth / AspectRatio);
							Rect->top -= AdjustedHeight - NewHeight;
							break;
						}
					case WMSZ_BOTTOMLEFT:
						{
							int32 AdjustedHeight = (int32)((float)NewWidth / AspectRatio);
							Rect->bottom += AdjustedHeight - NewHeight;
							break;
						}
					case WMSZ_BOTTOMRIGHT:
						{
							int32 AdjustedHeight = (int32)((float)NewWidth / AspectRatio);
							Rect->bottom += AdjustedHeight - NewHeight;
							break;
						}
					}

					if (CurrentNativeEventWindow->GetDefinition().HasOSWindowBorder)
					{
						AdjustWindowRectEx(Rect, WindowInfo.dwStyle, false, WindowInfo.dwExStyle);
					}

					return TRUE;
				}
		}
			break;
		case WM_ENTERSIZEMOVE:
			{
				CurrentNativeEventWindowPtr->EnterModalSizeLoop();
				DeferMessage( CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam, 0, 0 );
			}
			break;
		case WM_EXITSIZEMOVE:
			{
				CurrentNativeEventWindowPtr->ExitModalSizeLoop();
				DeferMessage( CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam, 0, 0 );
			}
			break;


		case WM_MOVE:
			{
				// client area position
				const int32 NewX = (int)(short)(LOWORD(lParam));
				const int32 NewY = (int)(short)(HIWORD(lParam));
				FIntPoint NewPosition(NewX,NewY);

				// Only cache the screen position if its not minimized
				if ( FWindowsApplication::MinimizedWindowPosition != NewPosition )
				{
					MessageHandler->OnMovedWindow( CurrentNativeEventWindow, NewX, NewY );
				}
				if (CurrentNativeEventWindowPtr->IsInModalSizeLoop())
				{
					MessageHandler->OnRegularWindowMovedOrResized(CurrentNativeEventWindow);
				}

				return 0;
			}

		case WM_NCHITTEST:
			{
				// Only needed if not using the os window border as this is determined automatically
				if( !CurrentNativeEventWindow->GetDefinition().HasOSWindowBorder )
				{
					// Use the client area origin rather than the window rect origin so that
					// LocalMouseX/Y aligns with the coordinate system Slate uses for ScreenPosition
					// (set from WM_MOVE, which reports the client area top-left).
					// For non-maximized borderless windows the two are identical, but when maximized
					// WM_NCCALCSIZE inserts the client rect inward by the resize-border bleed amount,
					// so GetWindowRect().top != client origin, causing an 8-px hit-test offset.
					POINT ClientOrigin = { 0, 0 };
					::ClientToScreen(hwnd, &ClientOrigin);

					const int32 LocalMouseX = (int)(short)(LOWORD(lParam)) - ClientOrigin.x;
					const int32 LocalMouseY = (int)(short)(HIWORD(lParam)) - ClientOrigin.y;
					if ( CurrentNativeEventWindow->IsRegularWindow() )
					{
						EWindowZone::Type Zone;
					
						if( MessageHandler->ShouldProcessUserInputMessages( CurrentNativeEventWindowPtr ) )
						{
							// Assumes this is not allowed to leave Slate or touch rendering
							Zone = MessageHandler->GetWindowZoneForPoint( CurrentNativeEventWindow, LocalMouseX, LocalMouseY );
						}
						else
						{
							// Default to client area so that we are able to see the feedback effect when attempting to click on a non-modal window when a modal window is active
							// Any other window zones could have side effects and NotInWindow prevents the feedback effect.
							Zone = EWindowZone::ClientArea;
						}

						static const LRESULT Results[] = {HTNOWHERE, HTCLIENT, HTTOPLEFT, HTTOP, HTTOPRIGHT, HTLEFT,
							HTRIGHT, HTBOTTOMLEFT, HTBOTTOM, HTBOTTOMRIGHT,
							HTCAPTION, HTMINBUTTON, HTMAXBUTTON, HTCLOSE, HTSYSMENU};

						return IntCastChecked<int32>(Results[Zone]);
					}
				}
			}
			break;
			
		case WM_DWMCOMPOSITIONCHANGED:
			{
				DeferMessage( CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam );
			}
			break;

			// Window focus and activation
		case WM_MOUSEACTIVATE:
			{
				DeferMessage(CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam);
			}
			break;

			// Window focus and activation
		case WM_ACTIVATE:
			{
				DeferMessage( CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam );
			}
			break;

		case WM_ACTIVATEAPP:
			{
				// When window activation changes we are not in a modal size loop
				// NOTE: WM_EXITSIZEMOVE might be reliable these days, but keep this around for now.
				CurrentNativeEventWindowPtr->ExitModalSizeLoop();

				DeferMessage( CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam );
				return 0;
			}

		case WM_SETTINGCHANGE:
			{
				// Convertible mode change
				if ((lParam != NULL) && (wcscmp(TEXT("ConvertibleSlateMode"), (TCHAR *)lParam) == 0))
				{
					DeferMessage(CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam);
				}
				if (wParam == UE::WindowsApplication::TouchpadSensitivity::SPI_SETTOUCHPADPARAMETERS)
				{
					DeferMessage(CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam);
				}
			}
			break;

		case WM_PAINT:
			{
				check(IsInGameThread());
				if (CurrentNativeEventWindowPtr->IsInModalSizeLoop())
				{
					MessageHandler->OnOSPaint(CurrentNativeEventWindowPtr.ToSharedRef());
				}
			}
			break;

		case WM_ERASEBKGND:
			{
				// Intercept background erasing to eliminate flicker.
				// Return non-zero to indicate that we'll handle the erasing ourselves
				return 1;
			}

		case WM_NCACTIVATE:
			{
				if( !CurrentNativeEventWindow->GetDefinition().HasOSWindowBorder )
				{
					// Unless using the OS window border, intercept calls to prevent non-client area drawing a border upon activation or deactivation
					// Return true to ensure standard activation happens
					return true;
				}
			}
			break;

		case WM_NCPAINT:
			{
				if( !CurrentNativeEventWindow->GetDefinition().HasOSWindowBorder )
				{
					// Unless using the OS window border, intercept calls to draw the non-client area - we do this ourselves
					return 0;
				}
			}
			break;

		case WM_DESTROY:
			{
				Windows.Remove( CurrentNativeEventWindow );
#if WITH_ACCESSIBILITY && UE_WINDOWS_USING_UIA
				// Tell UIA that the window no longer exists so that it can release some resources
				if (GetAccessibleMessageHandler()->ApplicationIsAccessible())
				{
					UiaReturnRawElementProvider(hwnd, 0, 0, nullptr);
				}
#endif
				return 0;
			}

		case WM_CLOSE:
			{
				DeferMessage( CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam );
				return 0;
			}

		case WM_ENDSESSION:
			{
				// wParam is true if the user session is going away. Note that WM_SESSION is a follow up for WM_QUERYENDSESSION, so wParam can be false if the user (from UI)
				// or another application (from WM_QUERYENDSESSION) canceled the shutdown.
				if (wParam == TRUE) // Shutdown/Reboot/Logoff
				{
					FCoreDelegates::OnUserLoginChangedEvent.Broadcast(false, 0, 0);
				}
				return IntCastChecked<int32>(DefWindowProc(hwnd, msg, wParam, lParam));
			}

		case WM_SYSCOMMAND:
			{
				switch( wParam & 0xfff0 )
				{
				case SC_RESTORE:
					// Checks to see if the window is minimized.
					if( IsIconic(hwnd) )
					{
						// This is required for restoring a minimized fullscreen window
						::ShowWindow(hwnd,SW_RESTORE);
						return 0;
					}
					else
					{
						if(!MessageHandler->OnWindowAction( CurrentNativeEventWindow, EWindowAction::Restore))
						{
							return 1;
						}
					}
					break;
				case SC_MAXIMIZE:
					{
						if(!MessageHandler->OnWindowAction( CurrentNativeEventWindow, EWindowAction::Maximize))
						{
							return 1;
						}
					}
					break;
				case SC_CLOSE:
					{
						// Do not allow Alt-f4 during slow tasks, or slate's loading thread.  This causes entry into the shutdown sequence at abnormal times which causes crashes.
						if (!GIsSlowTask && GSlateLoadingThreadId == 0)
						{
							DeferMessage(CurrentNativeEventWindowPtr, hwnd, WM_CLOSE, 0, 0);
						}
						return 1;
					}
				default:
					if( !( MessageHandler->ShouldProcessUserInputMessages( CurrentNativeEventWindow ) && IsInputMessage( msg ) ) )
					{
						return 0;
					}
					break;
				}
			}
			break;

		case WM_GETMINMAXINFO:
			{
				MINMAXINFO* MinMaxInfo = (MINMAXINFO*)lParam;
				FWindowSizeLimits SizeLimits = MessageHandler->GetSizeLimitsForWindow(CurrentNativeEventWindow);

				// We need to inflate the max values if using an OS window border
				int32 BorderWidth = 0;
				int32 BorderHeight = 0;
				if (CurrentNativeEventWindow->GetDefinition().HasOSWindowBorder)
				{
					const DWORD WindowStyle = ::GetWindowLong(hwnd, GWL_STYLE);
					const DWORD WindowExStyle = ::GetWindowLong(hwnd, GWL_EXSTYLE);

					// This adjusts a zero rect to give us the size of the border
					RECT BorderRect = { 0, 0, 0, 0 };
					::AdjustWindowRectEx(&BorderRect, WindowStyle, false, WindowExStyle);

					BorderWidth = BorderRect.right - BorderRect.left;
					BorderHeight = BorderRect.bottom - BorderRect.top;
				}

				const float DPIScaleFactor = CurrentNativeEventWindow->GetDPIScaleFactor();

				// We always apply BorderWidth and BorderHeight since Slate always works with client area window sizes
				MinMaxInfo->ptMinTrackSize.x = FMath::RoundToInt( SizeLimits.GetMinWidth().Get((float)MinMaxInfo->ptMinTrackSize.x) * DPIScaleFactor );
				MinMaxInfo->ptMinTrackSize.y = FMath::RoundToInt( SizeLimits.GetMinHeight().Get((float)MinMaxInfo->ptMinTrackSize.y) * DPIScaleFactor );
				MinMaxInfo->ptMaxTrackSize.x = FMath::RoundToInt( SizeLimits.GetMaxWidth().Get((float)MinMaxInfo->ptMaxTrackSize.x) * DPIScaleFactor ) + BorderWidth;
				MinMaxInfo->ptMaxTrackSize.y = FMath::RoundToInt( SizeLimits.GetMaxHeight().Get((float)MinMaxInfo->ptMaxTrackSize.y) * DPIScaleFactor ) + BorderHeight;
				return 0;
			}
			
		case WM_NCLBUTTONDOWN:
		case WM_NCRBUTTONDOWN:
		case WM_NCMBUTTONDOWN:
			{
				switch( wParam )
				{
				case HTMINBUTTON:
					{
						if(!MessageHandler->OnWindowAction( CurrentNativeEventWindow, EWindowAction::ClickedNonClientArea))
						{
							return 1;
						}
					}
					break;
				case HTMAXBUTTON:
					{
						if(!MessageHandler->OnWindowAction( CurrentNativeEventWindow, EWindowAction::ClickedNonClientArea))
						{
							return 1;
						}
					}
					break;
				case HTCLOSE:
					{
						if(!MessageHandler->OnWindowAction( CurrentNativeEventWindow, EWindowAction::ClickedNonClientArea))
						{
							return 1;
						}
					}
					break;
				case HTCAPTION:
					{
						if(!MessageHandler->OnWindowAction( CurrentNativeEventWindow, EWindowAction::ClickedNonClientArea))
						{
							return 1;
						}
					}
					break;
				}
			}
			break;

		case WM_DISPLAYCHANGE:
			{
				// Slate needs to know when desktop size changes.
				FDisplayMetrics DisplayMetrics;
				FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);
				BroadcastDisplayMetricsChanged(DisplayMetrics);
			}
			break;

		case WM_DPICHANGED:
			DeferMessage(CurrentNativeEventWindowPtr, hwnd, msg, wParam, lParam);
			break;

		case WM_GETDLGCODE:
			{
				// Slate wants all keys and messages.
				return DLGC_WANTALLKEYS;
			}
		
		case WM_CREATE:
			return 0;

		case WM_DEVICECHANGE:
			{
				static const FInputDeviceProperty RequestUpdateProp(TEXT("Request_Device_Update"));
				for( auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt )
				{
					// Pass in -1 because this isn't for any specific device, it is just a message requesting an update to any attached controllers.
					// At the moment, this will only be listened for by XInput.
					(*DeviceIt)->SetDeviceProperty(-1, &RequestUpdateProp);
				}
				
				QueryConnectedMice();
			}
			break;

		case WM_CAPTURECHANGED:
			{
				// This will usually be null, but we'll store it anyway in case capture is switching between two windows within the same process.
				// Otherwise we'd risk clobbering the cached handle from the new capturing window
				CaptureWindowHandle = (HWND)lParam;
			}
			break;

#if WITH_ACCESSIBILITY && UE_WINDOWS_USING_UIA
		case WM_GETOBJECT:
		{
			if (GetAccessibleMessageHandler()->ApplicationIsAccessible())
			{
				FScopedWidgetProvider Provider(UIAManager->GetWindowProvider(CurrentNativeEventWindow));
				LRESULT Result = UiaReturnRawElementProvider(hwnd, wParam, lParam, &Provider.Provider);
				return IntCastChecked<int32>(Result);
			}
			break;
		}
#endif

		default:
			if (bMessageExternallyHandled)
			{
				return ExternalMessageHandlerResult;
			}
		}
	}

	return IntCastChecked<int32>(DefWindowProc(hwnd, msg, wParam, lParam));
}

void FWindowsApplication::CheckForShiftUpEvents(const int32 KeyCode)
{
	check(KeyCode == VK_LSHIFT || KeyCode == VK_RSHIFT);

	// Since VK_SHIFT doesn't get an up message if the other shift key is held we need to poll for it
	const EModifierKey::Type ModifierKeyIndex = KeyCode == VK_LSHIFT ? EModifierKey::LeftShift : EModifierKey::RightShift;
	if (ModifierKeyState[ModifierKeyIndex] && ((::GetKeyState(KeyCode) & 0x8000) == 0) )
	{
		ModifierKeyState[ModifierKeyIndex] = false;
		
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FInputDeviceScope InputScope(nullptr, UE::WindowsInput::InputClassName, IPlatformInputDeviceMapper::Get().GetDefaultInputDevice().GetId(), UE::WindowsInput::KBMInputHardwareName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		MessageHandler->OnKeyUp( KeyCode, 0, false );
	}
}

int32 FWindowsApplication::ProcessDeferredMessage( const FDeferredWindowsMessage& DeferredMessage )
{
	if ( Windows.Num() && DeferredMessage.NativeWindow.IsValid() )
	{
		HWND hwnd = DeferredMessage.hWND;
		uint32 msg = DeferredMessage.Message;
		WPARAM wParam = DeferredMessage.wParam;
		LPARAM lParam = DeferredMessage.lParam;

		TSharedPtr< FWindowsWindow > CurrentNativeEventWindowPtr = DeferredMessage.NativeWindow.Pin();

		// This effectively disables a window without actually disabling it natively with the OS.
		// This allows us to continue receiving messages for it.
		if ( !MessageHandler->ShouldProcessUserInputMessages( CurrentNativeEventWindowPtr ) && IsInputMessage( msg ) )
		{
			if (IsKeyboardInputMessage(msg))
			{
				// Force an update since we may have just consumed a modifier key state change
				UpdateAllModifierKeyStates();
			}
			return 0;	// consume input messages
		}

		switch(msg)
		{
		case WM_INPUTLANGCHANGEREQUEST:
		case WM_INPUTLANGCHANGE:
		case WM_IME_SETCONTEXT:
		case WM_IME_NOTIFY:
		case WM_IME_REQUEST:
		case WM_IME_STARTCOMPOSITION:
		case WM_IME_COMPOSITION:
		case WM_IME_ENDCOMPOSITION:
		case WM_IME_CHAR:
			{
				if(TextInputMethodSystem.IsValid())
				{
					TextInputMethodSystem->ProcessMessage(hwnd, msg, wParam, lParam);
				}
				if (msg == WM_INPUTLANGCHANGE)
				{
					MessageHandler->OnInputLanguageChanged();
				}
				return 0;
			}
			// Character
		case WM_CHAR:
			{
				// Character code is stored in WPARAM
				const TCHAR Character = IntCastChecked<TCHAR>(wParam);

				// LPARAM bit 30 will be ZERO for new presses, or ONE if this is a repeat
				const bool bIsRepeat = ( lParam & 0x40000000 ) != 0;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FInputDeviceScope InputScope(nullptr, UE::WindowsInput::InputClassName, IPlatformInputDeviceMapper::Get().GetDefaultInputDevice().GetId(), UE::WindowsInput::KBMInputHardwareName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				MessageHandler->OnKeyChar( Character, bIsRepeat );

				// Note: always return 0 to handle the message.  Win32 beeps if WM_CHAR is not handled...
				return 0;
			}


			// Key down
		case WM_SYSKEYDOWN:
		case WM_KEYDOWN:
			{
				// Character code is stored in WPARAM
				const int32 Win32Key = IntCastChecked<int32>(wParam);

				// The actual key to use.  Some keys will be translated into other keys. 
				// I.E VK_CONTROL will be translated to either VK_LCONTROL or VK_RCONTROL as these
				// keys are never sent on their own
				int32 ActualKey = Win32Key;

				// LPARAM bit 30 will be ZERO for new presses, or ONE if this is a repeat
				bool bIsRepeat = ( lParam & 0x40000000 ) != 0;

				switch( Win32Key )
				{
				case VK_MENU:
					// Differentiate between left and right alt
					if( (lParam & 0x1000000) == 0 )
					{
						ActualKey = VK_LMENU;
						bIsRepeat = ModifierKeyState[EModifierKey::LeftAlt];
						ModifierKeyState[EModifierKey::LeftAlt] = true;
					}
					else
					{
						ActualKey = VK_RMENU;
						bIsRepeat = ModifierKeyState[EModifierKey::RightAlt];
						ModifierKeyState[EModifierKey::RightAlt] = true;
					}
					break;
				case VK_CONTROL:
					// Differentiate between left and right control
					if( (lParam & 0x1000000) == 0 )
					{
						ActualKey = VK_LCONTROL;
						bIsRepeat = ModifierKeyState[EModifierKey::LeftControl];
						ModifierKeyState[EModifierKey::LeftControl] = true;
					}
					else
					{
						ActualKey = VK_RCONTROL;
						bIsRepeat = ModifierKeyState[EModifierKey::RightControl];
						ModifierKeyState[EModifierKey::RightControl] = true;
					}
					break;
				case VK_SHIFT:
					// Differentiate between left and right shift
					ActualKey = MapVirtualKey( (lParam & 0x00ff0000) >> 16, MAPVK_VSC_TO_VK_EX);
					if (ActualKey == VK_LSHIFT)
					{
						bIsRepeat = ModifierKeyState[EModifierKey::LeftShift];
						ModifierKeyState[EModifierKey::LeftShift] = true;
					}
					else
					{
						bIsRepeat = ModifierKeyState[EModifierKey::RightShift];
						ModifierKeyState[EModifierKey::RightShift] = true;
					}
					break;
				case VK_CAPITAL:
					ModifierKeyState[EModifierKey::CapsLock] = (::GetKeyState(VK_CAPITAL) & 0x0001) != 0;
					break;
				default:
					// No translation needed
					break;
				}

				// Get the character code from the virtual key pressed.  If 0, no translation from virtual key to character exists
				uint32 CharCode = ::MapVirtualKey( Win32Key, MAPVK_VK_TO_CHAR );
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FInputDeviceScope InputScope(nullptr, UE::WindowsInput::InputClassName, IPlatformInputDeviceMapper::Get().GetDefaultInputDevice().GetId(), UE::WindowsInput::KBMInputHardwareName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				const bool Result = MessageHandler->OnKeyDown( ActualKey, CharCode, bIsRepeat );

				// Always return 0 to handle the message or else windows will beep
				if( Result || msg != WM_SYSKEYDOWN )
				{
					// Handled
					return 0;
				}
			}
			break;


			// Key up
		case WM_SYSKEYUP:
		case WM_KEYUP:
			{
				// Character code is stored in WPARAM
				int32 Win32Key = IntCastChecked<int32>(wParam);

				// The actual key to use.  Some keys will be translated into other keys. 
				// I.E VK_CONTROL will be translated to either VK_LCONTROL or VK_RCONTROL as these
				// keys are never sent on their own
				int32 ActualKey = Win32Key;

				bool bModifierKeyReleased = false;
				switch( Win32Key )
				{
				case VK_MENU:
					// Differentiate between left and right alt
					if( (lParam & 0x1000000) == 0 )
					{
						ActualKey = VK_LMENU;
						ModifierKeyState[EModifierKey::LeftAlt] = false;
					}
					else
					{
						ActualKey = VK_RMENU;
						ModifierKeyState[EModifierKey::RightAlt] = false;
					}
					break;
				case VK_CONTROL:
					// Differentiate between left and right control
					if( (lParam & 0x1000000) == 0 )
					{
						ActualKey = VK_LCONTROL;
						ModifierKeyState[EModifierKey::LeftControl] = false;
					}
					else
					{
						ActualKey = VK_RCONTROL;
						ModifierKeyState[EModifierKey::RightControl] = false;
					}
					break;
				case VK_SHIFT:
					// Differentiate between left and right shift
					ActualKey = MapVirtualKey( (lParam & 0x00ff0000) >> 16, MAPVK_VSC_TO_VK_EX);
					if (ActualKey == VK_LSHIFT)
					{
						ModifierKeyState[EModifierKey::LeftShift] = false;
					}
					else
					{
						ModifierKeyState[EModifierKey::RightShift] = false;
					}
					break;
				case VK_CAPITAL:
					ModifierKeyState[EModifierKey::CapsLock] = (::GetKeyState(VK_CAPITAL) & 0x0001) != 0;
					break;
				default:
					// No translation needed
					break;
				}

				// Get the character code from the virtual key pressed.  If 0, no translation from virtual key to character exists
				uint32 CharCode = ::MapVirtualKey( Win32Key, MAPVK_VK_TO_CHAR );

				// Key up events are never repeats
				const bool bIsRepeat = false;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FInputDeviceScope InputScope(nullptr, UE::WindowsInput::InputClassName, IPlatformInputDeviceMapper::Get().GetDefaultInputDevice().GetId(), UE::WindowsInput::KBMInputHardwareName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				const bool Result = MessageHandler->OnKeyUp( ActualKey, CharCode, bIsRepeat );

				// Note that we allow system keys to pass through to DefWndProc here, so that core features
				// like Alt+F4 to close a window work.
				if( Result || msg != WM_SYSKEYUP )
				{
					// Handled
					return 0;
				}
			}
			break;

			// Mouse Button Down
		case WM_LBUTTONDBLCLK:
		case WM_LBUTTONDOWN:
		case WM_MBUTTONDBLCLK:
		case WM_MBUTTONDOWN:
		case WM_RBUTTONDBLCLK:
		case WM_RBUTTONDOWN:
		case WM_XBUTTONDBLCLK:
		case WM_XBUTTONDOWN:
		case WM_LBUTTONUP:
		case WM_MBUTTONUP:
		case WM_RBUTTONUP:
		case WM_XBUTTONUP:
			{
				POINT CursorPoint;
				CursorPoint.x = GET_X_LPARAM(lParam);
				CursorPoint.y = GET_Y_LPARAM(lParam); 

				ClientToScreen(hwnd, &CursorPoint);

				const FVector2D CursorPos(CursorPoint.x, CursorPoint.y);

				EMouseButtons::Type MouseButton = EMouseButtons::Invalid;
				bool bDoubleClick = false;
				bool bMouseUp = false;
				switch(msg)
				{
				case WM_LBUTTONDBLCLK:
					bDoubleClick = true;
					MouseButton = EMouseButtons::Left;
					break;
				case WM_LBUTTONUP:
					bMouseUp = true;
					MouseButton = EMouseButtons::Left;
					break;
				case WM_LBUTTONDOWN:
					MouseButton = EMouseButtons::Left;
					break;
				case WM_MBUTTONDBLCLK:
					bDoubleClick = true;
					MouseButton = EMouseButtons::Middle;
					break;
				case WM_MBUTTONUP:
					bMouseUp = true;
					MouseButton = EMouseButtons::Middle;
					break;
				case WM_MBUTTONDOWN:
					MouseButton = EMouseButtons::Middle;
					break;
				case WM_RBUTTONDBLCLK:
					bDoubleClick = true;
					MouseButton = EMouseButtons::Right;
					break;
				case WM_RBUTTONUP:
					bMouseUp = true;
					MouseButton = EMouseButtons::Right;
					break;
				case WM_RBUTTONDOWN:
					MouseButton = EMouseButtons::Right;
					break;
				case WM_XBUTTONDBLCLK:
					bDoubleClick = true;
					MouseButton = ( HIWORD(wParam) & XBUTTON1 ) ? EMouseButtons::Thumb01  : EMouseButtons::Thumb02;
					break;
				case WM_XBUTTONUP:
					bMouseUp = true;
					MouseButton = ( HIWORD(wParam) & XBUTTON1 ) ? EMouseButtons::Thumb01  : EMouseButtons::Thumb02;
					break;
				case WM_XBUTTONDOWN:
					MouseButton = ( HIWORD(wParam) & XBUTTON1 ) ? EMouseButtons::Thumb01  : EMouseButtons::Thumb02;
					break;
				default:
					check(0);
				}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FInputDeviceScope InputScope(nullptr, UE::WindowsInput::InputClassName, IPlatformInputDeviceMapper::Get().GetDefaultInputDevice().GetId(), UE::WindowsInput::KBMInputHardwareName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				if (bMouseUp)
				{
					return MessageHandler->OnMouseUp( MouseButton, CursorPos ) ? 0 : 1;
				}
				else if (bDoubleClick)
				{
					MessageHandler->OnMouseDoubleClick( CurrentNativeEventWindowPtr, MouseButton, CursorPos );
				}
				else
				{
					MessageHandler->OnMouseDown( CurrentNativeEventWindowPtr, MouseButton, CursorPos );
				}
				return 0;
			}

		// Mouse Movement
		case WM_INPUT:
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FInputDeviceScope InputScope(nullptr, UE::WindowsInput::InputClassName, IPlatformInputDeviceMapper::Get().GetDefaultInputDevice().GetId(), UE::WindowsInput::KBMInputHardwareName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				if( DeferredMessage.RawInputFlags == MOUSE_MOVE_RELATIVE )
				{
					MessageHandler->OnRawMouseMove(DeferredMessage.X, DeferredMessage.Y);
				}
				else
				{
					// Absolute coordinates given through raw input are simulated using MouseMove to get relative coordinates
					MessageHandler->OnMouseMove();
				}

				return 0;
			}

		// Mouse Movement
		case WM_NCMOUSEMOVE:
		case WM_MOUSEMOVE:
			{
				BOOL Result = false;
				if (!bUsingHighPrecisionMouseInput)
				{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FInputDeviceScope InputScope(nullptr, UE::WindowsInput::InputClassName, IPlatformInputDeviceMapper::Get().GetDefaultInputDevice().GetId(), UE::WindowsInput::KBMInputHardwareName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
					Result = MessageHandler->OnMouseMove();
				}

				return Result ? 0 : 1;
			}
			// Mouse Wheel
		case WM_MOUSEWHEEL:
			{
				const float SpinFactor = 1 / 120.0f;
				const SHORT WheelDelta = GET_WHEEL_DELTA_WPARAM(wParam);

				POINT CursorPoint;
				CursorPoint.x = GET_X_LPARAM(lParam);
				CursorPoint.y = GET_Y_LPARAM(lParam); 

				const FVector2D CursorPos(CursorPoint.x, CursorPoint.y);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FInputDeviceScope InputScope(nullptr, UE::WindowsInput::InputClassName, IPlatformInputDeviceMapper::Get().GetDefaultInputDevice().GetId(), UE::WindowsInput::KBMInputHardwareName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				const BOOL Result = MessageHandler->OnMouseWheel( static_cast<float>( WheelDelta ) * SpinFactor, CursorPos );
				return Result ? 0 : 1;
			}

			// Mouse Cursor
		case WM_SETCURSOR:
			{
				// WM_SETCURSOR - Sent to a window if the mouse causes the cursor to move within a window and mouse input is not captured.						

				// When we use the OSWindowBorder the only zone that "belongs" to Slate for cursor purposes is the Client Zone
				if (CurrentNativeEventWindowPtr->GetDefinition().HasOSWindowBorder)
				{
					const UINT MouseWindowMessage = HIWORD(lParam);
					const UINT CursorHitTestResult = LOWORD(lParam);

					if (MouseWindowMessage == WM_MOUSEMOVE && CursorHitTestResult != HTCLIENT)
					{
						return 0;
					}
				}
				return  MessageHandler->OnCursorSet() ? 1 : 0;
			}

		case WM_TOUCH:
			{
				UINT InputCount = LOWORD( wParam );
				if ( InputCount > 0 )
				{
					TUniquePtr<TOUCHINPUT[]> Inputs = MakeUnique<TOUCHINPUT[]>( InputCount );
					if ( GetTouchInputInfo( (HTOUCHINPUT)lParam, InputCount, Inputs.Get(), sizeof(TOUCHINPUT) ) )
					{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FInputDeviceScope InputScope(nullptr, UE::WindowsInput::InputClassName, IPlatformInputDeviceMapper::Get().GetDefaultInputDevice().GetId(), UE::WindowsInput::TouchInputHardwareName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

						for ( uint32 i = 0; i < InputCount; i++ )
						{
							TOUCHINPUT Input = Inputs[i];
							FVector2D Location((float)Input.x / 100.0f, (float)Input.y / 100.0f );
							if ( Input.dwFlags & TOUCHEVENTF_DOWN )
							{
								int32 TouchIndex = GetTouchIndexForID( Input.dwID );
								if (TouchIndex == INDEX_NONE)
								{
									TouchIndex = GetFirstFreeTouchIndex();
									check(TouchIndex >= 0);
									
									TouchInfoArray[TouchIndex].TouchID = TOptional<int32>(Input.dwID);
									TouchInfoArray[TouchIndex].HasMoved = false;
									TouchInfoArray[TouchIndex].PreviousLocation = Location;

									UE_LOGF(LogWindowsDesktop, Verbose, "OnTouchStarted at (%f, %f), finger %d (system touch id %d)", Location.X, Location.Y, TouchIndex, Input.dwID);
									MessageHandler->OnTouchStarted(CurrentNativeEventWindowPtr, Location, 1.0f, TouchIndex, 0);
								}
								else
								{
									// TODO: Error handling.
								}
							}
							else if ( Input.dwFlags & TOUCHEVENTF_MOVE )
							{
								int32 TouchIndex = GetTouchIndexForID( Input.dwID );
								if ( TouchIndex >= 0 )
								{
									UE_LOGF(LogWindowsDesktop, Verbose, "OnTouchMoved at (%f, %f), finger %d (system touch id %d)", Location.X, Location.Y, TouchIndex, Input.dwID);

									if (bEnableFirstTouchEvent)
									{
										// track first move event, for helping with "pop" on the filtered small movements
										if (!TouchInfoArray[TouchIndex].HasMoved)
										{
											if (TouchInfoArray[TouchIndex].PreviousLocation != Location)
											{
												TouchInfoArray[TouchIndex].HasMoved = true;
												MessageHandler->OnTouchFirstMove(Location, 1.0f, TouchIndex, 0);
											}
										}
									}

									TouchInfoArray[TouchIndex].PreviousLocation = Location;
									MessageHandler->OnTouchMoved(Location, 1.0f, TouchIndex, 0);
								}
							}
							else if ( Input.dwFlags & TOUCHEVENTF_UP )
							{
								int32 TouchIndex = GetTouchIndexForID( Input.dwID );
								if ( TouchIndex >= 0 )
								{
									TouchInfoArray[TouchIndex].TouchID = TOptional<int32>();
									UE_LOGF(LogWindowsDesktop, Verbose, "OnTouchEnded at (%f, %f), finger %d (system touch id %d)", Location.X, Location.Y, TouchIndex, Input.dwID);
									MessageHandler->OnTouchEnded(Location, TouchIndex, 0);
								}
								else
								{
									// TODO: Error handling.
								}
							}
						}
						CloseTouchInputHandle( (HTOUCHINPUT)lParam );
						return 0;
					}
				}
				break;
			}

		case WM_POINTERDOWN:
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FInputDeviceScope InputScope(nullptr, UE::WindowsInput::InputClassName, IPlatformInputDeviceMapper::Get().GetDefaultInputDevice().GetId(), UE::WindowsInput::TouchInputHardwareName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

				int32 PointerId = GET_POINTERID_WPARAM(wParam);

				// Touch and pen input share a common POINTER_INFO member allowing both to use the same code
				POINTER_INPUT_TYPE Type;
				POINTER_TOUCH_INFO TouchInfo;
				POINTER_PEN_INFO PenInfo;
				POINTER_INFO* PointerInfo = nullptr;

				if (GetPointerType(PointerId, &Type))
				{
					if (Type == PT_TOUCH)
					{
						if (GetPointerTouchInfo(PointerId, &TouchInfo))
						{
							PointerInfo = &TouchInfo.pointerInfo;
						}
					}
					else if (Type == PT_PEN)
					{
						if (GetPointerPenInfo(PointerId, &PenInfo))
						{
							PointerInfo = &PenInfo.pointerInfo;
						}
					}
				}

				if (PointerInfo == nullptr)
				{
					return 0;
				}

				FVector2D Location(PointerInfo->ptPixelLocation.x, PointerInfo->ptPixelLocation.y);

				int32 TouchIndex = GetTouchIndexForID(PointerId);
				if (TouchIndex == INDEX_NONE)
				{
					TouchIndex = GetFirstFreeTouchIndex();
					check(TouchIndex >= 0);

					TouchInfoArray[TouchIndex].TouchID = TOptional<int32>(PointerId);
					TouchInfoArray[TouchIndex].HasMoved = false;
					TouchInfoArray[TouchIndex].PreviousLocation = Location;

					UE_LOGF(LogWindowsDesktop, Verbose, "OnTouchStarted at (%f, %f), finger %d (system touch id %d)", Location.X, Location.Y, TouchIndex, PointerId);
					MessageHandler->OnTouchStarted(CurrentNativeEventWindowPtr, Location, 1.0f, TouchIndex, 0);
				}
			
				return 0;
			}


		case WM_POINTERUPDATE:
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FInputDeviceScope InputScope(nullptr, UE::WindowsInput::InputClassName, IPlatformInputDeviceMapper::Get().GetDefaultInputDevice().GetId(), UE::WindowsInput::TouchInputHardwareName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

				int32 PointerId = GET_POINTERID_WPARAM(wParam);
		
				// Touch and pen input share a common POINTER_INFO member allowing both to use the same code
				POINTER_INPUT_TYPE Type;
				POINTER_TOUCH_INFO TouchInfo;
				POINTER_PEN_INFO PenInfo;
				POINTER_INFO* PointerInfo = nullptr;

				if (GetPointerType(PointerId, &Type))
				{
					if (Type == PT_TOUCH)
					{
						if (GetPointerTouchInfo(PointerId, &TouchInfo))
						{
							PointerInfo = &TouchInfo.pointerInfo;
						}
					}
					else if (Type == PT_PEN)
					{
						if (GetPointerPenInfo(PointerId, &PenInfo))
						{
							PointerInfo = &PenInfo.pointerInfo;
						}
					}
				}

				if (PointerInfo == nullptr)
				{
					return 0;
				}

				FVector2D Location(PointerInfo->ptPixelLocation.x, PointerInfo->ptPixelLocation.y);

				int32 TouchIndex = GetTouchIndexForID(PointerId);
				if (TouchIndex >= 0)
				{
					UE_LOGF(LogWindowsDesktop, Verbose, "OnTouchMoved at (%f, %f), finger %d (system touch id %d)", Location.X, Location.Y, TouchIndex, PointerId);

					if (bEnableFirstTouchEvent)
					{
						// track first move event, for helping with "pop" on the filtered small movements
						if (!TouchInfoArray[TouchIndex].HasMoved)
						{
							if (TouchInfoArray[TouchIndex].PreviousLocation != Location)
							{
								TouchInfoArray[TouchIndex].HasMoved = true;
								MessageHandler->OnTouchFirstMove(Location, 1.0f, TouchIndex, 0);
							}
						}
					}

					TouchInfoArray[TouchIndex].PreviousLocation = Location;
					MessageHandler->OnTouchMoved(Location, 1.0f, TouchIndex, 0);
				}

				return 0;
			}

		case WM_POINTERUP:
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FInputDeviceScope InputScope(nullptr, UE::WindowsInput::InputClassName, IPlatformInputDeviceMapper::Get().GetDefaultInputDevice().GetId(), UE::WindowsInput::TouchInputHardwareName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

				int32 PointerId = GET_POINTERID_WPARAM(wParam);

				// Touch and pen input share a common POINTER_INFO member allowing both to use the same code
				POINTER_INPUT_TYPE Type;
				POINTER_TOUCH_INFO TouchInfo;
				POINTER_PEN_INFO PenInfo;
				POINTER_INFO* PointerInfo = nullptr;

				if (GetPointerType(PointerId, &Type))
				{
					if (Type == PT_TOUCH)
					{
						if (GetPointerTouchInfo(PointerId, &TouchInfo))
						{
							PointerInfo = &TouchInfo.pointerInfo;
						}
					}
					else if (Type == PT_PEN)
					{
						if (GetPointerPenInfo(PointerId, &PenInfo))
						{
							PointerInfo = &PenInfo.pointerInfo;
						}
					}
				}

				if (PointerInfo == nullptr)
				{
					return 0;
				}

				FVector2D Location(PointerInfo->ptPixelLocation.x, PointerInfo->ptPixelLocation.y);
			
				int32 TouchIndex = GetTouchIndexForID(PointerId);
				if (TouchIndex >= 0)
				{
					TouchInfoArray[TouchIndex].TouchID = TOptional<int32>();
					UE_LOGF(LogWindowsDesktop, Verbose, "OnTouchEnded at (%f, %f), finger %d (system touch id %d)", Location.X, Location.Y, TouchIndex, PointerId);
					MessageHandler->OnTouchEnded(Location, TouchIndex, 0);
				}

				return 0;
			}

			// Window focus and activation
		case WM_MOUSEACTIVATE:
			{
				// If the mouse activate isn't in the client area we'll force the WM_ACTIVATE to be EWindowActivation::ActivateByMouse
				// This ensures that clicking menu buttons on the header doesn't generate a WM_ACTIVATE with EWindowActivation::Activate
				// which may cause mouse capture to be taken because is not differentiable from Alt-Tabbing back to the application.
				bForceActivateByMouse = !(LOWORD(lParam) & HTCLIENT);
				return 0;
			}

			// Window focus and activation
		case WM_ACTIVATE:
			{
				EWindowActivation ActivationType;

				if (LOWORD(wParam) & WA_ACTIVE)
				{
					ActivationType = bForceActivateByMouse ? EWindowActivation::ActivateByMouse : EWindowActivation::Activate;
				}
				else if (LOWORD(wParam) & WA_CLICKACTIVE)
				{
					ActivationType = EWindowActivation::ActivateByMouse;
				}
				else
				{
					ActivationType = EWindowActivation::Deactivate;
				}
				bForceActivateByMouse = false;

				UpdateAllModifierKeyStates();

				if ( CurrentNativeEventWindowPtr.IsValid() )
				{
					BOOL Result = false;
					Result = MessageHandler->OnWindowActivationChanged( CurrentNativeEventWindowPtr.ToSharedRef(), ActivationType );
					return Result ? 0 : 1;
				}

				return 1;
			}

		case WM_ACTIVATEAPP:
			UpdateAllModifierKeyStates();
			MessageHandler->OnApplicationActivationChanged( !!wParam );
			UE::WindowsApplication::TouchpadSensitivity::HandleActivateApp(!!wParam);
			break;

		case WM_SETTINGCHANGE:
			if ((lParam != 0) && (FCString::Strcmp((LPCTSTR)lParam, TEXT("ConvertibleSlateMode")) == 0))
			{
				MessageHandler->OnConvertibleLaptopModeChanged();
			}
			if (wParam == UE::WindowsApplication::TouchpadSensitivity::SPI_SETTOUCHPADPARAMETERS)
			{
				UE::WindowsApplication::TouchpadSensitivity::OnWMSettingChangeSetTouchpadParameters();
			}

			break;
	
		case WM_NCACTIVATE:
			{
				if( CurrentNativeEventWindowPtr.IsValid() && !CurrentNativeEventWindowPtr->GetDefinition().HasOSWindowBorder )
				{
					// Unless using the OS window border, intercept calls to prevent non-client area drawing a border upon activation or deactivation
					// Return true to ensure standard activation happens
					return true;
				}
			}
			break;

		case WM_NCPAINT:
			{
				if( CurrentNativeEventWindowPtr.IsValid() && !CurrentNativeEventWindowPtr->GetDefinition().HasOSWindowBorder )
				{
					// Unless using the OS window border, intercept calls to draw the non-client area - we do this ourselves
					return 0;
				}
			}
			break;

		case WM_CLOSE:
			{
				if ( CurrentNativeEventWindowPtr.IsValid() )
				{
					// Called when the OS close button is pressed
					MessageHandler->OnWindowClose( CurrentNativeEventWindowPtr.ToSharedRef() );
				}
				return 0;
			}

		case WM_SHOWWINDOW:
			{
				if( CurrentNativeEventWindowPtr.IsValid() )
				{
					switch(lParam)
					{
					case SW_PARENTCLOSING:
						CurrentNativeEventWindowPtr->OnParentWindowMinimized();
						bMinimized = true;
						break;
					case SW_PARENTOPENING:
						CurrentNativeEventWindowPtr->OnParentWindowRestored();
						bMinimized = false;
						break;
					default:
						break;
					}
				}
			}
			break;

		case WM_SIZE:
			{
				if( CurrentNativeEventWindowPtr.IsValid() )
				{
					// @todo Fullscreen - Perform deferred resize
					// Note WM_SIZE provides the client dimension which is not equal to the window dimension if there is a windows border 
					const int32 NewWidth = (int)(short)(LOWORD(lParam));
					const int32 NewHeight = (int)(short)(HIWORD(lParam));

					const FGenericWindowDefinition& Definition = CurrentNativeEventWindowPtr->GetDefinition();
					if ( Definition.IsRegularWindow && !Definition.HasOSWindowBorder )
					{
						CurrentNativeEventWindowPtr->AdjustWindowRegion(NewWidth, NewHeight);
					}

					const bool bWasMinimized = (wParam == SIZE_MINIMIZED);

					const bool bIsFullscreen = (CurrentNativeEventWindowPtr->GetWindowMode() == EWindowMode::Type::Fullscreen);

					// When in fullscreen Windows rendering size should be determined by the application. Do not adjust based on WM_SIZE messages.
 					if ( !bIsFullscreen )
					{
						const bool Result = MessageHandler->OnSizeChanged(CurrentNativeEventWindowPtr.ToSharedRef(), NewWidth, NewHeight, bWasMinimized);
					}
				}
			}
			break;
		case WM_SIZING:
			{
				if( CurrentNativeEventWindowPtr.IsValid() )
				{
					MessageHandler->OnResizingWindow( CurrentNativeEventWindowPtr.ToSharedRef() );
				}
			}
			break;
		case WM_ENTERSIZEMOVE:
			{
				if( CurrentNativeEventWindowPtr.IsValid() )
				{
					MessageHandler->BeginReshapingWindow( CurrentNativeEventWindowPtr.ToSharedRef() );
				}
			}
			break;
		case WM_EXITSIZEMOVE:
			{
				if( CurrentNativeEventWindowPtr.IsValid() )
				{
					MessageHandler->FinishedReshapingWindow( CurrentNativeEventWindowPtr.ToSharedRef() );
				}
			}
			break;

		case WM_DWMCOMPOSITIONCHANGED:
			{
				CurrentNativeEventWindowPtr->OnTransparencySupportChanged(GetWindowTransparencySupport());
			}
			break;

		case WM_DPICHANGED:
			{
				if( CurrentNativeEventWindowPtr.IsValid())
				{
					MessageHandler->SignalSystemDPIChanged(CurrentNativeEventWindowPtr.ToSharedRef());

					if (!CurrentNativeEventWindowPtr->IsManualManageDPIChanges())
					{
						CurrentNativeEventWindowPtr->SetDPIScaleFactor(LOWORD(wParam) / static_cast<float>(USER_DEFAULT_SCREEN_DPI));

						if (CurrentNativeEventWindowPtr->IsInModalSizeLoop())
						{
							LPRECT NewRect = (LPRECT)lParam;
							SetWindowPos(hwnd, nullptr, NewRect->left, NewRect->top, NewRect->right - NewRect->left, NewRect->bottom - NewRect->top, SWP_NOZORDER | SWP_NOACTIVATE);
						}

						MessageHandler->HandleDPIScaleChanged(CurrentNativeEventWindowPtr.ToSharedRef());
					}
				}
			}
			break;
		}
	}

	return 0;
}

void FWindowsApplication::ProcessDeferredDragDropOperation(const FDeferredWindowsDragDropOperation& Op)
{
	// Since we deferred the drag/drop event, we could not specify the correct cursor effect in time. Now we will just throw away the value.
	DWORD DummyCursorEffect = 0;

	switch (Op.OperationType)
	{
		case EWindowsDragDropOperationType::DragEnter:
			OnOLEDragEnter(Op.HWnd, Op.OLEData, Op.KeyState, Op.CursorPosition, &DummyCursorEffect);
			break;
		case EWindowsDragDropOperationType::DragOver:
			OnOLEDragOver(Op.HWnd, Op.KeyState, Op.CursorPosition, &DummyCursorEffect);
			break;
		case EWindowsDragDropOperationType::DragLeave:
			OnOLEDragOut(Op.HWnd);
			break;
		case EWindowsDragDropOperationType::Drop:
			OnOLEDrop(Op.HWnd, Op.OLEData, Op.KeyState, Op.CursorPosition, &DummyCursorEffect);
			break;
		default:
			ensureMsgf(0, TEXT("Unhandled deferred drag/drop operation type: %d"), Op.OperationType);
			break;
	}
}

bool FWindowsApplication::IsKeyboardInputMessage( uint32 msg )
{
	switch(msg)
	{
	// Keyboard input notification messages...
	case WM_CHAR:
	case WM_SYSCHAR:
	case WM_SYSKEYDOWN:
	case WM_KEYDOWN:
	case WM_SYSKEYUP:
	case WM_KEYUP:
	case WM_SYSCOMMAND:
		return true;
	}
	return false;
}

bool FWindowsApplication::IsMouseInputMessage( uint32 msg )
{
	switch(msg)
	{
	// Mouse input notification messages...
	case WM_MOUSEHWHEEL:
	case WM_MOUSEWHEEL:
	case WM_MOUSEHOVER:
	case WM_MOUSELEAVE:
	case WM_MOUSEMOVE:
	case WM_NCMOUSEHOVER:
	case WM_NCMOUSELEAVE:
	case WM_NCMOUSEMOVE:
	case WM_NCMBUTTONDBLCLK:
	case WM_NCMBUTTONDOWN:
	case WM_NCMBUTTONUP:
	case WM_NCRBUTTONDBLCLK:
	case WM_NCRBUTTONDOWN:
	case WM_NCRBUTTONUP:
	case WM_NCXBUTTONDBLCLK:
	case WM_NCXBUTTONDOWN:
	case WM_NCXBUTTONUP:
	case WM_LBUTTONDBLCLK:
	case WM_LBUTTONDOWN:
	case WM_LBUTTONUP:
	case WM_MBUTTONDBLCLK:
	case WM_MBUTTONDOWN:
	case WM_MBUTTONUP:
	case WM_RBUTTONDBLCLK:
	case WM_RBUTTONDOWN:
	case WM_RBUTTONUP:
	case WM_XBUTTONDBLCLK:
	case WM_XBUTTONDOWN:
	case WM_XBUTTONUP:
		return true;
	}
	return false;
}

bool FWindowsApplication::IsInputMessage( uint32 msg )
{
	if (IsKeyboardInputMessage(msg) || IsMouseInputMessage(msg))
	{
		return true;
	}

	switch(msg)
	{
	// Raw input notification messages...
	case WM_INPUT:
	case WM_INPUT_DEVICE_CHANGE:
		return true;
	}
	return false;
}

#define MOUSEEVENTF_FROMTOUCH 0xFF515700

bool FWindowsApplication::IsFakeMouseInputMessage(uint32 msg)
{
	const bool bShouldPrevent = !!bPreventDuplicateMouseEventsForTouch;

	if (bShouldPrevent && IsMouseInputMessage(msg))
	{
		// This is only legal to call when handling messages in the pump, and is not valid
		// to call in a deferred fashion.
		// Click was generated by wisptis / Windows Touch
		return (GetMessageExtraInfo() & MOUSEEVENTF_FROMTOUCH) == MOUSEEVENTF_FROMTOUCH;
	}
	
	// Click was generated by the mouse.
	return false;
}

LRESULT CALLBACK FWindowsApplication::HandleLowLevelMouseFilterHook(int nCode, WPARAM wParam, LPARAM lParam)
{
	if (nCode != HC_ACTION)
	{
		//Action was not a mouse input.
		return CallNextHookEx(NULL, nCode, wParam, lParam);
	}

	LPARAM extra = GetMessageExtraInfo();
	if ((extra & MOUSEEVENTF_FROMTOUCH) == MOUSEEVENTF_FROMTOUCH)
	{
		// Mouse Input was generated by wisptis / Windows Touch
		CallNextHookEx(NULL, nCode, wParam, lParam);
		return 1;
	}
	else
	{
		PMSLLHOOKSTRUCT hookStruct = reinterpret_cast<PMSLLHOOKSTRUCT>(lParam);
		extra = hookStruct->dwExtraInfo;
		if (((extra & MOUSEEVENTF_FROMTOUCH) == MOUSEEVENTF_FROMTOUCH))
		{
			// Mouse Input was generated by wisptis / Windows Touch
			CallNextHookEx(NULL, nCode, wParam, lParam);
			return 1;
		}
	}

	// Mouse Input was generated by the mouse.
	return CallNextHookEx(NULL, nCode, wParam, lParam);
}

#undef MOUSEEVENTF_FROMTOUCH

void FWindowsApplication::OnInputDeviceModuleRegistered(const FName& Type, IModularFeature* ModularFeature)
{
	if (Type == IInputDeviceModule::GetModularFeatureName())
	{
		IInputDeviceModule* Module = static_cast<IInputDeviceModule*>(ModularFeature);

		TSharedPtr<IInputDevice> Device = Module->CreateInputDeviceIfPreferred(MessageHandler);
		AddExternalInputDevice(MoveTemp(Device));
	}
}

void FWindowsApplication::DeferMessage( TSharedPtr<FWindowsWindow>& NativeWindow, HWND InHWnd, uint32 InMessage, WPARAM InWParam, LPARAM InLParam, int32 MouseX, int32 MouseY, uint32 RawInputFlags )
{
	if( GPumpingMessagesOutsideOfMainLoop && bAllowedToDeferMessageProcessing )
	{
		DeferredMessages.Add( FDeferredWindowsMessage( NativeWindow, InHWnd, InMessage, InWParam, InLParam, MouseX, MouseY, RawInputFlags ) );
	}
	else
	{
		// When not deferring messages, process them immediately
		ProcessDeferredMessage( FDeferredWindowsMessage( NativeWindow, InHWnd, InMessage, InWParam, InLParam, MouseX, MouseY, RawInputFlags ) );
	}
}

void FWindowsApplication::PumpMessages( const float TimeDelta )
{
	MSG Message;

	// standard Windows message handling
	while(PeekMessage(&Message, NULL, 0, 0, PM_REMOVE))
	{ 
		TranslateMessage(&Message);
		DispatchMessage(&Message); 
	}

	PostPumpMessages();
}

void FWindowsApplication::ProcessDeferredEvents( const float TimeDelta )
{
	// Process windows messages
	{
		// This function can be reentered when entering a modal tick loop.
		// We need to make a copy of the events that need to be processed or we may end up processing the same messages twice 
		TArray<FDeferredWindowsMessage> EventsToProcess( DeferredMessages );

		DeferredMessages.Empty();
		for( int32 MessageIndex = 0; MessageIndex < EventsToProcess.Num(); ++MessageIndex )
		{
			const FDeferredWindowsMessage& DeferredMessage = EventsToProcess[MessageIndex];
			ProcessDeferredMessage( DeferredMessage );
		}

		CheckForShiftUpEvents(VK_LSHIFT);
		CheckForShiftUpEvents(VK_RSHIFT);
	}

	// Process drag/drop operations
	{
		TArray<FDeferredWindowsDragDropOperation> DragDropOperationsToProcess(DeferredDragDropOperations);

		DeferredDragDropOperations.Empty();
		for (int32 OperationIndex = 0; OperationIndex < DragDropOperationsToProcess.Num(); ++OperationIndex)
		{
			const FDeferredWindowsDragDropOperation& DeferredDragDropOperation = DragDropOperationsToProcess[OperationIndex];
			ProcessDeferredDragDropOperation(DeferredDragDropOperation);
		}
	}
}

void FWindowsApplication::PollGameDeviceState( const float TimeDelta )
{
	if (bForceNoGamepads)
	{
		return;
	}

	if (!bHasLoadedInputPlugins)
	{
		// Create all externally-implemented input devices from loaded modules.
		TArray<IInputDeviceModule*> PluginImplementations = IModularFeatures::Get().GetModularFeatureImplementations<IInputDeviceModule>( IInputDeviceModule::GetModularFeatureName() );
		for( auto InputPluginIt = PluginImplementations.CreateIterator(); InputPluginIt; ++InputPluginIt )
		{
			TSharedPtr<IInputDevice> Device = (*InputPluginIt)->CreateInputDeviceIfPreferred(MessageHandler);
			AddExternalInputDevice(MoveTemp(Device));			
		}

		// Modules loaded after this point are handled by this callback
		IModularFeatures::Get().OnModularFeatureRegistered().AddRaw(this, &FWindowsApplication::OnInputDeviceModuleRegistered);

		bHasLoadedInputPlugins = true;
	}

	if (FApp::UseVRFocus() && !FApp::HasVRFocus())
	{
		return; // do not proceed if the app uses VR focus but doesn't have it
	}

	// Poll externally-implemented devices
	for( auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt )
	{
		(*DeviceIt)->Tick( TimeDelta );
		(*DeviceIt)->SendControllerEvents();
	}
}

void FWindowsApplication::SetForceFeedbackChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value)
{
	if (FApp::UseVRFocus() && !FApp::HasVRFocus())
	{
		return; // do not proceed if the app uses VR focus but doesn't have it
	}

	// send vibration to externally-implemented devices
	for( auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt )
	{
		(*DeviceIt)->SetChannelValue(ControllerId, ChannelType, Value);
	}
}

void FWindowsApplication::SetForceFeedbackChannelValues(int32 ControllerId, const FForceFeedbackValues &Values)
{
	if (FApp::UseVRFocus() && !FApp::HasVRFocus())
	{
		return; // do not proceed if the app uses VR focus but doesn't have it
	}

	const FForceFeedbackValues* InternalValues = &Values;
 
	// send vibration to externally-implemented devices
	for( auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt )
	{
		if ((*DeviceIt)->IsGamepadAttached()) 
		{
			(*DeviceIt)->SetChannelValues(ControllerId, Values);
		}
	}
}

void FWindowsApplication::SetHapticFeedbackValues(int32 ControllerId, int32 Hand, const FHapticFeedbackValues& Values)
{
	if (FApp::UseVRFocus() && !FApp::HasVRFocus())
	{
		return; // do not proceed if the app uses VR focus but doesn't have it
	}

	for (auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt)
	{
		IHapticDevice* HapticDevice = (*DeviceIt)->GetHapticDevice();
		if (HapticDevice)
		{
			HapticDevice->SetHapticFeedbackValues(ControllerId, Hand, Values);
		}
	}
}

void FWindowsApplication::SetLightColor(int32 ControllerId, FColor Color)
{
	const FInputDeviceLightColorProperty Property(true, Color);

	SetDeviceProperty(ControllerId, &Property);
}

void FWindowsApplication::ResetLightColor(int32 ControllerId)
{
	const FInputDeviceLightColorProperty Property(false, FColor::Black);

	SetDeviceProperty(ControllerId, &Property);
}

void FWindowsApplication::SetDeviceProperty(int32 ControllerId, const FInputDeviceProperty* Property)
{
	if (FApp::UseVRFocus() && !FApp::HasVRFocus())
	{
		return; // do not proceed if the app uses VR focus but doesn't have it
	}

	// set on externally-implemented devices
	for (auto DeviceIt = ExternalInputDevices.CreateIterator(); DeviceIt; ++DeviceIt)
	{
		(*DeviceIt)->SetDeviceProperty(ControllerId, Property);
	}
}

void FWindowsApplication::AddExternalInputDevice(TSharedPtr<IInputDevice> InputDevice)
{
	if (InputDevice.IsValid())
	{
		ExternalInputDevices.Add(MoveTemp(InputDevice));
	}
}

void FWindowsApplication::FinishedInputThisFrame()
{

}

TSharedPtr<FTaskbarList> FWindowsApplication::GetTaskbarList()
{
	return TaskbarList;
}

void FWindowsApplication::DeferDragDropOperation(const FDeferredWindowsDragDropOperation& DeferredDragDropOperation)
{
	DeferredDragDropOperations.Add(DeferredDragDropOperation);
}

HRESULT FWindowsApplication::OnOLEDragEnter( const HWND HWnd, const FDragDropOLEData& OLEData, DWORD KeyState, POINTL CursorPosition, DWORD *CursorEffect)
{
	const TSharedPtr< FWindowsWindow > Window = FindWindowByHWND( Windows, HWnd );

	if ( !Window.IsValid() )
	{
		return 0;
	}

	CachedDragAndDropKeyState = 0;

	if ( Window->IsEnabled() )
	{
		UpdateAllModifierKeyStates();
		CachedDragAndDropKeyState = KeyState;
		if ((OLEData.Type & FDragDropOLEData::Text) && (OLEData.Type & FDragDropOLEData::Files))
		{
			*CursorEffect = MessageHandler->OnDragEnterExternal(Window.ToSharedRef(), OLEData.OperationText, OLEData.OperationFilenames);
		}
		else if (OLEData.Type & FDragDropOLEData::Text)
		{
			*CursorEffect = MessageHandler->OnDragEnterText(Window.ToSharedRef(), OLEData.OperationText);
		}
		else if (OLEData.Type & FDragDropOLEData::Files)
		{
			*CursorEffect = MessageHandler->OnDragEnterFiles(Window.ToSharedRef(), OLEData.OperationFilenames);
		}
	}
	else
	{
		*CursorEffect = EDropEffect::None;
	}

	return 0;
}

HRESULT FWindowsApplication::OnOLEDragOver( const HWND HWnd, DWORD KeyState, POINTL CursorPosition, DWORD *CursorEffect)
{
	const TSharedPtr< FWindowsWindow > Window = FindWindowByHWND( Windows, HWnd );

	if ( Window.IsValid() )
	{
		if ( Window->IsEnabled() )
		{
			if (KeyState != CachedDragAndDropKeyState)
			{
				UpdateAllModifierKeyStates();
				CachedDragAndDropKeyState = KeyState;
			}
			*CursorEffect = MessageHandler->OnDragOver( Window.ToSharedRef() );
		}
		else
		{
			*CursorEffect = EDropEffect::None;
		}
	}

	return 0;
}

HRESULT FWindowsApplication::OnOLEDragOut( const HWND HWnd )
{
	const TSharedPtr< FWindowsWindow > Window = FindWindowByHWND( Windows, HWnd );

	if ( Window.IsValid() && Window->IsEnabled() )
	{
		// User dragged out of a Slate window. We must tell Slate it is no longer in drag and drop mode.
		// Note that this also gets triggered when the user hits ESC to cancel a drag and drop.
		MessageHandler->OnDragLeave( Window.ToSharedRef() );
	}

	return 0;
}

HRESULT FWindowsApplication::OnOLEDrop( const HWND HWnd, const FDragDropOLEData& OLEData, DWORD KeyState, POINTL CursorPosition, DWORD *CursorEffect)
{
	const TSharedPtr< FWindowsWindow > Window = FindWindowByHWND( Windows, HWnd );

	if ( Window.IsValid() )
	{
		if ( Window->IsEnabled() )
		{
			if (KeyState != CachedDragAndDropKeyState)
			{
				UpdateAllModifierKeyStates();
				CachedDragAndDropKeyState = KeyState;
			}

			*CursorEffect = MessageHandler->OnDragDrop(Window.ToSharedRef());
		}
		else
		{
			*CursorEffect = EDropEffect::None;
		}
	}

	return 0;
}


void FWindowsApplication::AddMessageHandler(IWindowsMessageHandler& InMessageHandler)
{
	WindowsApplication->MessageHandlers.AddUnique(&InMessageHandler);
}


void FWindowsApplication::RemoveMessageHandler(IWindowsMessageHandler& InMessageHandler)
{
	WindowsApplication->MessageHandlers.RemoveSwap(&InMessageHandler);
}


void FWindowsApplication::QueryConnectedMice()
{
	TArray<RAWINPUTDEVICELIST> DeviceList;
	UINT DeviceCount = 0;

	GetRawInputDeviceList(nullptr, &DeviceCount, sizeof(RAWINPUTDEVICELIST));
	if (DeviceCount == 0)
	{
		bIsMouseAttached = false;
		return;
	}

	DeviceList.AddUninitialized(DeviceCount);
	GetRawInputDeviceList(DeviceList.GetData(), &DeviceCount, sizeof(RAWINPUTDEVICELIST));
	
	int32 MouseCount = 0;
	for (const auto& Device : DeviceList)
	{
		UINT NameLen = 0;
		TUniquePtr<char[]> Name;
		if (Device.dwType != RIM_TYPEMOUSE)
			continue;

		//Force the use of ANSI versions of these calls
		if (GetRawInputDeviceInfoA(Device.hDevice, RIDI_DEVICENAME, nullptr, &NameLen) == static_cast<UINT>(-1))
			continue;

		Name = MakeUnique<char[]>(NameLen+1);
		if (GetRawInputDeviceInfoA(Device.hDevice, RIDI_DEVICENAME, Name.Get(), &NameLen) == static_cast<UINT>(-1))
			continue;

		Name[NameLen] = 0;
		FString WName = ANSI_TO_TCHAR(Name.Get());
		WName.ReplaceInline(TEXT("#"), TEXT("\\"), ESearchCase::CaseSensitive);

		/*
		 * Name XP starts with \??\, vista+ starts \\?\
		 * In the device list exists a fake Mouse device with the device name of RDP_MOU
		 * This is used for Remote Desktop so ignore it.
		 */
		if (WName.StartsWith(TEXT("\\??\\ROOT\\RDP_MOU\\")) || WName.StartsWith(TEXT("\\\\?\\ROOT\\RDP_MOU\\")))
		{
			continue;
		}

		++MouseCount;
	}

	// If the session is a remote desktop session - assume that a mouse is present, it seems that you can end up
	// in a situation where RDP mice don't have a valid name, so the code above fails to locate a valid mouse, 
	// even though one is present.
	if ( MouseCount == 0 )
	{
		if ( GetSystemMetrics(SM_REMOTESESSION) )
		{
			MouseCount++;
		}
	}

	bIsMouseAttached = MouseCount > 0;
}

uint32 FWindowsApplication::GetTouchIndexForID( int32 TouchID )
{
	for (int i = 0; i < TouchInfoArray.Num(); i++)
	{
		if (TouchInfoArray[i].TouchID.IsSet() && TouchInfoArray[i].TouchID.GetValue() == TouchID )
		{
			return i;
		}
	}
	return INDEX_NONE;
}

uint32 FWindowsApplication::GetFirstFreeTouchIndex()
{
	for ( int i = 0; i < TouchInfoArray.Num(); i++ )
	{
		if (!TouchInfoArray[i].TouchID.IsSet())
		{
			return i;
		}
	}

	return TouchInfoArray.Add(TouchInfo());
}

void FTaskbarList::Initialize()
{
	if (LIKELY(FApp::CanEverRender()) && FWindowsPlatformMisc::CoInitialize())
	{
		if (CoCreateInstance(CLSID_TaskbarList, NULL, CLSCTX_INPROC_SERVER, IID_ITaskbarList3, (void **)&TaskBarList3) != S_OK)
		{
			TaskBarList3 = nullptr;
		}
	}
}

FTaskbarList::FTaskbarList() 
	: TaskBarList3(NULL)
{

}

FTaskbarList::~FTaskbarList()
{
	if (TaskBarList3 && FWindowsPlatformMisc::CoInitialize())
	{
		TaskBarList3->Release();
	}

	TaskBarList3 = NULL;
}

void FTaskbarList::SetOverlayIcon(HWND WindowHandle, HICON Icon, FText Description)
{
	if (TaskBarList3)
	{
		TaskBarList3->SetOverlayIcon(WindowHandle, Icon, *Description.ToString());
	}
}

void FTaskbarList::SetProgressValue(HWND WindowHandle, uint64 Current, uint64 Total)
{
	if (TaskBarList3)
	{
		TaskBarList3->SetProgressValue(WindowHandle, (ULONGLONG)Current, (ULONGLONG)Total);
	}
}

void FTaskbarList::SetProgressState(HWND WindowHandle, ETaskbarProgressState::Type State)
{
	if (TaskBarList3)
	{
		TaskBarList3->SetProgressState(WindowHandle, (TBPFLAG)State);
	}
}

TSharedRef<FTaskbarList> FTaskbarList::Create()
{
	TSharedRef<FTaskbarList> TaskbarList = MakeShareable(new FTaskbarList());
	TaskbarList->Initialize();

	return TaskbarList;
}

void FWindowsApplication::StaticPostPumpMessages()
{
	if (WindowsApplication)
	{
		WindowsApplication->PostPumpMessages();
	}
}

void FWindowsApplication::PostPumpMessages()
{
	WindowsApplication->UpdateHighPrecisionMouseMode();
	WindowsApplication->ProcessWorkerInputs();
}

void FWindowsApplication::ProcessRawMouseInput(const RAWMOUSE& Mouse, RawMouseInputResult& OutResult)
{
	OutResult.bIsAbsoluteInput = (Mouse.usFlags & MOUSE_MOVE_ABSOLUTE) == MOUSE_MOVE_ABSOLUTE;
	if (OutResult.bIsAbsoluteInput)
	{
		if (bSimulatingHighPrecisionMouseInputForRDP)
		{
			const bool IsVirtualScreen = (Mouse.usFlags & MOUSE_VIRTUAL_DESKTOP) == MOUSE_VIRTUAL_DESKTOP;

			// Get the new cursor position
			POINT CursorPoint;
			const int32 Left = IsVirtualScreen ? GetSystemMetrics(SM_XVIRTUALSCREEN) : 0;
			const int32 Top = IsVirtualScreen ? GetSystemMetrics(SM_YVIRTUALSCREEN) : 0;
			const int32 Width = GetSystemMetrics(IsVirtualScreen ? SM_CXVIRTUALSCREEN : SM_CXSCREEN);
			const int32 Height = GetSystemMetrics(IsVirtualScreen ? SM_CYVIRTUALSCREEN : SM_CYSCREEN);

			CursorPoint.x = (LONG)(((LONGLONG)Mouse.lLastX * (LONGLONG)Width) / 65535LL) + Left;
			CursorPoint.y = (LONG)(((LONGLONG)Mouse.lLastY * (LONGLONG)Height) / 65535LL) + Top;

			const int32 ClipWidth = ClipCursorRect.right - ClipCursorRect.left;
			const int32 ClipHeight = ClipCursorRect.bottom - ClipCursorRect.top;

			const int32 DeltaWidthMax = (int32)((float)ClipWidth * 0.4f);
			const int32 DeltaHeightMax = (int32)((float)ClipHeight * 0.4f);

			const POINT CursorPointNoWrap = CursorPoint;

			const bool bCanAcceptPreWrapMsg = NumPreWrapMsgsToRespect > 0;
			NumPreWrapMsgsToRespect--;

			// Calculate the cursor delta from the last position
			// We do this prior to and wrapping for continuous input as that would mean an incorrect delta we'd just ignore
			int32 DeltaX = CursorPoint.x - LastCursorPoint.X;
			int32 DeltaY = CursorPoint.y - LastCursorPoint.Y;

			// Always skip wrapping if no movement has occured.
			bool bAcceptingPreWrapDelta = DeltaX == 0 && DeltaY == 0;

			if (bCanAcceptPreWrapMsg && !bAcceptingPreWrapDelta)
			{
				const int32 DeltaXPreWrap = CursorPoint.x - LastCursorPointPreWrap.X;
				const int32 DeltaYPreWrap = CursorPoint.y - LastCursorPointPreWrap.Y;

				const int32 DeltaLen = (DeltaX * DeltaX) + (DeltaY * DeltaY);
				const int32 DeltaPreWrapLen = (DeltaXPreWrap * DeltaXPreWrap) + (DeltaYPreWrap * DeltaYPreWrap);

				if (DeltaPreWrapLen < DeltaLen)
				{
					bAcceptingPreWrapDelta = true;

					DeltaX = DeltaXPreWrap;
					DeltaY = DeltaYPreWrap;
					LastCursorPointPreWrap.X = CursorPoint.x;
					LastCursorPointPreWrap.Y = CursorPoint.y;

					//UE_LOGF(LogWindowsDesktop, Log, "Accept PreWrap Delta X: %d Y: %d  ---- Last X: %d Y: %d ---- Cur X: %d Y: %d ---- Width: %d Height: %d", DeltaX, DeltaY, LastCursorPointPreWrap.X, LastCursorPointPreWrap.Y, CursorPoint.x, CursorPoint.y, (int32)Width, (int32)Height);
				}
			}

			if (!bAcceptingPreWrapDelta)
			{
				// Wrap and set cursor position in necessary
				const int32 WrapLeeway = 50; // We add some leeway on the wrap so that if the user is doing small movements hear the border we don't wrap back and fourth constantly
				const int32 TopEdge = ClipCursorRect.top + int32(0.1f * float(ClipHeight));
				const int32 BottomEdge = ClipCursorRect.top + int32(0.9f * float(ClipHeight));
				const int32 LeftEdge = ClipCursorRect.left + int32(0.1f * float(ClipWidth));
				const int32 RightEdge = ClipCursorRect.left + int32(0.9f * float(ClipWidth));

				bool bSet = false;
				if (CursorPoint.y < TopEdge) { CursorPoint.y = BottomEdge - WrapLeeway;	bSet = true; }
				else if (CursorPoint.y > BottomEdge) { CursorPoint.y = TopEdge + WrapLeeway; bSet = true; }

				if (CursorPoint.x < LeftEdge) { CursorPoint.x = RightEdge - WrapLeeway;	bSet = true; }
				else if (CursorPoint.x > RightEdge) { CursorPoint.x = LeftEdge + WrapLeeway; bSet = true; }

				if (bSet)
				{
					//UE_LOGF(LogWindowsDesktop, Log, "Wrapping Cursor to X: %d Y: %d ---- TopEdge: %d BottomEdge: %d LeftEdge: %d RightEdge: %d ", CursorPoint.x, CursorPoint.y, TopEdge, BottomEdge, LeftEdge, RightEdge);

					MessageHandler->SetCursorPos(FVector2D(CursorPoint.x, CursorPoint.y));
					LastCursorPoint.X = CursorPoint.x;
					LastCursorPoint.Y = CursorPoint.y;

					NumPreWrapMsgsToRespect = 10;
					LastCursorPointPreWrap.X = CursorPointNoWrap.x;
					LastCursorPointPreWrap.Y = CursorPointNoWrap.y;
				}

				/*
				if (DeltaX != 0 || DeltaY != 0)
				{
					if (FMath::Abs(DeltaX) < DeltaWidthMax && FMath::Abs(DeltaY) < DeltaHeightMax)
					{
						UE_LOGF(LogWindowsDesktop, Log, "Accept Delta X: %d Y: %d  ---- Last X: %d Y: %d ---- Cur X: %d Y: %d ---- Width: %d Height: %d", DeltaX, DeltaY, LastCursorPoint.X, LastCursorPoint.Y, CursorPoint.x, CursorPoint.y, (int32)Width, (int32)Height);
					}
					else
					{
						UE_LOGF(LogWindowsDesktop, Log, "IGNORE Delta X: %d Y: %d  ---- Last X: %d Y: %d ---- Cur X: %d Y: %d ---- Width: %d Height: %d", DeltaX, DeltaY, LastCursorPoint.X, LastCursorPoint.Y, CursorPoint.x, CursorPoint.y, (int32)Width, (int32)Height);
					}
				}*/

				LastCursorPoint.X = CursorPoint.x;
				LastCursorPoint.Y = CursorPoint.y;
			}

			// Send a delta assuming it's not zero or beyond our max delta
			OutResult.bSimulateRelativeMovement = (DeltaX != 0 || DeltaY != 0) && FMath::Abs(DeltaX) < DeltaWidthMax && FMath::Abs(DeltaY) < DeltaHeightMax;
			OutResult.DeltaX = DeltaX;
			OutResult.DeltaY = DeltaY;
		}
	}
	else
	{
		// Since raw input is coming in as relative it is likely a traditional mouse device
		OutResult.DeltaX = Mouse.lLastX;
		OutResult.DeltaY = Mouse.lLastY;
	}
}

void FWindowsApplication::ProcessWorkerInputs()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProcessWorkerInputs);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FInputDeviceScope InputScope(nullptr, UE::WindowsInput::InputClassName, IPlatformInputDeviceMapper::Get().GetDefaultInputDevice().GetId(), UE::WindowsInput::KBMInputHardwareName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	int32 xPosRelative = 0;
	int32 yPosRelative = 0;

	const auto FlushRelativeMovement =
		[this, &xPosRelative, &yPosRelative]()
		{
			if (xPosRelative || yPosRelative)
			{
				MessageHandler->OnRawMouseMove(xPosRelative, yPosRelative);
				xPosRelative = 0;
				yPosRelative = 0;
			}
		};

	EMouseButtons::Type PrimaryMouseButton = EMouseButtons::Left;
	EMouseButtons::Type SecondaryMouseButton = EMouseButtons::Right;
	if (GetSystemMetrics(SM_SWAPBUTTON))
	{
		Swap(PrimaryMouseButton, SecondaryMouseButton);
	}

	for (uint32 InputIndex = 0, InputCount = UE::WindowsApplication::RawInputThread::FWindowsRawInputRunnable::TakeMouseInputCount(); InputIndex < InputCount; ++InputIndex)
	{
		RAWMOUSE Mouse;
		UE::WindowsApplication::RawInputThread::FWindowsRawInputRunnable::PopMouseInput(Mouse);
		RawMouseInputResult Result;
		ProcessRawMouseInput(Mouse, Result);

		if (Result.bIsAbsoluteInput)
		{
			if (bSimulatingHighPrecisionMouseInputForRDP)
			{
				if (Result.bSimulateRelativeMovement)
				{
					FlushRelativeMovement();

					MessageHandler->OnRawMouseMove(Result.DeltaX, Result.DeltaY);
				}
			}
			else
			{
				FlushRelativeMovement();

				// Since the raw input is coming in as absolute it is likely the user is using a tablet
				// or perhaps is interacting through a virtual desktop
				MessageHandler->OnMouseMove();
			}
		}
		else
		{
			xPosRelative += Result.DeltaX;
			yPosRelative += Result.DeltaY;
		}

		const USHORT ButtonFlags = Mouse.usButtonFlags;
		if (ButtonFlags)
		{
			FlushRelativeMovement();

			TSharedPtr< FWindowsWindow > Window;

			POINT ButtonCursorPoint;
			if (::GetCursorPos(&ButtonCursorPoint))
			{
				const HWND hWnd = ::WindowFromPoint(ButtonCursorPoint);
				if (hWnd)
				{
					Window = FindWindowByHWND(Windows, hWnd);
				}
			}
			else
			{
				UE_LOGF(LogWindowsDesktop, Error, "Could not GetCursorPos for raw input. Error code: %u", GetLastError());
				ButtonCursorPoint.x = 0;
				ButtonCursorPoint.y = 0;
			}

			const FVector2D ButtonCursorPos = FVector2D(ButtonCursorPoint.x, ButtonCursorPoint.y);
			const double CurrentTime = FPlatformTime::Seconds();

			ProcessMouseButton(CurrentTime, Window, ButtonFlags, RI_MOUSE_LEFT_BUTTON_DOWN, RI_MOUSE_LEFT_BUTTON_UP, PrimaryMouseButton, ButtonCursorPos);
			ProcessMouseButton(CurrentTime, Window, ButtonFlags, RI_MOUSE_RIGHT_BUTTON_DOWN, RI_MOUSE_RIGHT_BUTTON_UP, SecondaryMouseButton, ButtonCursorPos);
			ProcessMouseButton(CurrentTime, Window, ButtonFlags, RI_MOUSE_MIDDLE_BUTTON_DOWN, RI_MOUSE_MIDDLE_BUTTON_UP, EMouseButtons::Middle, ButtonCursorPos);
			ProcessMouseButton(CurrentTime, Window, ButtonFlags, RI_MOUSE_BUTTON_4_DOWN, RI_MOUSE_BUTTON_4_UP, EMouseButtons::Thumb01, ButtonCursorPos);
			ProcessMouseButton(CurrentTime, Window, ButtonFlags, RI_MOUSE_BUTTON_5_DOWN, RI_MOUSE_BUTTON_5_UP, EMouseButtons::Thumb02, ButtonCursorPos);

			if (ButtonFlags & RI_MOUSE_WHEEL)
			{
				SHORT SignedDistance;
				memcpy(&SignedDistance, &Mouse.usButtonData, sizeof(SignedDistance));
				MessageHandler->OnMouseWheel(static_cast<float>(SignedDistance) / WHEEL_DELTA, ButtonCursorPos);
			}
		}
	}

	FlushRelativeMovement();
}

void FWindowsApplication::ProcessMouseButton(double CurrentTime, const TSharedPtr< FGenericWindow >& Window, USHORT ButtonFlags, USHORT DownFlag, USHORT UpFlag, EMouseButtons::Type MouseButton, FVector2D CursorPos)
{
	if (ButtonFlags & DownFlag)
	{
		constexpr float MouseDoubleClickDelay = 0.5f;

		const bool bHasDoubleClick = (CurrentTime <= RepeatTime[MouseButton]);
		if (bHasDoubleClick)
		{
			MessageHandler->OnMouseDoubleClick(Window, MouseButton, CursorPos);
		}
		else
		{
			MessageHandler->OnMouseDown(Window, MouseButton, CursorPos);
		}

		RepeatTime[MouseButton] = CurrentTime + MouseDoubleClickDelay;
	}
	else if (ButtonFlags & UpFlag)
	{
		MessageHandler->OnMouseUp(MouseButton);
	}
}

// Restore the windowsx.h macro for IsMaximized
#pragma pop_macro("IsMaximized")

#include "Windows/HideWindowsPlatformTypes.h"

