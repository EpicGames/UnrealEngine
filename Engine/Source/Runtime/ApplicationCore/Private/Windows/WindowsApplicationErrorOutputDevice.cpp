// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsApplicationErrorOutputDevice.h"
#include "Windows/WindowsHWrapper.h"
#include "HAL/PlatformApplicationMisc.h"

void FWindowsApplicationErrorOutputDevice::HandleErrorRestoreUI()
{
	// Unhide the mouse.
	while (::ShowCursor(true) < 0);
	// Release capture.
	::ReleaseCapture();
	// Allow mouse to freely roam around.
	::ClipCursor(NULL);

	// Copy to clipboard in non-cooked editor builds.
	FPlatformApplicationMisc::ClipboardCopy(GErrorHist);
}
