// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsConfigRules.h"

#if WITH_WINDOWS_CONFIGRULES
#include "GenericPlatform/GenericPlatformDriver.h"

/** URL to download ConfigRules file from. Must be a WIDE string starting with http: or https: */
#ifndef WINDOWS_CONFIGRULES_URL
#define WINDOWS_CONFIGRULES_URL TEXT("")
#endif

/** Secret used to derive AES key. Must be an ASCII string or char array. */
#ifndef WINDOWS_CONFIGRULES_KEY
#define WINDOWS_CONFIGRULES_KEY ""
#endif

/** Max miliseconds to spend waiting for download. Otherwise just proceed with bundled file */
#ifndef WINDOWS_CONFIGRULES_DOWNLOAD_TIMEOUT
#define WINDOWS_CONFIGRULES_DOWNLOAD_TIMEOUT (3000)
#endif

/** Path to save downloaded binary rules file */
#ifndef WINDOWS_CONFIGRULES_DOWNLOAD_PATH
#define WINDOWS_CONFIGRULES_DOWNLOAD_PATH FPaths::ProjectPersistentDownloadDir() / TEXT("ConfigRules") / TEXT("configrules.bin")
#endif

/** Path to find bundled config rules file */
#ifndef WINDOWS_CONFIGRULES_BUNDLED_PATH
#define WINDOWS_CONFIGRULES_BUNDLED_PATH TEXT("configrules.bin")
#endif

#include "Windows/WindowsHWrapper.h"
#include "RHIGlobals.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/Compression.h"
#include "Misc/FileHelper.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Windows/WindowsD3D.h"

#pragma comment(lib, "wininet.lib")
#pragma comment(lib, "bcrypt.lib")
#include <wininet.h>
#include <bcrypt.h>

#define CONFIGRULES_FILE_SIGNATURE (0x39d8)
#define CONFIGRULES_KEY_SALT { 0x23, 0x71, 0xd3, 0xa3, 0x30, 0x71, 0x63, 0xe3 }
#define CONFIGRULES_KEY_ROUNDS (1000)

#pragma pack(push, 1)
union FConfigRuleFileHeader // NOTE: Packed to match layout on disk
{
	uint8 Bytes[10];
	struct
	{
		uint16 Signature;
		int32 Version;
		int32 UncompressedSize;
	};
};
#pragma pack(pop)

struct FConfigRuleFile
{
	TArray<uint8> FileBytes;
	FConfigRuleFileHeader Header;
	const TCHAR *FilePath;

	void ParseHeader();
	bool Decrypt(TArray<uint8>& DecryptedBytes) const;

	FORCEINLINE TConstArrayView<uint8> GetContent() const
	{
		// Exclude header as actual content
		const uint8* Ptr = FileBytes.GetData() + sizeof(FConfigRuleFileHeader);
		const int32 Num = FMath::Max<int32>(0, FileBytes.Num() - sizeof(FConfigRuleFileHeader));
		return MakeConstArrayView(Ptr, Num);
	}
};

void FConfigRuleFile::ParseHeader()
{
	Header = {};
	Header.Version = INDEX_NONE;

	if (FileBytes.Num() >= sizeof(FConfigRuleFileHeader))
	{
		FConfigRuleFileHeader ReadHeader;
		FMemory::Memcpy(ReadHeader.Bytes, FileBytes.GetData(), sizeof(FConfigRuleFileHeader));

		// NOTE: Header is stored in big endian
#if PLATFORM_LITTLE_ENDIAN
		uint16 Signature = ByteSwap(ReadHeader.Signature);
		int32 Version = ByteSwap(ReadHeader.Version);
		int32 UncompressedSize = ByteSwap(ReadHeader.UncompressedSize);
#else
		uint16 Signature = ReadHeader.Signature;
		int32 Version = ReadHeader.Version;
		int32 UncompressedSize = ReadHeader.UncompressedSize;
#endif

		if (Signature == CONFIGRULES_FILE_SIGNATURE)
		{
			Header.Signature = Signature;
			Header.Version = Version;
			Header.UncompressedSize = UncompressedSize;
		}
	}
}

bool FConfigRuleFile::Decrypt(TArray<uint8>& DecryptedBytes) const
{
	TConstArrayView<uint8> BytesToDecrypt = GetContent();
	DecryptedBytes.SetNumUninitialized(BytesToDecrypt.Num());

	const uint8 Secret[] = WINDOWS_CONFIGRULES_KEY;
	const uint8 Salt[] = CONFIGRULES_KEY_SALT;
	const uint32 Rounds = CONFIGRULES_KEY_ROUNDS;

	uint8 GeneratedKey[16] = {};

	BCRYPT_ALG_HANDLE AlgoHandle = NULL;
	BCRYPT_KEY_HANDLE KeyHandle = NULL;

	NTSTATUS DeriveStatus = 0;
	DeriveStatus |= ::BCryptOpenAlgorithmProvider(&AlgoHandle, BCRYPT_SHA1_ALGORITHM, NULL, BCRYPT_ALG_HANDLE_HMAC_FLAG);
	DeriveStatus |= ::BCryptDeriveKeyPBKDF2(AlgoHandle,
		(PUCHAR)Secret,
		sizeof(Secret) - 1,
		(PUCHAR)Salt,
		sizeof(Salt),
		Rounds,
		(PUCHAR)GeneratedKey,
		sizeof(GeneratedKey),
		0
	);
	::BCryptCloseAlgorithmProvider(AlgoHandle, 0);

	if (BCRYPT_SUCCESS(DeriveStatus))
	{
		NTSTATUS KeyStatus = 0;
		KeyStatus |= ::BCryptOpenAlgorithmProvider(&AlgoHandle, BCRYPT_AES_ALGORITHM, NULL, 0);
		KeyStatus |= ::BCryptSetProperty(AlgoHandle, BCRYPT_CHAINING_MODE, (PUCHAR)BCRYPT_CHAIN_MODE_ECB, sizeof(BCRYPT_CHAIN_MODE_ECB), 0);
		KeyStatus |= ::BCryptGenerateSymmetricKey(AlgoHandle, &KeyHandle, NULL, 0, GeneratedKey, sizeof(GeneratedKey), 0);
		::BCryptCloseAlgorithmProvider(AlgoHandle, 0);

		if (BCRYPT_SUCCESS(KeyStatus))
		{
			ULONG DecryptedBytesNum = 0;
			NTSTATUS DecryptStatus = ::BCryptDecrypt(
				KeyHandle,
				(PUCHAR)BytesToDecrypt.GetData(),
				BytesToDecrypt.Num(),
				NULL,
				NULL,
				0,
				(PUCHAR)DecryptedBytes.GetData(),
				DecryptedBytes.Num(),
				&DecryptedBytesNum,
				BCRYPT_BLOCK_PADDING
			);
			
			if (BCRYPT_SUCCESS(DecryptStatus))
			{
				::BCryptDestroyKey(KeyHandle);
				DecryptedBytes.SetNum(DecryptedBytesNum);
				return true;
			}
		}
		::BCryptDestroyKey(KeyHandle);
	}

	return false;
}

struct FWindowsConfigRulesDownload
{
	HANDLE HDownloadThread;
	TArray<uint8> DownloadContent;

	FWindowsConfigRulesDownload();
	~FWindowsConfigRulesDownload();

	void SaveDownloadedFile();
};

// NOTE: static boots up the download thread ASAP, and allows us to register with GetPreMainInitDelegate()
static FWindowsConfigRulesDownload GWindowsConfigRulesDownload;

TMap<FString, FString> FWindowsConfigRules::PredefinedConfigRuleVars;
TMap<FString, FString> FWindowsConfigRules::ConfigRuleVariablesMap;
TArray<uint8> FWindowsConfigRules::ConfigRulesBytes;

void FWindowsConfigRules::Init()
{
#if !UE_BUILD_SHIPPING
	// Use a raw .txt file for quickly testing rule behavior.
	// Does not consider any other config rule file, nor does it check version.
	FString TestRawConfigRulesPath = TEXT("");
	if (FParse::Value(FCommandLine::Get(), TEXT("-TestRawConfigRulesPath="), TestRawConfigRulesPath))
	{
		bool bResult = FFileHelper::LoadFileToArray(ConfigRulesBytes, *TestRawConfigRulesPath);
		if (!bResult)
		{
			ConfigRulesBytes.Reset();
			UE_LOGF(LogConfigRules, Error, "Failed to load Testing file: %ls", *TestRawConfigRulesPath);
		}
		return;
	}
#endif

	const FString& DownloadPath = WINDOWS_CONFIGRULES_DOWNLOAD_PATH;
	TArray<FString> FilePaths = {
		DownloadPath, /* Previously downloaded file */
		WINDOWS_CONFIGRULES_BUNDLED_PATH
	};

#if !UE_BUILD_SHIPPING
	FString DebugConfigRulesPath = TEXT("");
	if (FParse::Value(FCommandLine::Get(), TEXT("-DebugConfigRulesPath="), DebugConfigRulesPath))
	{
		FilePaths.Add(DebugConfigRulesPath);
	}
#endif // !UE_BUILD_SHIPPING

	TArray<FConfigRuleFile> Files;
	Files.Reserve(FilePaths.Num() + 1);

	int32 LatestFileIndex = INDEX_NONE;
	int32 LatestVersion = INDEX_NONE;

	// Load file in to memory and find the one with highest version
	for (const FString& FilePath : FilePaths)
	{
		const bool bExists = FPaths::FileExists(*FilePath);
		if (bExists)
		{
			const int32 Index = Files.AddDefaulted();
			FConfigRuleFile& File = Files[Index];
			File.FilePath = *FilePath;

			bool bLoadFileResult = FFileHelper::LoadFileToArray(File.FileBytes, *FilePath);
			if (bLoadFileResult)
			{
				File.ParseHeader();
				if (File.Header.Version >= 0 && File.Header.Version > LatestVersion)
				{
					LatestFileIndex = Index;
					LatestVersion = File.Header.Version;
				}
			}
			else
			{
				UE_LOGF(LogConfigRules, Error, "Could not LoadFileToArray: %ls", *FilePath);
			}
		}
		else
		{
			UE_LOGF(LogConfigRules, Log, "Could not find file: %ls", *FilePath);
		}
	}

	DWORD DownloadWaitCode = ::WaitForSingleObject(GWindowsConfigRulesDownload.HDownloadThread, WINDOWS_CONFIGRULES_DOWNLOAD_TIMEOUT);
	DWORD DownloadExitCode = INDEX_NONE;
	::GetExitCodeThread(GWindowsConfigRulesDownload.HDownloadThread, &DownloadExitCode);

	if (DownloadExitCode == 0 && DownloadWaitCode == WAIT_OBJECT_0)
	{
		// NOTE: Performing the save here since paths/cmdline might not be initialized on the download thread yet
		GWindowsConfigRulesDownload.SaveDownloadedFile();

		// If download was successful check if it has a higher version
		const int32 Index = Files.AddDefaulted();
		FConfigRuleFile& File = Files[Index];
		File.FilePath = *DownloadPath;

		File.FileBytes = MoveTemp(GWindowsConfigRulesDownload.DownloadContent);
		File.ParseHeader();
		if (File.Header.Version >= 0 && File.Header.Version > LatestVersion)
		{
			LatestFileIndex = Index;
			LatestVersion = File.Header.Version;
		}
	}
	else
	{
		UE_LOGF(LogConfigRules, Warning, "DownloadThread unsuccessful, only using bundled file");
	}
	
	// Decrypt and Inflate the latest file
	if (LatestFileIndex != INDEX_NONE && LatestVersion >= 0)
	{
		const FConfigRuleFile& File = Files[LatestFileIndex];
		UE_LOGF(LogConfigRules, Log, "Using file %ls with version %d", File.FilePath, LatestVersion);

		TArray<uint8> DecryptedBytes;
		const bool bDecrypted = File.Decrypt(DecryptedBytes);

		if (bDecrypted)
		{
			TArray<uint8> InflatedBytes;
			InflatedBytes.SetNumUninitialized(File.Header.UncompressedSize);
			const bool bInflated = FCompression::UncompressMemory(NAME_Zlib, InflatedBytes.GetData(), InflatedBytes.Num(), DecryptedBytes.GetData(), DecryptedBytes.Num());

			if (bInflated)
			{
				ConfigRulesBytes = MoveTemp(InflatedBytes);
			}
			else
			{
				UE_LOGF(LogConfigRules, Error, "Failed to Uncompress file");
			}
		}
		else
		{
			UE_LOGF(LogConfigRules, Error, "Failed to Decrypt file");
		}
	}
	else
	{
		UE_LOGF(LogConfigRules, Error, "Unable to find a file with valid version");
	}
}

void FWindowsConfigRules::Parse()
{
	if (ConfigRulesBytes.Num())
	{
		TMap<FString, FString> PredefinedVars = PredefinedConfigRuleVars;
		ConfigRuleVariablesMap = FGenericConfigRules::ParseConfigRules(ConfigRulesBytes, MoveTemp(PredefinedVars));
	}
}

void FWindowsConfigRules::SetStandardPredefinedVars()
{
	FString CPUVendor = FWindowsPlatformMisc::GetCPUVendor();
	FString CPUBrand = FWindowsPlatformMisc::GetCPUBrand();
	FString OSVersion = FWindowsPlatformMisc::GetOSVersion();
	FString Cores = FString::FromInt(FWindowsPlatformMisc::NumberOfCores());
	FString HostArch = FWindowsPlatformMisc::GetHostArchitecture();
	FString MakeAndModel = FPlatformMisc::GetDeviceMakeAndModel();
	FString BranchName = FApp::GetBranchName();
	FString BuildVersion = FApp::GetBuildVersion();
	FString BuildConfig = LexToString(FApp::GetBuildConfiguration());
	FString BuildTarget = LexToString(FApp::GetBuildTargetType());
#ifdef UE_BUILD_FLAVOR
	const FString BuildFlavor(UE_BUILD_FLAVOR);
#else
	const FString BuildFlavor;
#endif
	FString TotalPhysicalGB = FString::FromInt(static_cast<int32>(FPlatformMemory::GetConstants().TotalPhysicalGB));
	FString TotalVirtualGB = FString::FromInt(static_cast<int32>(FPlatformMemory::GetConstants().TotalVirtual / 1024 / 1024 / 1024));

	PredefinedConfigRuleVars.Emplace(TEXT("SRC_BranchName"), BranchName);
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_BuildVersion"), BuildVersion);
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_BuildConfig"), BuildConfig);
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_BuildTarget"), BuildTarget);
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_BuildFlavor"), BuildFlavor);
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_MakeAndModel"), MakeAndModel);
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_CPUVendor"), CPUVendor);
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_CPUBrand"), CPUBrand);
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_HostArch"), HostArch);
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_NumberOfCores"), Cores);
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_OSVersion"), OSVersion);
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_TotalPhysicalGB"), TotalPhysicalGB);
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_TotalVirtualGB"), TotalVirtualGB);
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_IsHandheld"), FWindowsPlatformMisc::IsDeviceGamingHandheld() ? TEXT("1") : TEXT("0"));

	const FWindowsGPUInfo GPUInfo = FWindowsD3D::GetExpectedGPUInfo();
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_ExpectedGPUVendorId"), FString::Printf(TEXT("%u"), GPUInfo.VendorId));
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_ExpectedGPUDeviceId"), FString::Printf(TEXT("%u"), GPUInfo.DeviceId));
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_ExpectedDedicatedVideoMemoryMB"), FString::Printf(TEXT("%f"), static_cast<double>(GPUInfo.DedicatedVideoMemory) / 1048576.));
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_ExpectedGPUAdapterName"), GPUInfo.AdapterName.IsEmpty() ? FPlatformMisc::GetPrimaryGPUBrand() : GPUInfo.AdapterName);
}

void FWindowsConfigRules::SetRHIPredefinedVars(const FString& DynamicRHIModuleName, const FString& RequestedFeatureLevel)
{
	FGPUDriverInfo GPUDriverInfo = {};
	if (!GRHIAdapterName.IsEmpty())
	{
		GPUDriverInfo = FPlatformMisc::GetGPUDriverInfo(GRHIAdapterName);
	}

	PredefinedConfigRuleVars.Emplace(TEXT("SRC_DynamicRHIModuleName"), DynamicRHIModuleName);
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_DynamicRHIFeatureLevel"), RequestedFeatureLevel);
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_GRHIDeviceId"), FString::Printf(TEXT("%u"), GRHIDeviceId));
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_GRHIVendorId"), FString::Printf(TEXT("%u"), GRHIVendorId));
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_RHIDedicatedVideoMemoryMB"), FString::Printf(TEXT("%f"), static_cast<double>(GRHIGlobals.GpuInfo.DedicatedVideoMemory) / 1048576.));
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_GRHIAdapterName"), GRHIAdapterName);
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_GRHIAdapterUserDriverVersion"), GPUDriverInfo.UserDriverVersion);
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_GRHIAdapterInternalDriverVersion"), GPUDriverInfo.InternalDriverVersion);
	PredefinedConfigRuleVars.Emplace(TEXT("SRC_GRHIAdapterDriverDate"), GPUDriverInfo.DriverDate);

	TArray<FString> DriverDateComponents;
	GPUDriverInfo.DriverDate.ParseIntoArray(DriverDateComponents, TEXT("-"), true);
	if (DriverDateComponents.Num() == 3)
	{
		PredefinedConfigRuleVars.Emplace(TEXT("SRC_GRHIAdapterDriverDateMonth"), DriverDateComponents[0]);
		PredefinedConfigRuleVars.Emplace(TEXT("SRC_GRHIAdapterDriverDateDay"),   DriverDateComponents[1]);
		PredefinedConfigRuleVars.Emplace(TEXT("SRC_GRHIAdapterDriverDateYear"),  DriverDateComponents[2]);
	}

	// Split the driver version(s) into a field for each component, usually integers (but some vendors have strings like AMD with 'Adrenalin').
	// Since the number of components varies by vendor, collect components from the LAST (component 1) to the FIRST (component N),
	// to make it easier to build match rules.
	TArray<FString> UserDriverVersionComponents;
	GPUDriverInfo.UserDriverVersion.ParseIntoArray(UserDriverVersionComponents, TEXT("."), true);
	if (!UserDriverVersionComponents.IsEmpty())
	{
		uint32 ComponentNumFromLast = 1;
		for (int32 Idx = UserDriverVersionComponents.Num() - 1; Idx >= 0; --Idx)
		{
			FString NameStr = FString(TEXT("SRC_GRHIAdapterUserDriverVersionComponent")) + FString::FromInt(ComponentNumFromLast);
			PredefinedConfigRuleVars.Emplace(NameStr, UserDriverVersionComponents[Idx]);
			ComponentNumFromLast++;
		}
	}

	TArray<FString> InternalDriverVersionComponents;
	GPUDriverInfo.InternalDriverVersion.ParseIntoArray(InternalDriverVersionComponents, TEXT("."), true);
	if (!InternalDriverVersionComponents.IsEmpty())
	{
		uint32 ComponentNumFromLast = 1;
		for (int32 Idx = InternalDriverVersionComponents.Num() - 1; Idx >= 0; --Idx)
		{
			FString NameStr = FString(TEXT("SRC_GRHIAdapterInternalDriverVersionComponent")) + FString::FromInt(ComponentNumFromLast);
			PredefinedConfigRuleVars.Emplace(NameStr, InternalDriverVersionComponents[Idx]);
			ComponentNumFromLast++;
		}
	}
}

void FWindowsConfigRules::SetAdditionalPredefinedVars(TMap<FString, FString>&& AdditionalPredefinedVars)
{
	PredefinedConfigRuleVars.Append(MoveTemp(AdditionalPredefinedVars));
}

static DWORD ReportInternetError(const TCHAR* Str)
{
	DWORD LastError = ::GetLastError();

	if (LastError == ERROR_INTERNET_EXTENDED_ERROR)
	{
		DWORD IError = 0;
		WCHAR IErrorStr[1024] = {};
		DWORD IErrorStrLen = UE_ARRAY_COUNT(IErrorStr);
		BOOL IErrorResult = ::InternetGetLastResponseInfoW(&IError, (LPWSTR)&IErrorStr, &IErrorStrLen);
		UE_LOGF(LogConfigRules, Log, "%ls, LastError %d, Response (%d: %ls)",
			Str, (uint32)LastError, (uint32)IError, IErrorStr);
	}
	else
	{
		UE_LOGF(LogConfigRules, Log, "%ls, LastError: %d", Str, (uint32)LastError);
	}

	return LastError;
}

static DWORD WINAPI WinConfigRulesDownloadProc(void* Arg)
{
#if !UE_BUILD_SHIPPING
	if (FParse::Param(::GetCommandLineW(), TEXT("NoConfigRulesDownload")))
	{
		return 0;
	}
#endif // !UE_BUILD_SHIPPING

	FWindowsConfigRulesDownload* This = (FWindowsConfigRulesDownload *)Arg;
	DWORD ExitStatus = 0;

	HINTERNET HInternet = ::InternetOpenW(L"WinConfigRulesDownloadProc", INTERNET_OPEN_TYPE_DIRECT, NULL, NULL, 0);
	if (HInternet)
	{
		FString DownloadUrl;
#if !UE_BUILD_SHIPPING
		if (!FParse::Value(::GetCommandLineW(), TEXT("-DebugConfigRulesUrl="), DownloadUrl))
#endif // !UE_BUILD_SHIPPING
		{
			DownloadUrl = WINDOWS_CONFIGRULES_URL;
		}

		// This actually sends the request (blocking)
		HINTERNET HFile = ::InternetOpenUrlW(HInternet, *DownloadUrl, NULL, 0, INTERNET_FLAG_SECURE, NULL);
		if (HFile)
		{
			TArray<uint8>& DownloadContent = This->DownloadContent;

			uint8 Buffer[4096];
			DWORD ReadBytes = 0;
			BOOL ReadResult = 0;
			do
			{
				ReadResult = ::InternetReadFile(HFile, Buffer, UE_ARRAY_COUNT(Buffer), &ReadBytes);
				if (ReadResult)
				{
					DownloadContent.Append(Buffer, (int32)ReadBytes);
				}
				else
				{
					ExitStatus = ReportInternetError(TEXT("Failed to read response"));
				}
			} while (ReadResult && ReadBytes != 0);

			::InternetCloseHandle(HFile);
			HFile = NULL;
		}
		else
		{
			ExitStatus = ReportInternetError(TEXT("Failed to open url"));
		}

		::InternetCloseHandle(HInternet);
		HInternet = NULL;
	}
	else
	{
		ExitStatus = ReportInternetError(TEXT("Failed to open connection"));
	}

	return ExitStatus;
}

FWindowsConfigRulesDownload::FWindowsConfigRulesDownload()
{
	HDownloadThread = ::CreateThread(NULL, 32 << 10, WinConfigRulesDownloadProc, (void*)this, 0, NULL);
}

FWindowsConfigRulesDownload::~FWindowsConfigRulesDownload()
{
	if (HDownloadThread)
	{
		if (::WaitForSingleObject(HDownloadThread, 1000) == WAIT_OBJECT_0)
		{
			::CloseHandle(HDownloadThread);
		}
		HDownloadThread = NULL;
	}
}

void FWindowsConfigRulesDownload::SaveDownloadedFile()
{
	const FString& FilePath = WINDOWS_CONFIGRULES_DOWNLOAD_PATH;
	const FString& DirectoryPath = FPaths::GetPath(FilePath);

	IPlatformFile& PlatformFile = IPlatformFile::GetPlatformPhysical();
	if (PlatformFile.CreateDirectoryTree(*DirectoryPath))
	{
		IFileHandle* Handle = PlatformFile.OpenWrite(*FilePath, false, true);
		if (Handle)
		{
			bool bWriteResult = Handle->Write(DownloadContent.GetData(), DownloadContent.Num());
			if (bWriteResult)
			{
				UE_LOGF(LogConfigRules, Log, "Wrote %d bytes to %ls", DownloadContent.Num(), *FilePath);
			}
			else
			{
				UE_LOGF(LogConfigRules, Error, "Failed to write to %ls", *FilePath);
			}

			delete Handle;
			Handle = nullptr;
		}
		else
		{
			UE_LOGF(LogConfigRules, Error, "Failed to OpenWrite %ls", *FilePath);
		}
	}
	else
	{
		UE_LOGF(LogConfigRules, Error, "Failed to CreateDirectoryTree %ls", *DirectoryPath);
	}
}

void FWindowsConfigRules::Release()
{
	PredefinedConfigRuleVars.Empty();
	ConfigRulesBytes.Empty();
}

/** Cleanup resources not needed after engine init */
static FDelayedAutoRegisterHelper GWindowsConfigRulesEndOfInit(
	EDelayedRegisterRunPhase::EndOfEngineInit, []
	{
		FWindowsConfigRules::Release();
	});

#endif // WITH_WINDOWS_CONFIGRULES
