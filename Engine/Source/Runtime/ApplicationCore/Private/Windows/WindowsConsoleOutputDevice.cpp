// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsConsoleOutputDevice.h"
#include "Logging/LogMacros.h"
#include "Misc/OutputDevice.h"
#include "HAL/PlatformTime.h"
#include "Math/UnrealMathUtility.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTemplate.h"
#include "CoreGlobals.h"
#include "Misc/CString.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "Internationalization/Internationalization.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/CoreDelegates.h"
#include "GenericPlatform/GenericApplication.h"
#include "Misc/App.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/ThreadHeartBeat.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#include "String/Find.h"

namespace OutputDeviceConstants
{
	uint32 WIN_STD_OUTPUT_HANDLE = STD_OUTPUT_HANDLE;
	uint32 WIN_ATTACH_PARENT_PROCESS = ATTACH_PARENT_PROCESS;
}

#include "Windows/HideWindowsPlatformTypes.h"

FWindowsConsoleOutputDevice::FWindowsConsoleOutputDevice()
	: ConsoleHandle(0)
	, OverrideColorSet(false)
	, bIsAttached(false)
{
#if !UE_BUILD_SHIPPING
	auto ParseColorStr = [](const FString& ColorStr) -> const TCHAR*
	{
		const TCHAR* ReturnVal = nullptr;

		if (ColorStr == TEXTVIEW("Black"))
		{
			ReturnVal = COLOR_BLACK;
		}
		else if (ColorStr == TEXTVIEW("DarkRed"))
		{
			ReturnVal = COLOR_DARK_RED;
		}
		else if (ColorStr == TEXTVIEW("DarkGreen"))
		{
			ReturnVal = COLOR_DARK_GREEN;
		}
		else if (ColorStr == TEXTVIEW("DarkBlue"))
		{
			ReturnVal = COLOR_DARK_BLUE;
		}
		else if (ColorStr == TEXTVIEW("DarkYellow"))
		{
			ReturnVal = COLOR_DARK_YELLOW;
		}
		else if (ColorStr == TEXTVIEW("DarkCyan"))
		{
			ReturnVal = COLOR_DARK_CYAN;
		}
		else if (ColorStr == TEXTVIEW("DarkPurple"))
		{
			ReturnVal = COLOR_DARK_PURPLE;
		}
		else if (ColorStr == TEXTVIEW("Gray"))
		{
			ReturnVal = COLOR_DARK_WHITE;
		}
		else if (ColorStr == TEXTVIEW("Red"))
		{
			ReturnVal = COLOR_RED;
		}
		else if (ColorStr == TEXTVIEW("Green"))
		{
			ReturnVal = COLOR_GREEN;
		}
		else if (ColorStr == TEXTVIEW("Blue"))
		{
			ReturnVal = COLOR_BLUE;
		}
		else if (ColorStr == TEXTVIEW("Yellow"))
		{
			ReturnVal = COLOR_YELLOW;
		}
		else if (ColorStr == TEXTVIEW("Cyan"))
		{
			ReturnVal = COLOR_CYAN;
		}
		else if (ColorStr == TEXTVIEW("Purple"))
		{
			ReturnVal = COLOR_PURPLE;
		}
		else if (ColorStr == TEXTVIEW("White"))
		{
			ReturnVal = COLOR_WHITE;
		}

		return ReturnVal;
	};

	FString HighlightsStr;

	// -LogHighlights="LogNet Cyan, LogTemp Green"
	if (FParse::Value(FCommandLine::Get(), TEXT("LogHighlights="), HighlightsStr))
	{
		TArray<FString> HighlightsList;

		HighlightsStr.ParseIntoArray(HighlightsList, TEXT(","));

		for (const FString& CurHighlightEntry : HighlightsList)
		{
			TArray<FString> CategoryAndColor;

			if (CurHighlightEntry.TrimStartAndEnd().ParseIntoArray(CategoryAndColor, TEXT(" ")) && CategoryAndColor.Num() == 2)
			{
				const TCHAR* ColorStr = ParseColorStr(CategoryAndColor[1].TrimStartAndEnd());

				if (ColorStr != nullptr)
				{
					FLogHighlight& NewEntry = LogHighlights.AddDefaulted_GetRef();

					NewEntry.Category = FName(CategoryAndColor[0]);
					NewEntry.Color = ColorStr;
				}
			}
		}
	}

	FString StringHighlights;

	// -LogStringHighlights="UNetConnection::Close=Purple, NotifyAcceptingConnection accepted from=DarkGreen"
	if (FParse::Value(FCommandLine::Get(), TEXT("LogStringHighlights="), StringHighlights))
	{
		TArray<FString> StringHighlightsList;

		StringHighlights.ParseIntoArray(StringHighlightsList, TEXT(","));

		for (const FString& CurStringHighlightEntry : StringHighlightsList)
		{
			TArray<FString> StringAndColor;

			if (CurStringHighlightEntry.ParseIntoArray(StringAndColor, TEXT("=")) && StringAndColor.Num() == 2)
			{
				const TCHAR* ColorStr = ParseColorStr(StringAndColor[1].TrimStartAndEnd());

				if (ColorStr != nullptr)
				{
					FLogStringHighlight& NewEntry = LogStringHighlights.AddDefaulted_GetRef();

					NewEntry.SearchString = StringAndColor[0].GetCharArray();
					NewEntry.Color = ColorStr;
				}
			}
		}
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("SuppressConsoleOutputVerboseLogging")))
	{
		AllowedLogVerbosity = ELogVerbosity::Log;
	}
#endif
}

FWindowsConsoleOutputDevice::~FWindowsConsoleOutputDevice()
{
	SaveToINI();

	// WRH - 2007/08/23 - This causes the process to take a LONG time to shut down when clicking the "close window"
	// button on the top-right of the console window.
	//FreeConsole();
}

void FWindowsConsoleOutputDevice::SaveToINI()
{
	if (GConfig && !IniFilename.IsEmpty())
	{
		HWND Handle = GetConsoleWindow();
		if (Handle != NULL)
		{
			RECT WindowRect;
			::GetWindowRect(Handle, &WindowRect);

			GConfig->SetInt(TEXT("DebugWindows"), TEXT("ConsoleX"), WindowRect.left, IniFilename);
			GConfig->SetInt(TEXT("DebugWindows"), TEXT("ConsoleY"), WindowRect.top, IniFilename);
		}
		
		if (ConsoleHandle != NULL)
		{
			CONSOLE_SCREEN_BUFFER_INFO Info;
			if (GetConsoleScreenBufferInfo(ConsoleHandle, &Info))
			{
				GConfig->SetInt(TEXT("DebugWindows"), TEXT("ConsoleWidth"), Info.dwSize.X, IniFilename);
				GConfig->SetInt(TEXT("DebugWindows"), TEXT("ConsoleHeight"), Info.dwSize.Y, IniFilename);
			}
		}
	}
}

void FWindowsConsoleOutputDevice::Show( bool ShowWindow )
{
	if( ShowWindow )
	{
		if( !ConsoleHandle )
		{
			if (!AllocConsole())
			{
				bIsAttached = true;
			}
			else
			{
				FWindowsPlatformMisc::SetGracefulTerminationHandler();
			}
			ConsoleHandle = GetStdHandle(OutputDeviceConstants::WIN_STD_OUTPUT_HANDLE);

			if( ConsoleHandle != INVALID_HANDLE_VALUE )
			{
				COORD Size;
				Size.X = 160;
				Size.Y = 4000;

				int32 ConsoleWidth = 160;
				int32 ConsoleHeight = 4000;
				int32 ConsolePosX = 0;
				int32 ConsolePosY = 0;
				bool bHasX = false;
				bool bHasY = false;

				if(GConfig)
				{
					GConfig->GetInt(TEXT("DebugWindows"), TEXT("ConsoleWidth"), ConsoleWidth, GGameIni);
					GConfig->GetInt(TEXT("DebugWindows"), TEXT("ConsoleHeight"), ConsoleHeight, GGameIni);
					bHasX = GConfig->GetInt(TEXT("DebugWindows"), TEXT("ConsoleX"), ConsolePosX, GGameIni);
					bHasY = GConfig->GetInt(TEXT("DebugWindows"), TEXT("ConsoleY"), ConsolePosY, GGameIni);

					Size.X = (SHORT)ConsoleWidth;
					Size.Y = (SHORT)ConsoleHeight;
				}

				SetConsoleScreenBufferSize( ConsoleHandle, Size );

				CONSOLE_SCREEN_BUFFER_INFO ConsoleInfo;

				// Try to set the window width to match the screen buffer width, so that no manual horizontal scrolling or resizing is necessary
				if (::GetConsoleScreenBufferInfo( ConsoleHandle, &ConsoleInfo ) != 0)
				{
					SMALL_RECT NewConsoleWindowRect = ConsoleInfo.srWindow;
					NewConsoleWindowRect.Right = ConsoleInfo.dwSize.X - 1;
					::SetConsoleWindowInfo( ConsoleHandle, true, &NewConsoleWindowRect );	
				}

				RECT WindowRect;
				::GetWindowRect( GetConsoleWindow(), &WindowRect );

				if (!FParse::Value(FCommandLine::Get(), TEXT("ConsoleX="), ConsolePosX) && !bHasX)
				{
					ConsolePosX = WindowRect.left;
				}

				if (!FParse::Value(FCommandLine::Get(), TEXT("ConsoleY="), ConsolePosY) && !bHasY)
				{
					ConsolePosY = WindowRect.top;
				}

				FString ConsoleTitle;
				if (FParse::Value(FCommandLine::Get(), TEXT("ConsoleTitle="), ConsoleTitle))
				{
					SetConsoleTitleW(*ConsoleTitle);
				}

				FDisplayMetrics DisplayMetrics;
				FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);

				// Make sure that the positions specified by INI/CMDLINE are proper
				static const int32 ActualConsoleWidth = WindowRect.right - WindowRect.left;
				static const int32 ActualConsoleHeight = WindowRect.bottom - WindowRect.top;

				static const int32 ActualScreenWidth = DisplayMetrics.VirtualDisplayRect.Right - DisplayMetrics.VirtualDisplayRect.Left;
				static const int32 ActualScreenHeight = DisplayMetrics.VirtualDisplayRect.Bottom - DisplayMetrics.VirtualDisplayRect.Top;

				static const int32 RightPadding = FMath::Max(50, FMath::Min((ActualConsoleWidth / 2), ActualScreenWidth / 2));
				static const int32 BottomPadding = FMath::Max(50, FMath::Min((ActualConsoleHeight / 2), ActualScreenHeight / 2));
				
				ConsolePosX = FMath::Min(FMath::Max(ConsolePosX, DisplayMetrics.VirtualDisplayRect.Left), DisplayMetrics.VirtualDisplayRect.Right - RightPadding);
				ConsolePosY = FMath::Min(FMath::Max(ConsolePosY, DisplayMetrics.VirtualDisplayRect.Top), DisplayMetrics.VirtualDisplayRect.Bottom - BottomPadding);

				::SetWindowPos( GetConsoleWindow(), HWND_TOP, ConsolePosX, ConsolePosY, 0, 0, SWP_NOSIZE | SWP_NOSENDCHANGING | SWP_NOZORDER );
			}
		}
	}
	else if( ConsoleHandle )
	{
		SaveToINI();

		ConsoleHandle = NULL;
		FreeConsole();
		bIsAttached = false;
	}
}

bool FWindowsConsoleOutputDevice::IsShown()
{
	return ConsoleHandle != NULL;
}

void FWindowsConsoleOutputDevice::Serialize( const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category, const double Time )
{
	using namespace UE::String;

	if ((Verbosity & ELogVerbosity::VerbosityMask) > AllowedLogVerbosity)
		return;

	if( ConsoleHandle )
	{
		const double RealTime = Time == -1.0f ? FPlatformTime::Seconds() - GStartTime : Time;

		static bool Entry=false;
		if( !GIsCriticalError || Entry )
		{
			if (Verbosity == ELogVerbosity::SetColor)
			{
				SetColor( Data );
				OverrideColorSet = FCString::Strcmp(COLOR_NONE, Data) != 0;
			}
			else
			{
				bool bNeedToResetColor = false;
				if( !OverrideColorSet )
				{
					if( Verbosity == ELogVerbosity::Error )
					{
						SetColor( COLOR_RED );
						bNeedToResetColor = true;
					}
					else if( Verbosity == ELogVerbosity::Warning )
					{
						SetColor( COLOR_YELLOW );
						bNeedToResetColor = true;
					}
#if !UE_BUILD_SHIPPING
					else
					{
						if (LogHighlights.Num() > 0)
						{
							if (const FLogHighlight* CurHighlight = LogHighlights.FindByKey(Category))
							{
								SetColor(CurHighlight->Color);
								bNeedToResetColor = true;
							}
						}

						if (LogStringHighlights.Num() > 0)
						{
							FStringView DataView = TStringView(Data);

							for (const FLogStringHighlight& CurStringHighlight : LogStringHighlights)
							{
								FStringView SearchView = TStringView(CurStringHighlight.SearchString.GetData(),
																		CurStringHighlight.SearchString.Num()-1);

								if (FindFirst(DataView, SearchView, ESearchCase::IgnoreCase) != INDEX_NONE)
								{
									SetColor(CurStringHighlight.Color);
									bNeedToResetColor = true;

									break;
								}
							}
						}
					}
#endif
				}
				TCHAR OutputString[MAX_SPRINTF]=TEXT(""); //@warning: this is safe as FCString::Sprintf only use 1024 characters max
				FCString::Sprintf(OutputString,TEXT("%s%s"),*FOutputDeviceHelper::FormatLogLine(Verbosity, Category, Data, GPrintLogTimes,RealTime),LINE_TERMINATOR);

				// WriteConsole blocks when user interacting with console window (text is selected, scrollbar held, etc..)
				FSlowHeartBeatScope SuspendHeartBeat;
				uint32 Written;
				WriteConsole( ConsoleHandle, OutputString, FCString::Strlen(OutputString), (::DWORD*)&Written, NULL );

				if( bNeedToResetColor )
				{
					SetColor( COLOR_NONE );
				}
			}
		}
		else
		{
			Entry=true;
			Serialize( Data, Verbosity, Category );
			Entry=false;
		}
	}
}

void FWindowsConsoleOutputDevice::Serialize( const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category )
{
	Serialize( Data, Verbosity, Category, -1.0 );
}

void FWindowsConsoleOutputDevice::SetColor( const TCHAR* Color )
{
	// here we can change the color of the text to display, it's in the format:
	// ForegroundRed | ForegroundGreen | ForegroundBlue | ForegroundBright | BackgroundRed | BackgroundGreen | BackgroundBlue | BackgroundBright
	// where each value is either 0 or 1 (can leave off trailing 0's), so 
	// blue on bright yellow is "00101101" and red on black is "1"
	// An empty string reverts to the normal gray on black
	if (FCString::Stricmp(Color, TEXT("")) == 0)
	{
		SetConsoleTextAttribute(ConsoleHandle, FOREGROUND_BLUE | FOREGROUND_GREEN | FOREGROUND_RED);
	}
	else
	{
		// turn the string into a bunch of 0's and 1's
		TCHAR String[9];
		FMemory::Memset(String, 0, sizeof(TCHAR) * UE_ARRAY_COUNT(String));
		FCString::Strncpy(String, Color, UE_ARRAY_COUNT(String));
		for (TCHAR* S = String; *S; S++)
		{
			*S -= '0';
		}
		// make the color
		SetConsoleTextAttribute(ConsoleHandle, 
			(String[0] ? FOREGROUND_RED			: 0) | 
			(String[1] ? FOREGROUND_GREEN		: 0) | 
			(String[2] ? FOREGROUND_BLUE		: 0) | 
			(String[3] ? FOREGROUND_INTENSITY	: 0) | 
			(String[4] ? BACKGROUND_RED			: 0) | 
			(String[5] ? BACKGROUND_GREEN		: 0) | 
			(String[6] ? BACKGROUND_BLUE		: 0) | 
			(String[7] ? BACKGROUND_INTENSITY	: 0) );
	}
}

bool FWindowsConsoleOutputDevice::IsAttached()
{
	if (ConsoleHandle != NULL)
	{
		return bIsAttached;
	}
	else if (!AttachConsole(OutputDeviceConstants::WIN_ATTACH_PARENT_PROCESS))
	{
		if (GetLastError() == ERROR_ACCESS_DENIED)
		{
			return true;
		}
	}
	else
	{
		FreeConsole();
	}
	return false;
}

bool FWindowsConsoleOutputDevice::CanBeUsedOnAnyThread() const 
{
	return true;
}

bool FWindowsConsoleOutputDevice::CanBeUsedOnPanicThread() const 
{
	return true;
}
