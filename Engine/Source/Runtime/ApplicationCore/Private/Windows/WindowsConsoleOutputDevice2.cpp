// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsConsoleOutputDevice2.h"
#include "Async/Async.h"
#include "Containers/RingBuffer.h"
#include "CoreGlobals.h"
#include "Features/IModularFeatures.h"
#include "GenericPlatform/GenericApplication.h"
#include "HAL/ConsoleManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformTime.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CString.h"
#include "Misc/FileHelper.h"
#include "Misc/OutputDevice.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/Parse.h"
#include "Misc/ScopeRWLock.h"
#include <Misc/Timespan.h>
#include "Misc/TrackedActivity.h"
#include "String/Find.h"
#include "Templates/UnrealTemplate.h"
#include "Windows/WindowsPlatformApplicationMisc.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include <CommCtrl.h>
#include <uxtheme.h>
#include "Windows/HideWindowsPlatformTypes.h"

class FWindowsConsoleOutputDevice2::FConsoleWindow
{
public:
	static constexpr int32 LogHistoryMaxSize = 200000;
	static constexpr const TCHAR* HelpTitle = TEXT("Unreal Console Help");
	static constexpr const TCHAR* HelpText = TEXT(
		"Welcome to Unreal Console!\r\n"
		"\r\n"
		"Unreal Console is a replacement of the built-in console. If you still want to use windows console, remove '-NewConsole'\r\n"
		"\r\n"
		"Use double down-arrow in top right corner to expand console to show command line and other features.\r\n"
		"\r\n"
		"Console commands:\r\n"
		"  showdate\t - Show time in log\r\n"
		"  showdatetime\t - Show date/time in log\r\n"
		"  hidedatetime\t - Hide date/time from log\r\n"
		"  showverbosity\t - Show verbosity from log\r\n"
		"  hideverbosity\t - Hide verbosity from log\r\n"
		"  <color>\t - Change color of console\r\n"
		"\r\n"
		"Supported colors: red,gray,darkgray,lightblue,darkblue\r\n"
		"\r\n"
		"Console settings are saved separately for client/server/other in basedir folder\r\n"
		"\r\n"
		"Enjoy!\r\n"
	);

	enum ENotificationId : int
	{
		ID_LOG = 0x8801,
		ID_LOG_INCLUDE_FILTER,
		ID_LOG_EXCLUDE_FILTER,
		ID_COMMAND,
		ID_COMMANDBUTTON,
		ID_CLEARLOGBUTTON,
		ID_ADDCHECKPOINTBUTTON,
		ID_TRACKEDACTIVITY,
	};
	enum : int
	{
		HTEXPAND = 1000, // Just some value that is not any of the existing HT-ones
	};

	FConsoleWindow(FWindowsConsoleOutputDevice2* InOwner)
	:	Owner(InOwner)
	{
		StartDateTime = FDateTime::Now();
		StartTime = FPlatformTime::Seconds() - GStartTime;

		// Set up status light brushes
		COLORREF Colors[] = {RGB(237, 28, 36), RGB(255, 255, 0), RGB(128, 255, 128)};
		for (int32 I = 0; I != 3; ++I)
			StatusLightBrush[I] = CreateSolidBrush(Colors[I]);

		// Create Event used to trigger update in log window thread
		DirtyEvent = CreateEvent(NULL, false, false, NULL);

		// Can't remove this in dtor since it happens after OnExit delegate is destroyed`
		FCoreDelegates::OnExit.AddLambda([this]
			{
				bool bSaveIni = bIsVisible;
				//bIsVisible = false;
				if (bSaveIni)
					Owner->SaveToINI();
			});
	}

	~FConsoleWindow()
	{
		FTrackedActivity::UnregisterEventListener();

		bExiting = true;
		PostMessageW(MainHwnd, WM_CLOSE, 0, 0);
		WaitForSingleObject(Thread, 3000);
		CloseHandle(Thread);
		Thread = nullptr;

		CloseHandle(DirtyEvent);

		for (int32 I=0; I!=3; ++I)
			DeleteObject(StatusLightBrush[I]);
	}

	void AddLogEntry(const FStringView& Text, ELogVerbosity::Type Verbosity, const class FName& Category, double Time, uint16 InTextAttribute)
	{
		FScopeLock _(&NewLogEntriesCs);
		NewLogEntries.Add({FString(Text), Verbosity, Category, Time, InTextAttribute});
		SetEvent(DirtyEvent);
	}

	// Light 0 = none, 1 = Red, 2 = Yellow, 3 = Green
	void SetActivity(const TCHAR* Id, const TCHAR* Name, const TCHAR* Status, int32 Light, int32 SortValue, bool bAlignLeft = true)
	{
		if (!*Name)
			return;

		FScopeLock _(&ActivityModificationsCs);

		bool IsNew = false;
		int Index;
		{
			if (int* IndexPtr = IdToActivityIndex.Find(Id))
			{
				Index = *IndexPtr;
			}
			else
			{
				IsNew = true;
				Index = IdToActivityIndex.Num();
				IdToActivityIndex.Add(Id, Index);
			}
		}

		ActivityModification& Modification = ActivityModifications.FindOrAdd(Index);
		if (IsNew)
		{
			Modification.Name = Name;
			Modification.SortValue = SortValue;
			Modification.bAlignLeft = bAlignLeft;
		}
		Modification.Status = Status;
		Modification.Light = Light;
		SetEvent(DirtyEvent);
	}

	void RemoveStatus(const TCHAR* Id)
	{
		FScopeLock _(&ActivityModificationsCs);
		if (int* IndexPtr = IdToActivityIndex.Find(Id))
		{
			ActivityModifications.FindOrAdd(*IndexPtr).bRemove = true;
			SetEvent(DirtyEvent);
		}
	}

	void Start()
	{
		Thread = CreateThread(NULL, 0, StaticThreadProc, this, 0, NULL);
	}

	static DWORD WINAPI StaticThreadProc(LPVOID lpParameter)
	{
		FWindowsPlatformProcess::SetThreadName(TEXT("LogConsoleHwnd"));
		return ((FConsoleWindow*)lpParameter)->ThreadProc();
	}

	uint32 ThreadProc()
	{
		HINSTANCE HInstance = (HINSTANCE)GetModuleHandle(NULL);

		CreateColors();

		Icon = LoadIcon(HInstance, MAKEINTRESOURCE(FWindowsPlatformApplicationMisc::GetAppIcon()));

		WNDCLASSEX WndClassEx;
		ZeroMemory(&WndClassEx, sizeof(WndClassEx));
		WndClassEx.cbSize = sizeof(WndClassEx);
		WndClassEx.style = CS_HREDRAW | CS_VREDRAW;
		WndClassEx.lpfnWndProc = &StaticMainWinProc;
		WndClassEx.hIcon = Icon;
		WndClassEx.hCursor = LoadCursor(NULL, IDC_ARROW);
		WndClassEx.hInstance = HInstance;
		WndClassEx.hbrBackground = NULL;
		WndClassEx.lpszClassName = TEXT("FConsoleWindow");
		ATOM WndClassAtom = RegisterClassEx(&WndClassEx);

		WndClassEx.lpfnWndProc = &StaticScrollBarWinProc;
		WndClassEx.lpszClassName = TEXT("FScrollBar");
		ATOM ScrollBarClassAtom = RegisterClassEx(&WndClassEx);

		NONCLIENTMETRICS NonClientMetrics;
		NonClientMetrics.cbSize = sizeof(NonClientMetrics);
		SystemParametersInfo(SPI_GETNONCLIENTMETRICS, sizeof(NonClientMetrics), &NonClientMetrics, 0);
		Font = (HFONT)CreateFontIndirect(&NonClientMetrics.lfMessageFont);

		const TCHAR* FontName = TEXT("Cascadia Mono"); //TEXT("Courier New"); // TEXT("Consolas");
		int32 FontHeight = -MulDiv(8, GetDeviceCaps(GetDC(NULL), LOGPIXELSY), 72);
		LogFont = (HFONT)CreateFontW(FontHeight, 0, 0, 0, FW_NORMAL, false, false, false, ANSI_CHARSET, OUT_DEFAULT_PRECIS, CLIP_DEFAULT_PRECIS, ANTIALIASED_QUALITY, FIXED_PITCH | FF_MODERN, FontName);

		DWORD WindowStyle = WS_CAPTION | WS_SYSMENU | WS_THICKFRAME | WS_MAXIMIZEBOX | WS_MINIMIZEBOX | WS_CLIPCHILDREN;
		const TCHAR* WindowClassName = MAKEINTATOM(WndClassAtom);
		MainHwnd = CreateWindow(WindowClassName, *ConsoleTitle, WindowStyle, ConsolePosX, ConsolePosY, ConsoleWidth, ConsoleHeight, NULL, NULL, HInstance, this);
		SetWindowLongPtr(MainHwnd, GWLP_USERDATA, (LONG_PTR)this);
		RECT MainRect;
		GetWindowRect(MainHwnd, &MainRect);
		ConsolePosX = MainRect.left;
		ConsolePosY = MainRect.top;
		ConsoleWidth = MainRect.right - ConsolePosX;
		ConsoleHeight = MainRect.bottom - ConsolePosY;

		SIZE FontSize;
		HDC Hdc = GetDC(MainHwnd);
		SelectObject(Hdc, LogFont);
		GetTextExtentPoint32(Hdc, TEXT("A"), 1, &FontSize);
		LogFontWidth = FontSize.cx;
		LogFontHeight = FontSize.cy;
		ReleaseDC(MainHwnd, Hdc);

		IncludeFilterHwnd = CreateWindow2(WC_EDIT, NULL, ES_AUTOHSCROLL, Font, 0, 0, 0, 0, ID_LOG_INCLUDE_FILTER);
		ExcludeFilterHwnd = CreateWindow2(WC_EDIT, NULL, ES_AUTOHSCROLL, Font, 0, 0, 0, 0, ID_LOG_EXCLUDE_FILTER);
		ClearLogButtonHwnd = CreateWindow2(WC_BUTTON, TEXT("Clear Log"), 0, Font, 0, 0, 0, 0, ID_CLEARLOGBUTTON);
		CheckpointButtonHwnd = CreateWindow2(WC_BUTTON, TEXT("Log CHECKPOINT0"), 0, Font, 0, 0, 0, 0, ID_ADDCHECKPOINTBUTTON);

		DWORD LogHwndStyle = LBS_NOINTEGRALHEIGHT | LBS_EXTENDEDSEL | LBS_WANTKEYBOARDINPUT | LBS_NOTIFY | LBS_HASSTRINGS | LBS_OWNERDRAWFIXED;
		LogHwnd = CreateWindow2(WC_LISTBOX, NULL, LogHwndStyle, LogFont, 0, 0, 1, 1, ID_LOG);
		SetWindowLongPtr(LogHwnd, GWLP_USERDATA, (LONG_PTR)this);
		LogHwndWndProcPtr = (WNDPROC)SetWindowLongPtr(LogHwnd, GWLP_WNDPROC, (LONG_PTR)StaticLogHwndWndProc);

		const TCHAR* ScrollBarClassName = MAKEINTATOM(ScrollBarClassAtom);
		LogScrollHwnd = CreateWindow2(ScrollBarClassName, NULL, SBS_VERT, 0);
		SetWindowLongPtr(LogScrollHwnd, GWLP_USERDATA, (LONG_PTR)this);

		CommandHwnd = CreateWindow2(WC_EDIT, NULL, ES_AUTOHSCROLL, Font, 0, 0, 1, 1, ID_COMMAND);
		SetWindowLongPtr(CommandHwnd, GWLP_USERDATA, (LONG_PTR)this);
		CommandHwndWndProcPtr = (WNDPROC)SetWindowLongPtr(CommandHwnd, GWLP_WNDPROC, (LONG_PTR)StaticCommandHwndWndProc);

		RunCommandHwnd = CreateWindow2(WC_BUTTON, TEXT("RunCommand"), 0, Font, 0, 0, 1, 1, ID_COMMANDBUTTON);

		SetFocus(CommandHwnd);

		RECT Rect;
		GetClientRect(MainHwnd, &Rect);
		UpdateSize(FromScale(Rect.right), FromScale(Rect.bottom), false);

		UpdateTime(0);

		FTrackedActivity::RegisterEventListener([&](FTrackedActivity::EEvent Event, const FTrackedActivity::FInfo& Info)
			{
				TCHAR Id[256];
				const TCHAR* Str = Info.Name;
				FCString::Sprintf(Id, TEXT("%u%s"), Info.Id, Str);

				if (Event != FTrackedActivity::EEvent::Removed)
					SetActivity(Id, Str, Info.Status, (int32)Info.Light, Info.SortValue, Info.Type == FTrackedActivity::EType::Activity);
				else
					RemoveStatus(Id);
			});

		FTrackedActivity::TraverseActivities([&](const FTrackedActivity::FInfo& Info)
			{
				TCHAR Id[256];
				const TCHAR* Str = Info.Name;
				FCString::Sprintf(Id, TEXT("%u%s"), Info.Id, Str);
				SetActivity(Id, Str, Info.Status, (int32)Info.Light, Info.SortValue, Info.Type == FTrackedActivity::EType::Activity);
			});

		UpdateWindow(MainHwnd);

		SetForegroundWindow(MainHwnd);

		TipHwnd = CreateWindowEx(WS_EX_TOOLWINDOW | WS_EX_TOPMOST | WS_EX_NOACTIVATE, WC_STATIC, TEXT(""), SS_OWNERDRAW | WS_VISIBLE | WS_POPUP | WS_DISABLED | WS_CLIPCHILDREN | WS_CLIPSIBLINGS, 1, 1, 1, 1, MainHwnd, NULL, HInstance, NULL);
		SendMessageW(TipHwnd, WM_SETFONT, (WPARAM)Font, 0);

		FDateTime startTime(FDateTime::Now());
		uint64 lastSeconds = 0;
		bool bLoop = true;
		bool bWasVisible = false;
		while (bLoop)
		{
			if (bWasVisible != bIsVisible)
			{
				if (bIsVisible)
				{
					SetWindowPos(MainHwnd, 0, ConsolePosX, ConsolePosY, ConsoleWidth, ConsoleHeight, SWP_SHOWWINDOW|SWP_NOZORDER|SWP_NOACTIVATE|SWP_NOOWNERZORDER|SWP_FRAMECHANGED);
				}
				else
				{
					ShowWindow(MainHwnd, SW_HIDE);
				}
				bWasVisible = bIsVisible;

				if (bIsVisible)
				{
					UpdateIncludeFilter(*ConsoleIncludeFilterStr);
					UpdateExcludeFilter(*ConsoleExcludeFilterStr);
				}
			}

			uint64 seconds = uint64(FPlatformTime::Seconds() - GStartTime);
			if (lastSeconds != seconds)
			{
				lastSeconds = seconds;
				UpdateTime(seconds);
			}

			DWORD Timeout = 200;
			DWORD WaitResult = MsgWaitForMultipleObjects(1, &DirtyEvent, false, Timeout, QS_ALLINPUT);

			{
				typedef UINT(WINAPI* LPGetDpiForWindow)(HWND Hwnd);
				static LPGetDpiForWindow GetDpiForWindow = (LPGetDpiForWindow)(void*)GetProcAddress(GetModuleHandle(L"user32.dll"), "GetDpiForWindow");
				if (GetDpiForWindow != nullptr)
				{
					uint32 dpi = GetDpiForWindow(MainHwnd);
					static uint32 lastDpi = dpi;
					if (lastDpi != dpi)
					{
						DpiScale = double(dpi) / 96;
						ShowWindow(MainHwnd, SW_HIDE);
						lastDpi = dpi;
						bWasVisible = false;

						ConsolePosX = ToScale(ConsolePosX);
						ConsolePosY = ToScale(ConsolePosY);
						ConsoleWidth = ToScale(ConsoleWidth);
						ConsoleHeight = ToScale(ConsoleHeight);

						SystemParametersInfo(SPI_GETNONCLIENTMETRICS, sizeof(NonClientMetrics), &NonClientMetrics, 0);
						Font = (HFONT)CreateFontIndirect(&NonClientMetrics.lfMessageFont);

						FontHeight = -MulDiv(8, dpi, 72);
						LogFont = (HFONT)CreateFontW(FontHeight, 0, 0, 0, FW_NORMAL, false, false, false, ANSI_CHARSET, OUT_DEFAULT_PRECIS, CLIP_DEFAULT_PRECIS, ANTIALIASED_QUALITY, FIXED_PITCH | FF_MODERN, FontName);

						Hdc = GetDC(MainHwnd);
						SelectObject(Hdc, LogFont);
						GetTextExtentPoint32(Hdc, TEXT("A"), 1, &FontSize);
						LogFontWidth = FontSize.cx;
						LogFontHeight = FontSize.cy;
						ReleaseDC(MainHwnd, Hdc);

						auto UpdateFont = [](HWND hwnd, HFONT font) { SendMessageW(hwnd, WM_SETFONT, (WPARAM)font, 0); };
						
						UpdateFont(IncludeFilterHwnd, Font);
						UpdateFont(ExcludeFilterHwnd, Font);
						UpdateFont(ClearLogButtonHwnd, Font);
						UpdateFont(CheckpointButtonHwnd, Font);
						UpdateFont(CommandHwnd, Font);
						UpdateFont(RunCommandHwnd, Font);

						for (Activity& A : Activities)
						{
							if (A.NameHwnd)
								UpdateFont(A.NameHwnd, Font);
							if (A.StatusHwnd)
								UpdateFont(A.StatusHwnd, Font);
						}

						UpdateFont(LogHwnd, LogFont);
						SendMessageW(LogHwnd, LB_SETITEMHEIGHT, 0, MAKELPARAM(LogFontHeight, 0));

						WindowBorderPen = CreatePen(PS_INSIDEFRAME, int(ceil(DpiScale)), BorderColor);
					}
				}
			}

			if (WaitResult == WAIT_TIMEOUT)
				continue;

			HandleActivityModifications();

			HandleNewLogEntries();

			MSG Msg;
			while (PeekMessage(&Msg, NULL, 0, 0, PM_REMOVE))
			{
				if (!IsDialogMessage(MainHwnd, &Msg))
				{
					TranslateMessage(&Msg);
					DispatchMessage(&Msg);
				}
				if (Msg.message == WM_QUIT)
				{
					DestroyWindow(MainHwnd);
					DeleteObject(LogFont);
					DeleteObject(Font);
					UnregisterClass(WindowClassName, HInstance);
					MainHwnd = 0;
					bLoop = false;
					break;
				}
			}
		}

		DestroyColors();

		if (!IsEngineExitRequested())
		{
			Owner->SaveToINI();

			FWindowsPlatformMisc::CallGracefulTerminationHandler();
		}
		else if (!bExiting)
		{
			// This means that we've pressed close after the process has already been exited. It is likely the shutdown process is hung and we should just terminate the process
			Sleep(500);
			TerminateProcess(GetCurrentProcess(), -1);
		}

		return 0;
	}

	void CreateColors()
	{
		BorderColor = RGB(0,0,0);
		COLORREF LogBackgroundColor = RGB(0, 0, 0);
		switch (ConsoleColor)
		{
		default:
		case EConsoleColor::DarkGray:
			BackgroundColor = RGB(32, 32, 32);
			TextColor = RGB(220, 220, 220);
			EditBackgroundColor = RGB(64, 64, 64);
			StatusBackgroundColor[0] = RGB(42, 42, 42);
			StatusBackgroundColor[1] = RGB(49, 49, 49);
			ButtonColor = RGB(48, 48, 48);
			ButtonHighlightColor = RGB(100, 100, 100);
			ButtonPressedColor = RGB(130, 130, 100);
			ThumbColor = RGB(77, 77, 77);
			ScrollBackgroundColor = RGB(48, 48, 48);
			BorderColor = RGB(90, 90, 90);
			break;

		case EConsoleColor::Gray:
			BackgroundColor = GetSysColor(COLOR_3DFACE);
			TextColor = RGB(0, 0, 0);
			EditBackgroundColor = BackgroundColor;
			StatusBackgroundColor[0] = GetSysColor(COLOR_3DLIGHT);
			StatusBackgroundColor[1] = GetSysColor(COLOR_BTNHIGHLIGHT);
			ButtonColor = GetSysColor(COLOR_3DLIGHT);
			ButtonHighlightColor = GetSysColor(COLOR_GRADIENTINACTIVECAPTION);
			ButtonPressedColor = GetSysColor(COLOR_GRADIENTACTIVECAPTION);
			ThumbColor = GetSysColor(COLOR_BTNSHADOW);
			ScrollBackgroundColor = ButtonColor;
			BorderColor = GetSysColor(COLOR_WINDOWFRAME);
			break;

		case EConsoleColor::Red:
			BackgroundColor = RGB(200, 50, 50);
			TextColor = RGB(230, 230, 230);
			EditBackgroundColor = RGB(160, 50, 50);
			StatusBackgroundColor[0] = RGB(190, 40, 40);
			StatusBackgroundColor[1] = RGB(210, 60, 60);
			ButtonColor = RGB(160, 50, 50);
			ButtonHighlightColor = RGB(190, 50, 50);
			ButtonPressedColor = RGB(180, 50, 50);
			ThumbColor = RGB(120, 50, 50);
			ScrollBackgroundColor = EditBackgroundColor;
			BorderColor = RGB(100, 40, 40);
			break;

		case EConsoleColor::LightBlue:
			BackgroundColor = RGB(115, 211, 244);
			LogBackgroundColor = RGB(0, 15, 20);
			TextColor = RGB(0, 0, 0);
			EditBackgroundColor = RGB(100, 190, 220);
			StatusBackgroundColor[0] = RGB(105, 201, 234);
			StatusBackgroundColor[1] = RGB(125, 221, 254);
			ButtonColor = RGB(100, 190, 220);
			ButtonHighlightColor = RGB(115, 211, 244);
			ButtonPressedColor = RGB(125, 221, 255);
			ThumbColor = RGB(60, 150, 180);
			ScrollBackgroundColor = RGB(100, 190, 220);
			BorderColor = RGB(30, 30, 220);
			break;

		case EConsoleColor::DarkBlue:
			BackgroundColor = RGB(10, 10, 150);
			TextColor = RGB(220, 220, 220);
			EditBackgroundColor = RGB(10, 10, 100);
			StatusBackgroundColor[0] = RGB(20, 20, 190);
			StatusBackgroundColor[1] = RGB(20, 20, 220);
			ButtonColor = RGB(10, 10, 100);
			ButtonHighlightColor = RGB(10, 10, 190);
			ButtonPressedColor = RGB(10, 10, 150);
			ThumbColor = RGB(10, 10, 220);
			ScrollBackgroundColor = RGB(10, 10, 120);
			BorderColor = RGB(30, 30, 220);
			break;

		case EConsoleColor::Load:
		{
			FString Filename = "DebugConsoleColors.ini";
			const TCHAR* Selection = TEXT("Colors");
			FConfigCacheIni Config(EConfigCacheType::Temporary);
			Config.LoadFile(Filename);
			FColor Color;
			if (Config.GetColor(Selection, TEXT("Background"), Color, Filename))
				BackgroundColor = Color.ToPackedARGB();
			if (Config.GetColor(Selection, TEXT("Text"), Color, Filename))
				TextColor = Color.ToPackedARGB();
			if (Config.GetColor(Selection, TEXT("LogBackground"), Color, Filename))
				LogBackgroundColor = Color.ToPackedARGB();
			if (Config.GetColor(Selection, TEXT("EditBackground"), Color, Filename))
				EditBackgroundColor = Color.ToPackedARGB();
			if (Config.GetColor(Selection, TEXT("StatusBackground1"), Color, Filename))
				StatusBackgroundColor[0] = Color.ToPackedARGB();
			if (Config.GetColor(Selection, TEXT("StatusBackground2"), Color, Filename))
				StatusBackgroundColor[1] = Color.ToPackedARGB();
			if (Config.GetColor(Selection, TEXT("EditBackground"), Color, Filename))
				EditBackgroundColor = Color.ToPackedARGB();
			if (Config.GetColor(Selection, TEXT("Button"), Color, Filename))
				ButtonColor = Color.ToPackedARGB();
			if (Config.GetColor(Selection, TEXT("ButtonHighlight"), Color, Filename))
				ButtonHighlightColor = Color.ToPackedARGB();
			if (Config.GetColor(Selection, TEXT("ButtonPressed"), Color, Filename))
				ButtonPressedColor = Color.ToPackedARGB();
			if (Config.GetColor(Selection, TEXT("Thumb"), Color, Filename))
				ThumbColor = Color.ToPackedARGB();
			if (Config.GetColor(Selection, TEXT("ScrollBackground"), Color, Filename))
				ScrollBackgroundColor = Color.ToPackedARGB();
			if (Config.GetColor(Selection, TEXT("Border"), Color, Filename))
				BorderColor = Color.ToPackedARGB();
			break;
		}
		}

		TextBrush = CreateSolidBrush(TextColor);
		BackgroundBrush = CreateSolidBrush(BackgroundColor);
		LogBackgroundBrush = CreateSolidBrush(LogBackgroundColor);
		EditBackgroundBrush = CreateSolidBrush(EditBackgroundColor);
		StatusBackgroundBrush[0] = CreateSolidBrush(StatusBackgroundColor[0]);
		StatusBackgroundBrush[1] = CreateSolidBrush(StatusBackgroundColor[1]);
		ButtonBrush = CreateSolidBrush(ButtonColor);
		ButtonHighlightBrush = CreateSolidBrush(ButtonHighlightColor);
		ButtonPressedBrush = CreateSolidBrush(ButtonPressedColor);
		ThumbBrush = CreateSolidBrush(ThumbColor);
		ScrollBackgroundBrush = CreateSolidBrush(ScrollBackgroundColor);

		NoPen = CreatePen(PS_SOLID, 0, TextColor);
		BorderPen = CreatePen(PS_SOLID, 1, BorderColor);
		WindowBorderPen = CreatePen(PS_INSIDEFRAME, 1, BorderColor);
		TextPen = CreatePen(PS_SOLID, 1, TextColor);
	}

	void DestroyColors()
	{
		DeleteObject(NoPen);
		DeleteObject(BorderPen);
		DeleteObject(WindowBorderPen);
		DeleteObject(TextPen);

		DeleteObject(TextBrush);
		DeleteObject(BackgroundBrush);
		DeleteObject(LogBackgroundBrush);
		DeleteObject(EditBackgroundBrush);
		DeleteObject(StatusBackgroundBrush[0]);
		DeleteObject(StatusBackgroundBrush[1]);
		DeleteObject(ButtonBrush);
		DeleteObject(ButtonHighlightBrush);
		DeleteObject(ThumbBrush);
	}

	HWND CreateWindow2(const TCHAR* ClassName, const TCHAR* Str, DWORD Style, HFONT hFont, int32 X = 0, int32 Y = 0, int32 Width = 1, int32 Height = 1, ENotificationId InNid = (ENotificationId)0, DWORD ExStyle  = 0)
	{
		HWND h = CreateWindowEx(ExStyle , ClassName, Str, Style | WS_VISIBLE | WS_CHILD, X, Y, Width, Height, MainHwnd, (HMENU)InNid, NULL, NULL);
		if (hFont)
			SendMessageW(h, WM_SETFONT, (WPARAM)hFont, 0);
		return h;
	}

	HWND CreateTextHwnd(const TCHAR* Str, HFONT hFont, ENotificationId InNid = (ENotificationId)0, DWORD Style = 0, DWORD ExStyle = 0)
	{
		HWND h = CreateWindow2(WC_STATIC, Str, Style | SS_OWNERDRAW, hFont, 0, 0, 1, 1, InNid, ExStyle );
		SetWindowLong(h, GWLP_USERDATA, -1);
		return h;
	}

	template<class Lambda>
	int32 TraverseActivityPositions(int32 Width, int32 Height, const Lambda& InLamda)
	{
		Width -= 6; // Margins

		TArray<Activity*, TInlineAllocator<32>> LeftActivities;
		TArray<Activity*, TInlineAllocator<32>> RightActivities;
		uint32 ActivityCount = 0;
		for (int32 i = 0, e = Activities.Num(); i != e; ++i)
		{
			Activity& A = Activities[i];
			if (A.Name.IsEmpty())
				continue;
			++ActivityCount;
			if (A.bAlignLeft)
				LeftActivities.Add(&A);
			else
				RightActivities.Add(&A);
		}

		auto SortFunc = [](const Activity* A, const Activity* B)
		{
			if (A->SortValue != B->SortValue)
				return A->SortValue < B->SortValue;
			return uintptr_t(A) < uintptr_t(B);
		};

		Algo::Sort(LeftActivities, SortFunc);
		Algo::Sort(RightActivities, SortFunc);

		int32 LeftCount = LeftActivities.Num();
		int32 RightCount = ActivityCount - LeftCount;

		int32 LeftColumnMinWidth = 600;
		int32 RightColumnWidth = 200;
		int32 LeftColumnCount = 1;
		int32 RightColumnCount = 1;

		int32 RowCount = FMath::Max(LeftCount, RightCount);

		if (RightCount > LeftCount)
		{
			if (Width - LeftColumnMinWidth >= RightColumnWidth * 2)
			{
				RightColumnCount = 2;
				RowCount = FMath::Max(LeftCount, (RightCount+1)/2);
			}
		}


		int32 RowHeight = 18;
		int32 TotalHeight = RowCount * RowHeight + 8;


		int32 StartY = Height - TotalHeight + 2;

		int32 X;
		int32 Y;
		int32 RowIndex;
		int32 ColWidth;
		int32 ColOffset;

		auto IterateActivities = [&](TArray<Activity*, TInlineAllocator<32>>& SortedActivities)
		{
			for (int32 i = 0, e = SortedActivities.Num(); i != e; ++i)
			{
				Activity& A = *SortedActivities[i];
				if (A.Name.IsEmpty())
					continue;

				InLamda(A, X, Y, ColWidth, RowIndex);
				Y += RowHeight;
				++RowIndex;
				if (RowIndex < RowCount)
					continue;
				RowIndex = 0;
				Y = StartY;
				X += ColOffset;
			}
		};

		X = 7;
		Y = StartY;
		RowIndex = 0;
		int TotalLeftWidth = Width - RightColumnCount * RightColumnWidth - X;
		ColWidth = TotalLeftWidth / LeftColumnCount;
		ColOffset = ColWidth - 2;
		IterateActivities(LeftActivities);

		X = Width - RightColumnWidth + 8;
		Y = StartY;
		RowIndex = 0;
		ColWidth = RightColumnWidth - 8;
		ColOffset = -RightColumnWidth;

		IterateActivities(RightActivities);
		return TotalHeight;
	}

	void UpdateSize(int32 ClientWidth, int32 ClientHeight, bool bRedraw)
	{
		if (bRedraw)
			SendMessageW(LogHwnd, WM_SETREDRAW, false, 0);

		int32 Flags = SWP_NOZORDER | SWP_NOACTIVATE | SWP_NOOWNERZORDER;
		if (!bRedraw)
			Flags |= SWP_NOREDRAW;

		ActivitiesTotalHeight = UpdateActivityPositions(ClientWidth, ClientHeight, bRedraw);

		int32 LogTop = 1;
		if (bConsoleExpanded)
			LogTop += 35;

		int32 ButtonWidth = 90;
		int32 CommandFlags = Flags | (bConsoleExpanded ? SWP_SHOWWINDOW : SWP_HIDEWINDOW);
		int32 FilterFlags = 0;

		int32 CommandY = ClientHeight - ActivitiesTotalHeight - 20 - 5;
		int32 LogHeight = CommandY - LogTop - 5;

		if (bConsoleExpanded)
		{
			CommandFlags = Flags | SWP_SHOWWINDOW;
			FilterFlags = Flags | SWP_SHOWWINDOW;
		}
		else
		{
			CommandFlags = SWP_HIDEWINDOW;
			FilterFlags = Flags | SWP_HIDEWINDOW;
			LogHeight += 25;
		}

		int32 Y = 10;
		int32 ButtonStart = ClientWidth - 235;

		SetWindowPosScale(ClearLogButtonHwnd, 0, ButtonStart, Y - 2, 80, 22, FilterFlags);
		SetWindowPosScale(CheckpointButtonHwnd, 0, ButtonStart + 94, Y - 2, 135, 22, FilterFlags);

		int32 WidthForFilters = ButtonStart - 45;
		if (WidthForFilters < 100)
			FilterFlags = Flags | SWP_HIDEWINDOW;
		int32 X = 8;
		int32 WidthForEditBoxes = WidthForFilters / 2;
		SetWindowPosScale(IncludeFilterHwnd, 0, X, Y - 1, WidthForEditBoxes, 20, FilterFlags);
		X += WidthForEditBoxes + 20;
		SetWindowPosScale(ExcludeFilterHwnd, 0, X, Y - 1, WidthForEditBoxes, 20, FilterFlags);

		SetWindowPosScale(LogHwnd, 0, 7, LogTop, ClientWidth - 33, LogHeight, Flags);
		SetWindowPosScale(LogScrollHwnd, 0, ClientWidth - 26, LogTop, 19, LogHeight, Flags);
		SetWindowPosScale(CommandHwnd, 0, 8, CommandY, ClientWidth - ButtonWidth - 20, 20, CommandFlags);
		SetWindowPosScale(RunCommandHwnd, 0, ClientWidth - ButtonWidth - 7, CommandY - 1, ButtonWidth, 22, CommandFlags);

		if (bRedraw)
			SendMessageW(LogHwnd, WM_SETREDRAW, true, 0);
	}

	int32 UpdateActivityPositions(int32 ClientWidth, int32 ClientHeight, bool bRedraw)
	{
		int32 Flags = SWP_NOZORDER | SWP_NOACTIVATE | SWP_NOOWNERZORDER;
		if (!bRedraw)
			Flags |= SWP_NOREDRAW;
		return TraverseActivityPositions(ClientWidth, ClientHeight, [&](Activity& A, int32 X, int32 Y, int32 Width, int32 RowIndex)
			{
				X += 16;
				Width -= 16;
				int32 NameWidth = FMath::Min(90, Width);
				SetWindowPosScale(A.NameHwnd, 0, X, Y, NameWidth, 18, Flags);
				SetWindowLong(A.NameHwnd, GWLP_USERDATA, RowIndex);
				int32 StatusWidth = FMath::Max(Width - NameWidth, 0);
				SetWindowPosScale(A.StatusHwnd, 0, X + NameWidth, Y, StatusWidth, 18, Flags);
				SetWindowLong(A.StatusHwnd, GWLP_USERDATA, RowIndex);
			});
	}

	void UpdateTime(uint64 seconds)
	{
		FTimespan span = FTimespan::FromSeconds((double)seconds);
		FString Res;
		if (int32 days = span.GetDays())
			Res = FString::Printf(TEXT("%i."), FMath::Abs(days));
		Res = FString::Printf(TEXT("%s%02i:%02i:%02i"), *Res, FMath::Abs(span.GetHours()), FMath::Abs(span.GetMinutes()), FMath::Abs(span.GetSeconds()));
		SetActivity(TEXT("Time"), TEXT("Time"), *Res, 0, 1, false);
	}

	struct LogEntry;

	int32 AddEntryToLogHwnd(LogEntry& E, int32 LogVirtualIndex)
	{
		for (const ELogVerbosity::Type Verbosity : IncludeVerbosity)
			if (E.Verbosity != Verbosity)
				return -1;
		for (const ELogVerbosity::Type Verbosity : ExcludeVerbosity)
			if (E.Verbosity == Verbosity)
				return -1;

		TStringBuilder<1024> OutString;
		CreateLogEntryText(OutString, E, true);

		const TCHAR* Str = *OutString;
		for (const FString& I : IncludeFilter)
			if (FCString::Stristr(Str, *I) == 0)
				return -1;
		for (const FString& I : ExcludeFilter)
			if (FCString::Stristr(Str, *I) != 0)
				return -1;
		AddedEntryLogVirtualIndex = LogVirtualIndex;
		const TCHAR* Empty = TEXT("");
		int32 ItemIndex = IntCastChecked<int32>(SendMessageW(LogHwnd, LB_ADDSTRING, 0, (LPARAM)Empty));
		SendMessageW(LogHwnd, LB_SETITEMDATA, ItemIndex, LogVirtualIndex);
		AddedEntryLogVirtualIndex = -1;
		return ItemIndex;
	}

	void RefreshLogHwnd()
	{
		int32 SelectedLogIndex = -1;
		int32 SelectedItemOffsetFromTop = 0;
		if (SendMessageW(LogHwnd, LB_GETSELCOUNT, 0, 0) == 1)
		{
			int32 SelectedItemIndex = -1;
			SendMessageW(LogHwnd, LB_GETSELITEMS, 1, (LPARAM)&SelectedItemIndex);
			SelectedLogIndex = IntCastChecked<int32>(SendMessageW(LogHwnd, LB_GETITEMDATA, SelectedItemIndex, 0) - LogIndexOffset);
			SelectedItemOffsetFromTop = IntCastChecked<int32>(SelectedItemIndex - SendMessageW(LogHwnd, LB_GETTOPINDEX, 0, 0));
		}

		LogIndexOffset = 0;

		SendMessageW(LogHwnd, WM_SETREDRAW, false, 0);
		SendMessageW(LogHwnd, LB_RESETCONTENT, 0, 0);
		SendMessageW(LogHwnd, LB_INITSTORAGE, Log.Num(), Log.Num()*256);
		int32 NewSelectedItemIndex = -1;
		int32 LogIndex = 0;
		for (LogEntry& E : Log)
		{
			int32 ItemIndex = AddEntryToLogHwnd(E, LogIndex);
			if (LogIndex == SelectedLogIndex)
				NewSelectedItemIndex = ItemIndex;
			++LogIndex;
		}

		int32 ScrollPos = -1;
		if (NewSelectedItemIndex != -1)
		{
			ScrollPos = NewSelectedItemIndex - SelectedItemOffsetFromTop;
			SetTopVisible(ScrollPos, false);
			SendMessageW(LogHwnd, LB_SETSEL, true, NewSelectedItemIndex);
		}
		else
			ScrollToBottom();

		RedrawLogScrollbar();

		SendMessageW(LogHwnd, WM_SETREDRAW, true, 0);
		RedrawWindow(LogScrollHwnd, NULL, NULL, RDW_INVALIDATE);
	}

	void ClearSelection()
	{
		TArray<int> SelectedItems;
		GetSelectedItems(SelectedItems);
		for (int SelectedItem : SelectedItems)
			SendMessageW(LogHwnd, LB_SETSEL, false, SelectedItem);;
	}

	void ScrollToBottom()
	{
		int32 ItemCount = (int)SendMessageW(LogHwnd, LB_GETCOUNT, 0, 0);

		// This is here just to force last selection to be at the bottom
		SendMessageW(LogHwnd, LB_SETSEL, true, ItemCount - 1);
		SendMessageW(LogHwnd, LB_SETSEL, false, ItemCount - 1);
		SetTopVisible(ItemCount - 1, true); // Using post to remove weird glitches with smooth scrolling
	}

	void SuspendAddingEntries()
	{
		bSuspendAddingEntries = true;
	}

	void ResumeAddingEntries()
	{
		if (!bSuspendAddingEntries)
			return;
		bSuspendAddingEntries = false;
		SetEvent(DirtyEvent);
	}

	static LRESULT CALLBACK StaticLogHwndWndProc(HWND hWnd, UINT Msg, WPARAM wParam, LPARAM lParam)
	{
		FConsoleWindow* ThisPtr = (FConsoleWindow*)GetWindowLongPtr(hWnd, GWLP_USERDATA);
		return ThisPtr->LogHwndWndProc(hWnd, Msg, wParam, lParam);
	}

	static LRESULT CALLBACK StaticCommandHwndWndProc(HWND hWnd, UINT Msg, WPARAM wParam, LPARAM lParam)
	{
		FConsoleWindow* ThisPtr = (FConsoleWindow*)GetWindowLongPtr(hWnd, GWLP_USERDATA);
		return ThisPtr->CommandHwndWndProc(hWnd, Msg, wParam, lParam);
	}

	static LRESULT CALLBACK StaticMainWinProc(HWND hWnd, UINT Msg, WPARAM wParam, LPARAM lParam)
	{
		FConsoleWindow* ThisPtr = (FConsoleWindow*)GetWindowLongPtr(hWnd, GWLP_USERDATA);
		if (!ThisPtr && Msg == WM_CREATE)
		{
			#pragma comment( lib, "UxTheme")
			SetWindowTheme(hWnd, L"", L""); // Needed to disable rounded edges on window
			ThisPtr = (FConsoleWindow*)lParam;
			SetWindowLongPtr(hWnd, GWLP_USERDATA, lParam);
		}
		if (ThisPtr && hWnd == ThisPtr->MainHwnd)
			return ThisPtr->MainWinProc(hWnd, Msg, wParam, lParam);
		else
			return DefWindowProc(hWnd, Msg, wParam, lParam);
	}

	static LRESULT CALLBACK StaticScrollBarWinProc(HWND hWnd, UINT Msg, WPARAM wParam, LPARAM lParam)
	{
		if (FConsoleWindow* ThisPtr = (FConsoleWindow*)GetWindowLongPtr(hWnd, GWLP_USERDATA))
			return ThisPtr->ScrollBarWinProc(hWnd, Msg, wParam, lParam);
		return DefWindowProc(hWnd, Msg, wParam, lParam);
	}

	void GetThumbPosAndHeight(int32& OutPos, int32& OutHeight, const RECT& ClientRect)
	{
		int32 ButtonHeight = ClientRect.right + ToScale(1);

		int32 TotalCount = (int)SendMessageW(LogHwnd, LB_GETCOUNT, 0, 0);
		int32 PageSize = ClientRect.bottom / LogFontHeight;

		int32 TotalScrollCount = TotalCount - PageSize;
		int32 ScrollHeight = ClientRect.bottom - ButtonHeight * 2;

		OutHeight = 0;

		if (TotalScrollCount <= 0)
			return;

		int32 TopVisible = IntCastChecked<int32>(SendMessageW(LogHwnd, LB_GETTOPINDEX, 0, 0));

		int32 ThumbHeight = FMath::Max((PageSize * ScrollHeight) / TotalCount, int32(ButtonHeight));

		int32 MoveSpace = ScrollHeight - ThumbHeight;
		if (MoveSpace <= 0)
			return;

		int32 ThumbPos = ButtonHeight + FMath::CeilToInt(float(TopVisible * MoveSpace) / (float)TotalScrollCount);

		OutPos = ThumbPos;
		OutHeight = ThumbHeight;
	}

	int32 GetTopVisible(int32 ThumbPos, const RECT& ClientRect)
	{
		int32 ButtonHeight = ClientRect.right + ToScale(1);

		int32 TotalCount = (int)SendMessageW(LogHwnd, LB_GETCOUNT, 0, 0);
		int32 PageSize = ClientRect.bottom / LogFontHeight;

		int32 TotalScrollCount = TotalCount - PageSize;
		int32 ScrollHeight = ClientRect.bottom - ButtonHeight * 2;

		int32 ThumbHeight = FMath::Max((PageSize * ScrollHeight) / TotalCount, int32(ButtonHeight));

		int32 MoveSpace = ScrollHeight - ThumbHeight;

		return FMath::FloorToInt((float)(ThumbPos - ButtonHeight) * (float)TotalScrollCount / (float)MoveSpace);
	}

	void SetTopVisible(int32 TopVisible, bool ShouldPost)
	{
		SetWindowPos(TipHwnd, MainHwnd, 1, 1, 1, 1, SWP_NOACTIVATE|SWP_NOSENDCHANGING|SWP_NOZORDER|SWP_NOOWNERZORDER);
		if (ShouldPost)
			PostMessageW(LogHwnd, LB_SETTOPINDEX, TopVisible, 0);
		else
			SendMessageW(LogHwnd, LB_SETTOPINDEX, TopVisible, 0);
	}

	void MoveTopVisible(const RECT& Rect, int32 Offset)
	{
		bAutoScrollLog = false;
		int32 TopVisible = IntCastChecked<int32>(SendMessageW(LogHwnd, LB_GETTOPINDEX, 0, 0));
		int32 PageSize = Rect.bottom / LogFontHeight;
		TopVisible = FMath::Max(0, TopVisible + Offset);
		SetTopVisible(TopVisible, false);
		RedrawWindow(LogScrollHwnd, NULL, NULL, RDW_INVALIDATE);
	}

	int32 HandleMouseWheel(WPARAM wParam)
	{
		RECT Rect;
		GetClientRect(LogHwnd, &Rect);
		MoveTopVisible(Rect, -GET_WHEEL_DELTA_WPARAM(wParam) * 3 / WHEEL_DELTA);
		return 0;
	}

	void RedrawLogScrollbar()
	{
		RedrawWindow(LogScrollHwnd, NULL, NULL, RDW_INVALIDATE);
	}

	LRESULT ScrollBarWinProc(HWND hWnd, UINT Msg, WPARAM wParam, LPARAM lParam)
	{
		switch (Msg)
		{
		case WM_ERASEBKGND:
		{
			HDC hdc = (HDC)wParam;
			RECT Rect;
			GetClientRect(hWnd, &Rect);
			int32 ButtonHeight = Rect.right + ToScale(1);

			RECT ScrollRect{ 0, ButtonHeight, Rect.right, Rect.bottom - ButtonHeight };
			FillRect(hdc, ScrollRect, ScrollBackgroundBrush);

			RECT ButtonRect{ 0, 0, Rect.right, ButtonHeight };
			FillRect(hdc, ButtonRect, ButtonBrush);

			RECT ButtonRect2{ 0, Rect.bottom - ButtonHeight, Rect.right, Rect.bottom };
			FillRect(hdc, ButtonRect2, ButtonBrush);

			int One = ToScale(1);
			int Two = ToScale(2);
			int Three = ToScale(3);

			SelectObject(hdc, NoPen);
			SelectObject(hdc, TextBrush);
			int32 X = Rect.right / 2;
			int32 Y = Rect.right / 2;
			POINT vertices[] = { {X - Three, Y + One}, {X, Y - Two}, {X + Three, Y + One} };
			Polygon(hdc, vertices, 3);

			Y = Rect.bottom - Y;
			POINT vertices2[] = { {X - Three, Y - Two}, {X, Y + One}, {X + Three, Y - Two} };
			Polygon(hdc, vertices2, 3);

			return 1;
		}
		case WM_PAINT:
		{
			RECT Rect;
			GetClientRect(hWnd, &Rect);
			int32 ButtonHeight = Rect.right + ToScale(1);

			PAINTSTRUCT ps;
			BeginPaint(hWnd, &ps);

			RECT ScrollRect{ 0, ButtonHeight, Rect.right, Rect.bottom - ButtonHeight };
			FillRect(ps.hdc, ScrollRect, ScrollBackgroundBrush);

			int ThumbPos;
			int ThumbHeight;
			GetThumbPosAndHeight(ThumbPos, ThumbHeight, Rect);

			if (ThumbHeight > 0)
			{
				RECT ThumbRect{ 1, ThumbPos, Rect.right-1, ThumbPos + ThumbHeight };
				FillRect(ps.hdc, ThumbRect, ThumbBrush);
			}

			EndPaint(hWnd, &ps);
			break;
		}
		case WM_LBUTTONDOWN:
		{
			RECT Rect;
			GetClientRect(hWnd, &Rect);
			int32 Y = HIWORD(lParam);

			int32 ButtonHeight = Rect.right + ToScale(1);
			if (Y <= ButtonHeight)
			{
				MoveTopVisible(Rect, -1);
				break;
			}
			if (Y > Rect.bottom - ButtonHeight)
			{
				MoveTopVisible(Rect, 1);
				break;
			}

			int ThumbPos;
			int ThumbHeight;
			GetThumbPosAndHeight(ThumbPos, ThumbHeight, Rect);
			if (ThumbHeight > 0 && Y > ThumbPos && Y < ThumbPos + ThumbHeight)
			{
				LogScrollGrabPos = Y - ThumbPos;
				SetCapture(hWnd);
				SuspendAddingEntries();
				break;
			}
			if (Y <= ThumbPos)
			{
				int32 PageSize = Rect.bottom / LogFontHeight;
				MoveTopVisible(Rect, -PageSize + 1);
				break;
			}
			if (Y > ThumbPos + ThumbHeight)
			{
				int32 PageSize = Rect.bottom / LogFontHeight;
				MoveTopVisible(Rect, PageSize - 1);
				break;
			}

			break;
		}
		case WM_LBUTTONUP:
			ReleaseCapture();
			ResumeAddingEntries();
			LogScrollGrabPos = -1;
			break;

		case WM_MOUSEMOVE:
			if (LogScrollGrabPos != -1)
			{
				RECT Rect;
				GetClientRect(hWnd, &Rect);

				int32 Y = HIWORD(lParam);
				int32 ThumbPos = Y - LogScrollGrabPos;
				int32 TopVisible = GetTopVisible(ThumbPos, Rect);
				SetTopVisible(TopVisible, false);
				RedrawWindow(hWnd, NULL, NULL, RDW_INVALIDATE);
			}
			break;

		case WM_MOUSEWHEEL:
			return HandleMouseWheel(wParam);
		}
		return DefWindowProc(hWnd, Msg, wParam, lParam);
	}

	LRESULT HitTestNCA(HWND hWnd, LPARAM lParam)
	{
		RECT WindowRect;
		GetWindowRect(hWnd, &WindowRect);
		WindowRect = FromScale(WindowRect);
		int MouseX = FromScale((int)LOWORD(lParam));
		int MouseY = FromScale((int)HIWORD(lParam));

		RECT FrameRect = { 0 };
		AdjustWindowRectEx(&FrameRect, WS_OVERLAPPEDWINDOW & ~WS_CAPTION, false, NULL);
		int32 Row = 1;
		int32 Col = 1;
		bool bOnResizeBorder = false;

		int BorderWidth = 2;

		if (MouseY >= WindowRect.top && MouseY < WindowRect.top + 27)
		{
			bOnResizeBorder = (MouseY < (WindowRect.top - FrameRect.top));
			Row = 0;
		}
		else if (MouseY < WindowRect.bottom && MouseY >= WindowRect.bottom - BorderWidth)
			Row = 2;

		if (MouseX >= WindowRect.left && MouseX < WindowRect.left + BorderWidth)
			Col = 0;
		else if (MouseX < WindowRect.right && MouseX >= WindowRect.right - BorderWidth)
			Col = 2;

		LRESULT hitTests[3][3] =
		{
			{ HTTOPLEFT,    bOnResizeBorder ? HTTOP : HTCAPTION,    HTTOPRIGHT },
			{ HTLEFT,       HTCLIENT,     HTRIGHT },
			{ HTBOTTOMLEFT, HTBOTTOM, HTBOTTOMRIGHT },
		};

		return hitTests[Row][Col];
	}

	int32 HitTestNCB(int32 X, int32 Y)
	{
		RECT WindowRect;
		GetWindowRect(MainHwnd, &WindowRect);
		WindowRect = FromScale(WindowRect);
		X = FromScale(X);
		Y = FromScale(Y);

		if (Y < WindowRect.top + 2 || Y > WindowRect.top + 27 || X > WindowRect.right - 2)
			return HTNOWHERE;
		int32 ButtonWidth = 27;
		if (X <= WindowRect.right - ButtonWidth*4)
			return HTCAPTION;
		int32 ButtonIndex = (WindowRect.right - X) / ButtonWidth;
		int32 Buttons[] = { HTCLOSE, HTMAXBUTTON, HTMINBUTTON, HTEXPAND };
		return Buttons[ButtonIndex];
	}

	int32 HitTestNCB(LPARAM lParam)
	{
		return HitTestNCB((int)(short)LOWORD(lParam), (int)(short)HIWORD(lParam));
	}

	void RedrawNC()
	{
		RedrawWindow(MainHwnd, NULL, NULL, RDW_INVALIDATE | RDW_FRAME);
	}

	void CreateLogEntryText(TStringBuilderBase<TCHAR>& OutString, const LogEntry& E, bool ForFilter)
	{
		TStringBuilder<128> CategoryBuilder;
		E.Category.AppendString(CategoryBuilder);


		if (ConsoleShowDateTime == 1)
		{
			FTimespan Ts = FTimespan::FromSeconds(E.Time);
			OutString.Appendf(TEXT("%02i:"), Ts.GetHours() + Ts.GetDays()*24);
			OutString.Appendf(TEXT("%02i:"), Ts.GetMinutes());
			OutString.Appendf(TEXT("%02i."), Ts.GetSeconds());
			OutString.Appendf(TEXT("%03i  "), Ts.GetFractionMilli());
			//OutString.Appendf(TEXT("[%3llu] "), GFrameCounter % 1000);
		}
		else if (ConsoleShowDateTime == 2)
		{
			(StartDateTime + FTimespan::FromSeconds(E.Time - StartTime)).ToString(TEXT("[%Y.%m.%d-%H.%M.%S:%s] "), OutString);
			//OutString.Appendf(TEXT("[%3llu] "), GFrameCounter % 1000);
		}

		if (ForFilter)
		{
			OutString << '[' << CategoryBuilder << TEXT("][") << ToString(E.Verbosity) << ']';
		}

		const TCHAR* Category = *CategoryBuilder;
		if (FCString::Strstr(Category, TEXT("Log")) == Category)
			Category += 3;
		const TCHAR CategorySpace[] = TEXT("                 ");
		if (FCString::Strlen(Category) >= UE_ARRAY_COUNT(CategorySpace))
			OutString.Append(Category, UE_ARRAY_COUNT(CategorySpace) - 1);
		else
			OutString.Append(Category).Append(CategorySpace + FCString::Strlen(Category));
		OutString.Append(": ");

		if (bConsoleShowVerbosity)
		{
			const TCHAR* VerbosityStr = ToString(E.Verbosity);
			const TCHAR VerbositySpace[] = TEXT("           ");
			if (FCString::Strlen(VerbosityStr) >= UE_ARRAY_COUNT(VerbositySpace))
				OutString.Append(VerbosityStr, UE_ARRAY_COUNT(VerbositySpace) - 1);
			else
				OutString.Append(VerbosityStr).Append(VerbositySpace + FCString::Strlen(VerbosityStr));
			OutString.Append(": ");
		}

		for (TCHAR C : E.String)
		{
			if (C == '\t')
				OutString.Append(TEXT("  "));
			else
				OutString.AppendChar(C);
		}
	}

	void EnableAutoScroll()
	{
		bAutoScrollLog = true;
		ClearSelection();
		ScrollToBottom();
		RedrawWindow(LogHwnd, NULL, NULL, RDW_ERASE | RDW_INVALIDATE);
		RedrawLogScrollbar();
	}

	void ClearLog()
	{
		LogIndexOffset = 0;
		Log.Reset();
		RefreshLogHwnd();
		CheckpointIndex = 0;
		SetDlgItemText(MainHwnd, ID_ADDCHECKPOINTBUTTON, TEXT("Log CHECKPOINT0"));
	}

	int32 GetSelectedItems(TArray<int>& OutSelectedItems)
	{
		int32 SelectionCount = IntCastChecked<int32>(SendMessageW(LogHwnd, LB_GETSELCOUNT, 0, 0));
		OutSelectedItems.SetNum(SelectionCount);
		SendMessageW(LogHwnd, LB_GETSELITEMS, SelectionCount, (LPARAM)OutSelectedItems.GetData());
		return SelectionCount;
	}

	void CopySelectionToClipboard()
	{
		TArray<int> SelectedItems;
		int32 SelectionCount = GetSelectedItems(SelectedItems);
		if (!SelectionCount)
			return;
		TArray<TCHAR> Buffer;
		TStringBuilder<512> StringBuilder;
		for (int32 I = 0; I != SelectionCount; ++I)
		{
			int32 Index = SelectedItems[I];
			int32 LogIndex = IntCastChecked<int32>(SendMessageW(LogHwnd, LB_GETITEMDATA, Index, 0) - LogIndexOffset);
			StringBuilder.Append(Log[LogIndex].String);
			StringBuilder.Append(TEXT("\r\n"));
		}

		FWindowsPlatformApplicationMisc::ClipboardCopy(*StringBuilder);
	}

	void UpdateIncludeFilter(const TCHAR* Str)
	{
		ConsoleIncludeFilterStr = Str;
		SetDlgItemText(MainHwnd, ID_LOG_INCLUDE_FILTER, *ConsoleIncludeFilterStr);
		ConsoleIncludeFilterStr.ParseIntoArray(IncludeFilter, TEXT(" "));
	}

	void UpdateExcludeFilter(const TCHAR* Str)
	{
		ConsoleExcludeFilterStr = Str;
		SetDlgItemText(MainHwnd, ID_LOG_EXCLUDE_FILTER, *ConsoleExcludeFilterStr);
		ConsoleExcludeFilterStr.ParseIntoArray(ExcludeFilter, TEXT(" "));
	}

	LRESULT LogHwndWndProc(HWND hWnd, UINT Msg, WPARAM wParam, LPARAM lParam)
	{
		switch (Msg)
		{
		case WM_MOUSEWHEEL:
			return HandleMouseWheel(wParam);

		case WM_MOUSEMOVE:
		{
			POINT Pos{ ((int)(short)LOWORD(lParam)), ((int)(short)HIWORD(lParam)) };
			if (Pos.x != TipPosition.x || Pos.y != TipPosition.y)
			{
				SetWindowPos(TipHwnd, MainHwnd, 1, 1, 1, 1, SWP_NOACTIVATE | SWP_NOSENDCHANGING | SWP_NOZORDER | SWP_NOOWNERZORDER);
				TRACKMOUSEEVENT tme;
				tme.cbSize = sizeof(tme);
				tme.dwFlags = TME_HOVER;
				tme.hwndTrack = hWnd;
				tme.dwHoverTime = HOVER_DEFAULT;
				TrackMouseEvent(&tme);
			}
			break;
		}

		case WM_LBUTTONDOWN:
			SuspendAddingEntries();
			break;

		case WM_LBUTTONUP:
			ResumeAddingEntries();
			break;

		case WM_RBUTTONDOWN:
		{
			bAutoScrollLog = false;
			SuspendAddingEntries();
			RightClickedItem = IntCastChecked<int32>(SendMessageW(hWnd, LB_ITEMFROMPOINT, 0, lParam));

			bool AlreadySelected = false;
			TArray<int> SelectedItems;
			GetSelectedItems(SelectedItems);
			for (int SelectedItem : SelectedItems)
				AlreadySelected |= RightClickedItem == SelectedItem;
			if (AlreadySelected)
				break;
			ClearSelection();
			SendMessageW(LogHwnd, LB_SETSEL, true, RightClickedItem);
			break;
		}

		case WM_RBUTTONUP:
		{
			ResumeAddingEntries();

			POINT MousePos{ ((int)(short)LOWORD(lParam)), ((int)(short)HIWORD(lParam)) };
			TStringBuilder<1024> OutString;

			int32 LogIndex = IntCastChecked<int32>(SendMessageW(LogHwnd, LB_GETITEMDATA, RightClickedItem, 0) - LogIndexOffset);
			RightClickedItem = -1;
			if (LogIndex < 0)
			{
				RECT Rect;
				GetClientRect(hWnd, &Rect);
				if (!PtInRect(&Rect, MousePos))
					break;
			}
			else
			{
				CreateLogEntryText(OutString, Log[LogIndex], false);
			}

			HMENU Menu = CreatePopupMenu();
			MENUITEMINFO Info;
			Info.cbSize = sizeof(Info);
			Info.dwItemData = 0;
			auto AddItem = [&](const TCHAR* Str, int32 id)
			{
				Info.fMask = MIIM_STRING | MIIM_ID | MIIM_DATA;
				Info.fType = MFT_STRING;
				Info.dwTypeData = (TCHAR*)Str;
				Info.wID = id;
				InsertMenuItem(Menu, IntCastChecked<UINT>(Info.dwItemData++), 1, &Info);
			};

			auto AddSeparator = [&]()
			{
				Info.fMask = 0;
				Info.fType = MFT_SEPARATOR;
				InsertMenuItem(Menu, IntCastChecked<UINT>(Info.dwItemData++), 1, &Info);
			};

			AddItem(TEXT("Activate auto scroll"), 101);
			AddItem(TEXT("Copy selected line(s)"), 102);
			AddSeparator();
			AddItem(TEXT("Clear all"), 103);
			AddItem(TEXT("Clear all entries above line"), 104);
			AddItem(TEXT("Clear filters"), 121);

			if (LogIndex >= 0)
			{
				SelectedCategory = Log[LogIndex].Category.ToString();
				SelectedVerbosity = Log[LogIndex].Verbosity;

				AddSeparator();
				AddItem(*TStringBuilder<64>().Append(TEXT("Include category '")).Append(*SelectedCategory).Append(TEXT("' in filter")), 124);
				AddItem(*TStringBuilder<64>().Append(TEXT("Exclude category '")).Append(*SelectedCategory).Append(TEXT("' in filter")), 125);

				AddItem(*TStringBuilder<64>().Append(TEXT("Include verbosity '")).Append(ToString(SelectedVerbosity)).Append(TEXT("' in filter")), 126);
				AddItem(*TStringBuilder<64>().Append(TEXT("Exclude verbosity '")).Append(ToString(SelectedVerbosity)).Append(TEXT("' in filter")), 127);
			}

			int CharIndex = (MousePos.x + (LogFontWidth / 2)) / LogFontWidth;
			if (CharIndex < OutString.Len())
			{
				auto ValidChar = [](TCHAR C) { return FChar::IsAlnum(C) || C == '_' || C == '-'; };

				const TCHAR* Str = *OutString;
				const TCHAR* Start = Str + CharIndex;
				while (Start != Str)
				{
					TCHAR C = *(Start - 1);
					if (!ValidChar(C))
						break;
					--Start;
				}
				const TCHAR* End = Str + CharIndex;
				while (true)
 				{
					if (!ValidChar(*End))
						break;
					++End;
				}
				if (Start != End)
				{
					SelectedWord = FStringView(Start, UE_PTRDIFF_TO_INT32(End - Start));

					AddSeparator();
					AddItem(*TStringBuilder<64>().Append("Copy word '").Append(SelectedWord).Append("'"), 120);
					AddSeparator();
					AddItem(*TStringBuilder<64>().Append("Include word '").Append(SelectedWord).Append("' in filter"), 122);
					AddItem(*TStringBuilder<64>().Append("Exclude word '").Append(SelectedWord).Append("' in filter"), 123);
				}
			}

			AddSeparator();
			if (ConsoleShowDateTime != 1)
				AddItem(TEXT("Show Time"), 131);
			if (ConsoleShowDateTime != 2)
				AddItem(TEXT("Show Date and Time"), 132);
			if (ConsoleShowDateTime != 0)
				AddItem(TEXT("Hide Date and Time"), 133);
			if (bConsoleShowVerbosity)
				AddItem(TEXT("Hide Verbosity"), 134);
			if (!bConsoleShowVerbosity)
				AddItem(TEXT("Show Verbosity"), 135);

			MapWindowPoints(hWnd, HWND_DESKTOP, &MousePos, 1);
			TrackPopupMenu(Menu, 0, MousePos.x, MousePos.y, 0, hWnd, NULL);
			break;
		}

		case WM_COMMAND:
		{
			switch (LOWORD(wParam))
			{
			case 101:
				EnableAutoScroll();
				break;

			case 102:
				CopySelectionToClipboard();
				break;

			case 103:
				ClearLog();
				bAutoScrollLog = true;
				break;

			case 104:
			{
				int32 SelectedItemIndex = -1;
				if (SendMessageW(LogHwnd, LB_GETSELITEMS, 1, (LPARAM)&SelectedItemIndex) != 1)
					break;
				int32 LogIndex = IntCastChecked<int32>(SendMessageW(LogHwnd, LB_GETITEMDATA, SelectedItemIndex, 0) - LogIndexOffset);
				LogIndexOffset += LogIndex;
				Log.PopFront(LogIndex);
				RefreshLogHwnd();
				break;
			}
			case 120:
				FWindowsPlatformApplicationMisc::ClipboardCopy(*SelectedWord);
				break;

			case 121:
				UpdateIncludeFilter(TEXT(""));
				UpdateExcludeFilter(TEXT(""));
				IncludeVerbosity.Reset();
				ExcludeVerbosity.Reset();
				RefreshLogHwnd();
				break;

			case 122:
				UpdateIncludeFilter(*(ConsoleIncludeFilterStr.IsEmpty() ? SelectedWord : (ConsoleIncludeFilterStr + " " + SelectedWord)));
				RefreshLogHwnd();
				break;

			case 123:
				UpdateExcludeFilter(*(ConsoleExcludeFilterStr.IsEmpty() ? SelectedWord : (ConsoleExcludeFilterStr + " " + SelectedWord)));
				RefreshLogHwnd();
				break;

			case 124:
			case 125:
			{
				TStringBuilder<128> Str;
				Str << (LOWORD(wParam) == 124 ? *ConsoleIncludeFilterStr : *ConsoleExcludeFilterStr);
				if (Str.Len() != 0)
					Str << ' ';
				Str << '[' << SelectedCategory << ']';
				if (LOWORD(wParam) == 124)
					UpdateIncludeFilter(*Str);
				else
					UpdateExcludeFilter(*Str);
				RefreshLogHwnd();
				break;
			}
			case 126:
			case 127:
			{
				if (LOWORD(wParam) == 126)
					IncludeVerbosity.AddUnique(SelectedVerbosity);
				else
					ExcludeVerbosity.AddUnique(SelectedVerbosity);
				RefreshLogHwnd();
				break;
			}
			case 131:
				ConsoleShowDateTime = 1;
				RefreshLogHwnd();
				break;

			case 132:
				ConsoleShowDateTime = 2;
				RefreshLogHwnd();
				break;

			case 133:
				ConsoleShowDateTime = 0;
				RefreshLogHwnd();
				break;

			case 134:
				bConsoleShowVerbosity = 0;
				RefreshLogHwnd();
				break;

			case 135:
				bConsoleShowVerbosity = 1;
				RefreshLogHwnd();
				break;
			}
			break;
		}

		case WM_KEYDOWN:
			if (wParam == VK_SHIFT)
				SuspendAddingEntries();
			break;

		case WM_KEYUP:
			if (wParam == VK_SHIFT)
				ResumeAddingEntries();
			break;

		case WM_KILLFOCUS:
			ResumeAddingEntries();
			break;

		case WM_MOUSEHOVER:
			{
				POINT MousePos{ ((int)(short)LOWORD(lParam)), ((int)(short)HIWORD(lParam)) };
				TipPosition = MousePos;
				int32 Res = IntCastChecked<int32>(SendMessageW(hWnd, LB_ITEMFROMPOINT, 0, lParam));
				int32 ItemIndex = LOWORD(Res);
				int32 LogIndex = IntCastChecked<int32>(SendMessageW(hWnd, LB_GETITEMDATA, ItemIndex, 0) - LogIndexOffset);
				if (LogIndex <= 0 || LogIndex >= Log.Num())
					break;
				TStringBuilder<1024> OutString;
				CreateLogEntryText(OutString, Log[LogIndex], false);
				int32 EntryStrLen = OutString.Len();
				RECT LogRect;
				GetClientRect(LogHwnd, &LogRect);
				int32 VisibleCharCount = LogRect.right / LogFontWidth;
				if (EntryStrLen <= VisibleCharCount)
					break;
				const TCHAR* Str = *OutString + VisibleCharCount;
				int32 ExtraLen = FCString::Strlen(Str) * LogFontWidth + 3;
				HDC dc = GetDC(TipHwnd);
				SelectObject(dc, LogFont);
				ReleaseDC(TipHwnd, dc);
				POINT ExtraPos{VisibleCharCount*LogFontWidth,(MousePos.y/LogFontHeight)*LogFontHeight};
				MapWindowPoints(LogHwnd, HWND_DESKTOP, &ExtraPos, 1);
				SetWindowPos(TipHwnd, MainHwnd, ExtraPos.x, ExtraPos.y-1, ExtraLen, LogFontHeight+2, SWP_NOACTIVATE | SWP_NOSENDCHANGING | SWP_NOZORDER | SWP_NOOWNERZORDER);
				SetWindowText(TipHwnd, Str);
				RedrawWindow(hWnd, NULL, NULL, RDW_INVALIDATE | RDW_UPDATENOW);
				break;
			}
		}

		return LogHwndWndProcPtr(hWnd, Msg, wParam, lParam);
	}

	LRESULT CommandHwndWndProc(HWND hWnd, UINT Msg, WPARAM wParam, LPARAM lParam)
	{
		switch (Msg)
		{
		case WM_KEYUP:
		{
			if (wParam == VK_UP)
			{
				TCHAR CurrentCommand[1024] = { '\0' };
				GetWindowText(CommandHwnd, CurrentCommand, 1024);

				FString Command;
				Owner->GetLastCommand(Command);

				if (Command == CurrentCommand)
				{
					Owner->GetPreviousCommand(Command);
				}
				SetWindowText(CommandHwnd, *Command);
			}
			else if (wParam == VK_DOWN)
			{
				FString NextCommand;
				Owner->GetNextCommand(NextCommand);
				SetWindowText(CommandHwnd, *NextCommand);
			}
			break;
		}
		}

		return CommandHwndWndProcPtr(hWnd, Msg, wParam, lParam);
	}

	static LRESULT CALLBACK StaticMessageBoxHookProc(int nCode, WPARAM wParam, LPARAM lParam)
	{
		if (nCode == HCBT_ACTIVATE)
		{
			HWND hwnd = (HWND)wParam;
			TCHAR Title[128];
			GetWindowText(hwnd, Title, 64);
			if (FCString::Strncmp(Title, HelpTitle, 63) == 0)
			{
				HWND parentHwnd = GetParent(hwnd);
				int32 Sx, Sy, Px, Py;
				RECT R1;
				RECT R2;
				GetWindowRect(parentHwnd, &R1); GetWindowRect(hwnd, &R2);
				Sx = R2.right - R2.left, Px = R1.left + (R1.right - R1.left) / 2 - Sx / 2;
				Sy = R2.bottom - R2.top, Py = R1.top + (R1.bottom - R1.top) / 2 - Sy / 2;
				MoveWindow(hwnd, Px, Py, Sx, Sy, 1);
			}
		}
		return CallNextHookEx(0, nCode, wParam, lParam);
	}

	bool ExtractHyperLinks(TStringBuilder<256>& OutStatus, TArray<FString>* OutHyperLinks, const FString& String)
	{
		const TCHAR* Str = *String;
		while (true)
		{
			const TCHAR* LinkScopeStart = FCString::Strchr(Str, '[');
			if (LinkScopeStart == nullptr)
			{
				break;
			}

			++LinkScopeStart;

			const TCHAR* LinkScopeEnd = FCString::Strchr(LinkScopeStart, ']');
			if (LinkScopeEnd == nullptr)
			{
				break;
			}

			OutStatus.Append(Str, int32(LinkScopeStart - Str - 1));
			Str = LinkScopeEnd + 1;

			if (!OutHyperLinks)
			{
				continue;
			}

			FStringView LinkScope = FStringView(LinkScopeStart, int32(LinkScopeEnd - LinkScopeStart)).TrimStartAndEnd();

			while (true)
			{
				int32 SpaceIndex;
				if (LinkScope.FindChar(' ', SpaceIndex))
				{
					FStringView Link = LinkScope.Left(SpaceIndex).TrimStartAndEnd();
					if (!Link.IsEmpty())
					{
						OutHyperLinks->Add(FString(Link));
					}
					LinkScope.RightChopInline(SpaceIndex + 1);
				}
				else
				{
					FStringView Link = LinkScope.TrimStartAndEnd();
					if (!Link.IsEmpty())
					{
						OutHyperLinks->Add(FString(Link));
					}
					break;
				}
			}
		}

		OutStatus << Str;

		return OutHyperLinks && OutHyperLinks->Num() != 0;
	}

	void HandleActivityModifications()
	{
		bool bActivitiesLightDirty = false;

		// Transfer modifications to Activities and update IdToActivity
		{
			TArray<int, TInlineAllocator<32>> IndicesToRemove;

			FScopeLock _(&ActivityModificationsCs);

			for (auto& ModPair : ActivityModifications)
			{
				int Index = ModPair.Key;
				ActivityModification& Mod = ModPair.Value;
				if (Mod.bRemove)
				{
					IndicesToRemove.Add(Index);
					continue;
				}
				if (Index >= Activities.Num())
				{
					Activities.SetNum(Index+1);
					Activity& A = Activities[Index];
					A.Name = Mod.Name;
					A.Light = Mod.Light;
					A.SortValue = Mod.SortValue;
					A.Status = Mod.Status;
					A.bAlignLeft = Mod.bAlignLeft;
					continue;
				}

				Activity& A = Activities[Index];
				if (A.Light != Mod.Light)
				{
					A.Light = Mod.Light;
					bActivitiesLightDirty = true;
				}
				if (A.Status != Mod.Status)
				{
					A.Status = Mod.Status;
					A.bStatusDirty = true;
				}
			}

			for (int I = IndicesToRemove.Num() - 1; I >= 0; --I)
			{
				int ToRemove = IndicesToRemove[I];
				if (ToRemove < Activities.Num())
					Activities[ToRemove].Name.Empty();
				for (auto It = IdToActivityIndex.CreateIterator(); It; ++It)
				{
					if (It.Value() == ToRemove)
					{
						It.RemoveCurrent();
					}
					else if (It.Value() > ToRemove)
					{
						--It.Value();
					}
				}
			}

			ActivityModifications.Reset();
		}

		bool bUpdatePositions = false;
		for (int32 i = 0, e = Activities.Num(); i != e;)
		{
			Activity& A = Activities[i];

			if (A.Name.IsEmpty())
			{
				DestroyWindow(A.NameHwnd);
				DestroyWindow(A.StatusHwnd);
				Activities.RemoveAt(i);
				bUpdatePositions = true;
				--e;
				continue;
			}

			if (!A.NameHwnd)
			{
				A.NameHwnd = CreateTextHwnd(*A.Name, Font);

				TStringBuilder<256> Status;
				ExtractHyperLinks(Status, nullptr, A.Status);
				A.StatusHwnd = CreateTextHwnd(*Status, Font, ID_TRACKEDACTIVITY, SS_NOTIFY);

				bUpdatePositions = true;
			}
			else if (A.bStatusDirty)
			{
				A.bStatusDirty = false;
				TStringBuilder<256> Status;
				ExtractHyperLinks(Status, nullptr, A.Status);
				SetWindowTextW(A.StatusHwnd, *Status);
			}
			++i;
		}

		if (bActivitiesLightDirty)
		{
			InvalidateRect(MainHwnd, NULL, false);
			bActivitiesLightDirty = false;
		}

		if (bUpdatePositions)
		{
			SendMessageW(LogHwnd, WM_SETREDRAW, false, 0);
			RECT Rect;
			GetClientRect(MainHwnd, &Rect);
			Rect = FromScale(Rect);

			int NewActivitiesTotalHeight = TraverseActivityPositions(Rect.right, Rect.bottom, [](Activity& A, int32 X, int32 Y, int32 Width, int32 RowIndex) {});
			if (NewActivitiesTotalHeight == ActivitiesTotalHeight)
			{
				UpdateActivityPositions(Rect.right, Rect.bottom, false);
			}
			else
			{
				ActivitiesTotalHeight = NewActivitiesTotalHeight;
				UpdateSize(Rect.right, Rect.bottom, false);
			}
			RedrawWindow(MainHwnd, NULL, NULL, RDW_ERASE | RDW_NOINTERNALPAINT | RDW_INVALIDATE | RDW_ALLCHILDREN | RDW_UPDATENOW); // Don't change flags, very sensitive with glitches during resizing
			SendMessageW(LogHwnd, WM_SETREDRAW, true, 0);
		}
	}

	template<typename T>
	T ToScale(T Value) const { return T(Value*DpiScale); }
	template<typename T>
	T FromScale(T Value) const { return T(Value/DpiScale); }

	RECT ToScale(const RECT& Rect)
	{
		return { ToScale(Rect.left), ToScale(Rect.top), ToScale(Rect.right), ToScale(Rect.bottom) };
	}

	RECT FromScale(const RECT& Rect)
	{
		return { FromScale(Rect.left), FromScale(Rect.top), FromScale(Rect.right), FromScale(Rect.bottom) };
	}

	void SetWindowPosScale(HWND hWnd, HWND hWndInsertAfter, int X, int Y, int Cx, int Cy, UINT Flags)
	{
		::SetWindowPos(hWnd, hWndInsertAfter, ToScale(X), ToScale(Y), ToScale(Cx), ToScale(Cy), Flags);
	}

	void FillRect(HDC Dc, const RECT& Rect, HBRUSH Brush)
	{
		RECT R = Rect;
		::FillRect(Dc, &R, Brush);
	}

	void RectangleScale(HDC Dc, int32 X, int32 Y, int32 Width, int32 Height)
	{
		::Rectangle(Dc, ToScale(X), ToScale(Y), ToScale(Width), ToScale(Height));
	}

	void GetTextRectW(HDC Dc, const wchar_t* Text, int32 Len, RECT* Rect, int32 Flags)
	{
		::DrawTextW(Dc, Text, Len, Rect, Flags | DT_MODIFYSTRING);
	}

	void DrawTextW(HDC Dc, const wchar_t* Text, int32 Len, const RECT& Rect, int32 Flags)
	{
		RECT R = Rect;
		::DrawTextW(Dc, Text, Len, &R, Flags);
	}

	void MoveToScale(HDC Dc, int32 X, int32 Y)
	{
		::MoveToEx(Dc, ToScale(X), ToScale(Y), NULL);
	}

	void LineToScale(HDC Dc, int32 X, int32 Y)
	{
		::LineTo(Dc, ToScale(X), ToScale(Y));
	}

	void HandleNewLogEntries()
	{
		if (bSuspendAddingEntries)
			return;

		{
			FScopeLock _(&NewLogEntriesCs);
			Swap(NewLogEntries, TempLogEntries);
		}

		if (TempLogEntries.IsEmpty())
			return;

		int32 OldSize = Log.Num();

		for (NewLogEntry& E : TempLogEntries)
		{
			const TCHAR* SearchStr = *E.String;
			while (true)
			{
				if (const TCHAR* LineBreak = FCString::Strchr(SearchStr, '\n'))
				{
					int32 Len = UE_PTRDIFF_TO_INT32(LineBreak - SearchStr);
					if (LineBreak > SearchStr && *(LineBreak - 1) == '\r')
						--Len;
					Log.Add({ FString::ConstructFromPtrSize(SearchStr, Len), E.Verbosity, E.Category, E.Time, E.TextAttribute, 1 });
					SearchStr = LineBreak + 1;
				}
				else
				{
					if (int32 Len = FCString::Strlen(SearchStr))
						Log.Add({ SearchStr, E.Verbosity, E.Category, E.Time, E.TextAttribute, 1 });
					break;
				}
			}
		}

		int32 NewSize = Log.Num();

		int32 ToAddToHwnd = NewSize - OldSize;
		int32 LogIndex = OldSize;
		int32 NewTopIndex = -1;
		int32 NewCaretIndex = -1;

		SendMessageW(LogHwnd, WM_SETREDRAW, false, 0);

		if (NewSize > LogHistoryMaxSize)
		{
			int32 ToRemove = NewSize - LogHistoryMaxSize;
			Log.PopFront(ToRemove);
			LogIndexOffset += ToRemove;

			int32 ToChangeInHwnd = FMath::Min(LogHistoryMaxSize, ToRemove);

			int32 ToRemoveFromHwnd = ToChangeInHwnd;
			if (!bAutoScrollLog)
			{
				int32 TopIndex = IntCastChecked<int32>(SendMessageW(LogHwnd, LB_GETTOPINDEX, 0, 0));
				NewTopIndex = FMath::Max(TopIndex - ToRemoveFromHwnd, 0);
				int32 CaretIndex = IntCastChecked<int32>(SendMessageW(LogHwnd, LB_GETCARETINDEX, 0, 0));
				NewCaretIndex = FMath::Max(CaretIndex - ToRemoveFromHwnd, 0);
			}

			while (ToRemoveFromHwnd--)
				SendMessageW(LogHwnd, LB_DELETESTRING, 0, 0);

			ToAddToHwnd = ToChangeInHwnd;
			LogIndex = LogHistoryMaxSize - ToAddToHwnd;
		}

		while (ToAddToHwnd--)
		{
			AddEntryToLogHwnd(Log[LogIndex], LogIndexOffset + LogIndex);
			++LogIndex;
		}
		TempLogEntries.Reset(0);

		if (bAutoScrollLog)
		{
			ScrollToBottom();
		}
		else if (NewTopIndex != -1)
		{
			SendMessageW(LogHwnd, LB_SETCARETINDEX, NewCaretIndex, 0);
			SendMessageW(LogHwnd, LB_SETSEL, true, NewCaretIndex);
			SetTopVisible(NewTopIndex, false);
		}
		RedrawLogScrollbar();

		SendMessageW(LogHwnd, WM_SETREDRAW, true, 0);
	}

	void DrawNonClientArea(HWND hWnd, WPARAM wParam)
	{
		RECT windowRect;
		GetWindowRect(hWnd, &windowRect);
		HRGN region = NULL;
		if (wParam == NULLREGION)
		{
			region = CreateRectRgn(windowRect.left, windowRect.top, windowRect.right, windowRect.bottom);
		}
		else
		{
			HRGN copy = CreateRectRgn(0, 0, 0, 0);
			if (CombineRgn(copy, (HRGN)wParam, NULL, RGN_COPY))
				region = copy;
			else
				DeleteObject(copy);
		}

		HDC dc = GetDCEx(hWnd, region, DCX_WINDOW | DCX_CACHE | DCX_INTERSECTRGN | DCX_LOCKWINDOWUPDATE);
		if (!dc)
		{
			if (region)
			{
				DeleteObject(region);
			}
			return;
		}

		RECT rect = FromScale(windowRect);

		int32 width = rect.right - rect.left;
		int32 height = rect.bottom - rect.top;

		int32 ButtonWidth = 27;

		RECT CaptionRect{ 0,0, width, 27 };
		FillRect(dc, ToScale(CaptionRect), BackgroundBrush);

		SelectObject(dc, GetStockObject(NULL_BRUSH));
		SelectObject(dc, WindowBorderPen);
		Rectangle(dc, 0, 0, windowRect.right - windowRect.left, windowRect.bottom - windowRect.top);

		DrawIconEx(dc, ToScale(3), ToScale(3), Icon, ToScale(21), ToScale(21), 0, NULL, DI_NORMAL);

		SetBkColor(dc, BackgroundColor);
		SetTextColor(dc, TextColor);
		SelectObject(dc, Font);
		CaptionRect.left += 32;
		CaptionRect.right -= ButtonWidth * 4;
		DrawTextW(dc, *ConsoleTitle, ConsoleTitle.Len(), ToScale(CaptionRect), DT_SINGLELINE | DT_VCENTER | DT_END_ELLIPSIS);

		SelectObject(dc, TextPen);

		if (NcButtonHot != -1 && (NcButtonDown == -1 || NcButtonHot == NcButtonDown))
		{
			int32 NcButton = NcButtonHot;
			HBRUSH Brush = ButtonBrush;
			if (NcButtonDown != -1)
				Brush = ButtonPressedBrush;
			int32 ButtonIndex = 0;
			if (NcButton == HTMAXBUTTON)
				ButtonIndex = 1;
			else if (NcButton == HTMINBUTTON)
				ButtonIndex = 2;
			else if (NcButton == HTEXPAND)
				ButtonIndex = 3;
			RECT ButtonRect{ width - ButtonWidth*(ButtonIndex+1), 1, width - 1 - ButtonWidth*ButtonIndex, 27};
			FillRect(dc, ToScale(ButtonRect), Brush);
		}

		int32 MiddleY = 27 / 2;
		int32 MiddleX = width - ButtonWidth / 2 - ButtonWidth * 3 - 1;

		// Double arrow
		if (bConsoleExpanded)
		{
			MoveToScale(dc, MiddleX - 4, MiddleY - 4);
			LineToScale(dc, MiddleX, MiddleY);
			LineToScale(dc, MiddleX + 5, MiddleY - 5);
			MoveToScale(dc, MiddleX - 4, MiddleY);
			LineToScale(dc, MiddleX, MiddleY + 4);
			LineToScale(dc, MiddleX + 5, MiddleY - 1);
		}
		else
		{
			MoveToScale(dc, MiddleX - 4, MiddleY);
			LineToScale(dc, MiddleX, MiddleY - 4);
			LineToScale(dc, MiddleX + 5, MiddleY + 1);
			MoveToScale(dc, MiddleX - 4, MiddleY + 4);
			LineToScale(dc, MiddleX, MiddleY);
			LineToScale(dc, MiddleX + 5, MiddleY + 5);
		}

		// Minimize
		MiddleX += ButtonWidth;
		MoveToScale(dc, MiddleX - 4, MiddleY);
		LineToScale(dc, MiddleX + 5, MiddleY);
			
		// Maximize
		MiddleX += ButtonWidth;
		WINDOWPLACEMENT Placement;
		Placement.length = sizeof(Placement);
		GetWindowPlacement(hWnd, &Placement);
		if (Placement.showCmd == SW_MAXIMIZE)
		{
			RectangleScale(dc, MiddleX, MiddleY - 3, MiddleX + 8, MiddleY + 5);
			SelectObject(dc, BackgroundBrush);
			RectangleScale(dc, MiddleX - 3, MiddleY, MiddleX + 5, MiddleY + 8);
		}
		else
		{
			RectangleScale(dc, MiddleX - 5, MiddleY - 5, MiddleX + 5, MiddleY + 5);
		}
			
		// Close
		MiddleX += ButtonWidth;
		MoveToScale(dc, MiddleX - 4, MiddleY - 4);
		LineToScale(dc, MiddleX + 5, MiddleY + 5);
		MoveToScale(dc, MiddleX + 4, MiddleY - 4);
		LineToScale(dc, MiddleX - 5, MiddleY + 5);

		ReleaseDC(hWnd, dc);
	}

	LRESULT MainWinProc(HWND hWnd, UINT Msg, WPARAM wParam, LPARAM lParam)
	{
		switch (Msg)
		{
		case WM_HELP:
			{
				HHOOK Hook = SetWindowsHookEx(WH_CBT, StaticMessageBoxHookProc, NULL, GetCurrentThreadId());
				MessageBox(MainHwnd, HelpText, HelpTitle, MB_OK | MB_ICONINFORMATION);
				UnhookWindowsHookEx(Hook);
		}
			break;

		case WM_SIZE:
		{
			switch (wParam)
			{
			case SIZE_MAXIMIZED:
			case SIZE_RESTORED:
				UpdateSize(FromScale(LOWORD(lParam)), FromScale(HIWORD(lParam)), true);
				RECT WindowRect;
				GetWindowRect(MainHwnd, &WindowRect);
				ConsoleWidth = WindowRect.right - WindowRect.left;
				ConsoleHeight = WindowRect.bottom - WindowRect.top;
			default:
				// ignore all other messages, such as minimize
				break;
			}
			return 0;
		}
		case WM_MOVE:
		{
			RECT WindowRect;
			GetWindowRect(MainHwnd, &WindowRect);
			ConsolePosX = WindowRect.left;
			ConsolePosY = WindowRect.top;
			break;
		}

		case WM_SETFOCUS:
			SetFocus(CommandHwnd);
			SendMessageW(CommandHwnd, EM_SETSEL, 0, -1);
			break;

		case WM_MOUSEWHEEL:
			return HandleMouseWheel(wParam);

		case WM_ERASEBKGND:
		{
			HDC Hdc = (HDC)wParam;
			RECT Rect;
			GetClientRect(hWnd, &Rect);
			FillRect(Hdc, Rect, BackgroundBrush);

			// Borders around edit boxes
			SelectObject(Hdc, BorderPen);
			SelectObject(Hdc, GetStockObject(NULL_BRUSH));
			for (int32 dlgId : {ID_LOG_INCLUDE_FILTER, ID_LOG_EXCLUDE_FILTER, ID_COMMAND})
			{
				HWND editHwnd = GetDlgItem(hWnd, dlgId);
				if (!IsWindowVisible(editHwnd))
					continue;
				RECT EditRect;
				GetClientRect(editHwnd, &EditRect);
				MapWindowPoints(editHwnd, hWnd, (LPPOINT)&EditRect, 2);
				Rectangle(Hdc, EditRect.left - 1, EditRect.top - 1, EditRect.right + 1, EditRect.bottom + 1);
			}
			return true;
		}

		case WM_CTLCOLORSTATIC:
		{
			HDC Hdc = (HDC)wParam;
			SetTextColor(Hdc, TextColor);
			SetBkColor(Hdc, BackgroundColor);
			return (INT_PTR)BackgroundBrush;
		}

		case WM_CTLCOLOREDIT:
		{
			HWND filterHwnd = (HWND)lParam;
			HDC Hdc = (HDC)wParam;
			SetBkColor(Hdc, EditBackgroundColor);
			RECT Rect;
			GetClientRect(filterHwnd, &Rect);
			FillRect(Hdc, Rect, EditBackgroundBrush);
			SetTextColor(Hdc, TextColor);
			if (!GetWindowTextLength(filterHwnd))
			{
				const TCHAR* Str = filterHwnd == CommandHwnd ? TEXT("Type command here") : (filterHwnd == IncludeFilterHwnd ? TEXT("Add include filter here") : TEXT("Add exclude filter here"));
				TextOut(Hdc, 0, 0, Str, FCString::Strlen(Str));
			}
			return (INT_PTR)GetStockObject(NULL_BRUSH);
		}

		case WM_CTLCOLORLISTBOX:
			if ((HWND)lParam == LogHwnd)
			{
				int32 ItemCount = (int)SendMessageW(LogHwnd, LB_GETCOUNT, 0, 0);
				RECT Rect;
				GetClientRect(LogHwnd, &Rect);
				int32 VisibleCount = FMath::CeilToInt(float(Rect.bottom) / (float)LogFontHeight);
				int32 TopIndex = IntCastChecked<int32>(SendMessageW(LogHwnd, LB_GETTOPINDEX, 0, 0));
				if (ItemCount - TopIndex < VisibleCount)
				{
					Rect.top = (ItemCount - TopIndex) * LogFontHeight;
					FillRect((HDC)wParam, Rect, LogBackgroundBrush); // Draw the padding below the last line
				}
				return (INT_PTR)GetStockObject(NULL_BRUSH);
			}
			break;

		case WM_MEASUREITEM:
		{
			PMEASUREITEMSTRUCT pmis = (PMEASUREITEMSTRUCT)lParam;
			pmis->itemHeight = LogFontHeight;
			return true;
		}

		case WM_NOTIFY:
		{
			LPNMHDR hdr = (LPNMHDR)lParam;
			if (hdr->code != NM_CUSTOMDRAW)
				break;
			switch (hdr->idFrom)
			{
			case ID_COMMANDBUTTON:
			case ID_CLEARLOGBUTTON:
			case ID_ADDCHECKPOINTBUTTON:
			{
				NMCUSTOMDRAW& nmcd = *(NMCUSTOMDRAW*)hdr;
				uint32 CtlId = IntCastChecked<uint32>(hdr->idFrom);
				int32 ItemState = nmcd.uItemState;
				TCHAR Str[32];
				int32 StrLen = GetDlgItemText(hWnd, CtlId, Str, 32);
				HBRUSH Brush = ButtonBrush;
				COLORREF Color = ButtonColor;
				if (ItemState & CDIS_SELECTED)
				{
					Brush = ButtonPressedBrush;
					Color = ButtonPressedColor;
				}
				else if (ItemState & CDIS_HOT)
				{
					Brush = ButtonHighlightBrush;
					Color = ButtonHighlightColor;
				}
				SelectObject(nmcd.hdc, GetStockObject(NULL_BRUSH));
				FillRect(nmcd.hdc, nmcd.rc, Brush);
				SetBkColor(nmcd.hdc, Color);
				SetTextColor(nmcd.hdc, TextColor);
				DrawTextW(nmcd.hdc, Str, StrLen, nmcd.rc, DT_SINGLELINE | DT_VCENTER | DT_CENTER | DT_NOCLIP);
				SelectObject(nmcd.hdc, BorderPen);
				Rectangle(nmcd.hdc, nmcd.rc.left, nmcd.rc.top, nmcd.rc.right, nmcd.rc.bottom);
				return CDRF_SKIPDEFAULT;
			}
			}
			break;
		}

		case WM_DRAWITEM:
		{
			PDRAWITEMSTRUCT pdis = (PDRAWITEMSTRUCT)lParam;
			HDC Hdc = pdis->hDC;

			if (pdis->CtlType == ODT_STATIC)
			{
				TCHAR Str[512];
				int32 StrLen = GetWindowText(pdis->hwndItem, Str, UE_ARRAY_COUNT(Str));

				if (pdis->hwndItem == TipHwnd)
				{
					if (pdis->rcItem.bottom != 1)
					{
						FillRect(Hdc, pdis->rcItem, BackgroundBrush);
						SetTextColor(Hdc, GetSysColor(COLOR_HIGHLIGHTTEXT));
						SelectObject(Hdc, LogFont);
						DrawTextW(Hdc, Str, StrLen, pdis->rcItem, DT_SINGLELINE | DT_VCENTER);
					}
					break;
				}


				HBRUSH Brush = BackgroundBrush;

				int32 RowIndex = GetWindowLong(pdis->hwndItem, GWLP_USERDATA);
				if (RowIndex != -1)
				{
					Brush = StatusBackgroundBrush[(RowIndex & 1)];
					SetBkColor(Hdc, StatusBackgroundColor[(RowIndex & 1)]);
				}

				SelectObject(Hdc, Brush);

				// All this stuff just to prevent flickering when text changes fast
				int32 TextOffset = 1;
				RECT ModifiedRect = pdis->rcItem;
				GetTextRectW(Hdc, Str, StrLen, &ModifiedRect, DT_SINGLELINE | DT_END_ELLIPSIS | DT_CALCRECT);
				RECT TempRect = pdis->rcItem;
				TempRect.bottom = TempRect.top + TextOffset;
				TempRect.right = ModifiedRect.right;
				FillRect(Hdc, TempRect, Brush);

				TempRect = pdis->rcItem;
				TempRect.left = ModifiedRect.right;
				FillRect(Hdc, TempRect, Brush);
				if (ModifiedRect.bottom < pdis->rcItem.bottom)
				{
					TempRect = pdis->rcItem;
					TempRect.right = ModifiedRect.right;
					TempRect.top = ModifiedRect.bottom;
					FillRect(Hdc, TempRect, Brush);
				}

				StrLen = FCString::Strlen(Str);
				ExtTextOut(Hdc, pdis->rcItem.left, pdis->rcItem.top + TextOffset, 0, &pdis->rcItem, Str, StrLen, 0);
				break;
			}

			if (pdis->CtlID != ID_LOG || pdis->itemID == -1)
				break;
			switch (pdis->itemAction)
			{
			case ODA_SELECT:
			case ODA_DRAWENTIRE:
			{
				uint32 LogVirtualIndex = IntCastChecked<uint32>(pdis->itemData);
				LogEntry& Entry = Log[LogVirtualIndex - LogIndexOffset];


				int Middle = (pdis->rcItem.top + pdis->rcItem.bottom) / 2;
				int YPos = Middle - LogFontHeight / 2 - (Entry.LineCount - 1) * LogFontHeight / 2;

				if (pdis->itemState & ODS_SELECTED)
				{
					FillRect(Hdc, pdis->rcItem, GetSysColorBrush(COLOR_HIGHLIGHT));
					SetTextColor(Hdc, GetSysColor(COLOR_HIGHLIGHTTEXT));
				}
				else
				{
					FillRect(Hdc, pdis->rcItem, LogBackgroundBrush);
					if (Entry.TextAttribute == (FOREGROUND_INTENSITY | FOREGROUND_RED))
						SetTextColor(Hdc, RGB(220, 0, 0));
					else if (Entry.TextAttribute == (FOREGROUND_INTENSITY | FOREGROUND_RED | FOREGROUND_GREEN))
						SetTextColor(Hdc, RGB(220, 220, 0));
					else
						SetTextColor(Hdc, RGB(204, 204, 204));
				}

				TStringBuilder<1024> OutString;
				CreateLogEntryText(OutString, Entry, false);
				int32 XPos = 0;
				TextOut(Hdc, XPos, YPos, *OutString, OutString.Len());
				return true;
			}
			case ODA_FOCUS: // We don't want the focus rectangle at all (dotted rectangle)
				return true;
			}
			break;
		}

		case WM_PAINT:
		{
			PAINTSTRUCT ps;
			HDC Hdc = BeginPaint(hWnd, &ps);
			RECT Rect;
			GetClientRect(hWnd, &Rect);

			TraverseActivityPositions(FromScale(Rect.right), FromScale(Rect.bottom), [&](Activity& A, int32 X, int32 Y, int32 Width, int32 RowIndex)
				{
					if (A.Light == 0)
						return;
					SelectObject(Hdc, StatusLightBrush[A.Light - 1]);
					int32 XOffset = 12;
					int32 YOffset = 1;
					int32 FontHeight = 16;
					int32 CircleSize = 11;
					int32 HalfDiff = (FontHeight - CircleSize) / 2;
					RoundRect(Hdc, ToScale(X - CircleSize + XOffset), ToScale(Y + HalfDiff + YOffset), ToScale(X + XOffset), ToScale(Y + FontHeight - HalfDiff + YOffset), ToScale(CircleSize), ToScale(CircleSize));
				});
			EndPaint(hWnd, &ps);
			return 0;
		}

		case WM_CLOSE:
			PostQuitMessage(0);
			break;

		case WM_VKEYTOITEM:
			if ((HWND)lParam == LogHwnd)
			{
				if ((LOWORD(wParam) == 0x43 || LOWORD(wParam) == VK_INSERT) && (::GetKeyState(VK_CONTROL) >> 15)) // Copy-paste
				{
					TArray<int> SelectedItems;
					if (int32 SelectionCount = GetSelectedItems(SelectedItems))
					{
						TArray<TCHAR> Buffer;
						TStringBuilder<512> StringBuilder;
						for (int32 I = 0; I != SelectionCount; ++I)
						{
							int32 Index = SelectedItems[I];
							int32 LogIndex = IntCastChecked<int32>(SendMessageW(LogHwnd, LB_GETITEMDATA, Index, 0) - LogIndexOffset);
							StringBuilder.Append(Log[LogIndex].String);
							StringBuilder.Append(TEXT("\r\n"));
						}

						FWindowsPlatformApplicationMisc::ClipboardCopy(*StringBuilder);
					}
				}
				else if (LOWORD(wParam) == VK_END && !(::GetKeyState(VK_SHIFT) >> 15)) // Enable auto scrolling and remove selection
				{
					EnableAutoScroll();
					return -2;
				}
			}

			break;
		case WM_COMMAND:
			if (LOWORD(wParam) == ID_LOG && HIWORD(wParam) == LBN_SELCHANGE)
			{
				if (SendMessageW(LogHwnd, LB_GETSELCOUNT, 0, 0) != 0)
					bAutoScrollLog = false;
				RedrawLogScrollbar();
			}
			else if ((LOWORD(wParam) == ID_LOG_INCLUDE_FILTER || LOWORD(wParam) == ID_LOG_EXCLUDE_FILTER) && HIWORD(wParam) == EN_CHANGE)
			{
				TCHAR Str[1024];
				GetDlgItemText(MainHwnd, LOWORD(wParam), Str, 1024);
				FString& FilterStr = LOWORD(wParam) == ID_LOG_INCLUDE_FILTER ? ConsoleIncludeFilterStr : ConsoleExcludeFilterStr;
				TArray<FString>& Filter = LOWORD(wParam) == ID_LOG_INCLUDE_FILTER ? IncludeFilter : ExcludeFilter;
				FilterStr = Str;
				TArray<FString> ParsedFilter;
				FilterStr.ParseIntoArray(ParsedFilter, TEXT(" "));
				if (Filter == ParsedFilter)
					break;
				Filter = ParsedFilter;
				RefreshLogHwnd();
			}
			else if (LOWORD(wParam) == ID_COMMANDBUTTON)
			{
				TCHAR Command[1024];
				if (!GetWindowText(CommandHwnd, Command, 1024))
					break;
				SetWindowText(CommandHwnd, TEXT(""));
				EConsoleColor NewConsoleColor = EConsoleColor(-1);
				int32 OldConsoleShowDateTime = ConsoleShowDateTime;
				bool bOldConsoleShowVerbosity = bConsoleShowVerbosity;
				if (FCString::Stricmp(Command, TEXT("red")) == 0)
					NewConsoleColor = EConsoleColor::Red;
				else if (FCString::Stricmp(Command, TEXT("darkgray")) == 0)
					NewConsoleColor = EConsoleColor::DarkGray;
				else if (FCString::Stricmp(Command, TEXT("gray")) == 0)
					NewConsoleColor = EConsoleColor::Gray;
				else if (FCString::Stricmp(Command, TEXT("lightblue")) == 0)
					NewConsoleColor = EConsoleColor::LightBlue;
				else if (FCString::Stricmp(Command, TEXT("darkblue")) == 0)
					NewConsoleColor = EConsoleColor::DarkBlue;
				else if (FCString::Stricmp(Command, TEXT("load")) == 0)
					NewConsoleColor = EConsoleColor::Load;
				else if (FCString::Stricmp(Command, TEXT("showtime")) == 0)
					ConsoleShowDateTime = 1;
				else if (FCString::Stricmp(Command, TEXT("showdatetime")) == 0)
					ConsoleShowDateTime = 2;
				else if (FCString::Stricmp(Command, TEXT("hidedatetime")) == 0)
					ConsoleShowDateTime = 0;
				else if (FCString::Stricmp(Command, TEXT("showverbosity")) == 0)
					bConsoleShowVerbosity = true;
				else if (FCString::Stricmp(Command, TEXT("hideverbosity")) == 0)
					bConsoleShowVerbosity = false;

				if (ConsoleShowDateTime != OldConsoleShowDateTime
					|| bConsoleShowVerbosity != bOldConsoleShowVerbosity)
					RefreshLogHwnd();

				if (NewConsoleColor != EConsoleColor(-1))
				{
					ConsoleColor = NewConsoleColor;
					DestroyColors();
					CreateColors();
					InvalidateRect(MainHwnd, NULL, true);
					EnumChildWindows(MainHwnd, [](HWND hwnd, LPARAM lParam) -> BOOL { InvalidateRect(hwnd, NULL, true); return true;}, 0);
					RedrawNC();
					break;
				}

				Owner->AddCommand(Command);

				// Note: This will execute during the CoreTicker part of the EngineLoop (after Engine Tick is complete).
				// This decision was made to avoid the previous Async implementation which ticks during Engine/World Tick (TG_PrePhysics).
				// The reasoning is that the new MultiServer Proxy does not execute TickGroups (UE_SUPPORT_FOR_ACTOR_TICK_DISABLE).
				ExecuteOnGameThread(UE_SOURCE_LOCATION, [Cmd = FString(Command)]()
					{
						TArray<IConsoleCommandExecutor*> CmdExecutors = IModularFeatures::Get().GetModularFeatureImplementations<IConsoleCommandExecutor>(IConsoleCommandExecutor::ModularFeatureName());
						if (IConsoleCommandExecutor** CmdExecutor = CmdExecutors.FindByPredicate([](IConsoleCommandExecutor* CmdExecutor) { return CmdExecutor->GetName() == NAME_Cmd; }))
						{
							UE_LOGF(LogExec, Log, "Executing console command: %ls", *Cmd);
							(*CmdExecutor)->Exec(*Cmd);
						}
						else
						{
							UE_LOGF(LogExec, Log, "Failed to execute console command: %ls", *Cmd);
						}
					});
			}
			else if (LOWORD(wParam) == ID_CLEARLOGBUTTON)
			{
				ClearLog();
				bAutoScrollLog = true;
			}
			else if (LOWORD(wParam) == ID_ADDCHECKPOINTBUTTON)
			{
				TStringBuilder<64> TempString;
				TempString.Appendf(TEXT("LOGCHECKPOINT%i"), CheckpointIndex);
				AddLogEntry(TempString, ELogVerbosity::Display, FName("LogCheckpoint"), 0, FOREGROUND_BLUE | FOREGROUND_GREEN | FOREGROUND_RED);

				++CheckpointIndex;
				TCHAR ButtonString[64] = TEXT("");
				FCString::Sprintf(ButtonString, TEXT("Log CHECKPOINT%i"), CheckpointIndex);
				SetDlgItemText(MainHwnd, ID_ADDCHECKPOINTBUTTON, ButtonString);
			}
			else if (LOWORD(wParam) == ID_TRACKEDACTIVITY)
			{
				TStringBuilder<256> Status;
				TArray<FString> HyperLinks;

				for (const Activity& A : Activities)
				{
					if (A.StatusHwnd == (HWND)lParam)
					{
						ExtractHyperLinks(Status, &HyperLinks, A.Status);
					}
				}

				for (const FString& Link : HyperLinks)
					FPlatformProcess::LaunchURL(*Link, NULL, NULL);
			}
			break;

		case DM_GETDEFID:
			return MAKEWPARAM(ID_COMMANDBUTTON, DC_HASDEFID);

		case WM_NCHITTEST: // This actually draw things
		{
			int32 Res = HitTestNCB(lParam);
			if (Res != HTNOWHERE)
				return Res;
			return HitTestNCA(hWnd, lParam);
		}

		case WM_MOUSEMOVE:
		case WM_NCMOUSELEAVE:
		{
			if (NcButtonHot == -1 && NcButtonDown == -1)
				break;
			POINT p;
			GetCursorPos(&p);
			int32 Res = HitTestNCB(p.x, p.y);
			if (Res != HTMINBUTTON && Res != HTMAXBUTTON && Res != HTCLOSE && Res != HTEXPAND)
				Res = -1;
			if (Res == NcButtonHot)
				break;
			NcButtonHot = Res;
			RedrawNC();
			break;
		}

		case WM_NCMOUSEMOVE:
		{
			int32 Res = HitTestNCB(lParam);
			if (Res != HTMINBUTTON && Res != HTMAXBUTTON && Res != HTCLOSE && Res != HTEXPAND)
				Res = -1;
			if (Res == NcButtonHot)
				break;
			NcButtonHot = Res;
			RedrawNC();
			if (Res == -1)
				break;
			TRACKMOUSEEVENT tme;
			tme.cbSize = sizeof(tme);
			tme.dwFlags = TME_NONCLIENT | TME_LEAVE;
			tme.hwndTrack = hWnd;
			TrackMouseEvent(&tme);
			break;
		}

		case WM_NCLBUTTONDOWN:
		{
			int32 Res = HitTestNCB(lParam);
			if (Res == HTMINBUTTON || Res == HTMAXBUTTON || Res == HTCLOSE || Res == HTEXPAND)
			{
				SetCapture(MainHwnd);
				NcButtonDown = Res;
				RedrawNC();
				return 0;
			}
			break;
		}

		case WM_LBUTTONUP:
		{
			if (NcButtonDown == -1)
				break;
			ReleaseCapture();
			int32 Old = NcButtonDown;
			NcButtonDown = -1;
			POINT p;
			GetCursorPos(&p);
			int32 Res = HitTestNCB(p.x, p.y);
			RedrawNC();
			if (Res != Old)
				break;
			if (Res == HTEXPAND)
			{
				bConsoleExpanded = !bConsoleExpanded;
				RedrawNC();
				RECT Rect;
				GetClientRect(MainHwnd, &Rect);
				UpdateSize(FromScale(Rect.right), FromScale(Rect.bottom), true);
				RedrawWindow(MainHwnd, NULL, NULL, RDW_INVALIDATE);
				if (bConsoleExpanded)
					SetFocus(CommandHwnd);
			}
			else if (Res == HTMINBUTTON)
				ShowWindow(hWnd, SW_MINIMIZE);
			else if (Res == HTMAXBUTTON)
			{
				WINDOWPLACEMENT Placement;
				Placement.length = sizeof(Placement);
				GetWindowPlacement(hWnd, &Placement);
				ShowWindow(hWnd, Placement.showCmd == SW_MAXIMIZE ? SW_NORMAL : SW_MAXIMIZE);
			}
			else if (Res == HTCLOSE)
				PostQuitMessage(0);
			break;
		}

		case WM_NCCALCSIZE:
		{
			LPNCCALCSIZE_PARAMS ncParams = (LPNCCALCSIZE_PARAMS)lParam;
			ncParams->rgrc[0].top += ToScale(27);
			ncParams->rgrc[0].left += ToScale(1);
			ncParams->rgrc[0].bottom -= ToScale(1);
			ncParams->rgrc[0].right -= ToScale(1);
			return 0;
		}
		case WM_NCPAINT:
		{
			DrawNonClientArea(hWnd, wParam);
			return 0;
		}
		case WM_NCACTIVATE:
			RedrawWindow(hWnd, NULL, NULL, RDW_UPDATENOW);
			return 1;
		}
		return DefWindowProc(hWnd, Msg, wParam, lParam);
	}

	enum class EConsoleColor : int32
	{
		DarkGray,
		Gray,
		Red,
		LightBlue,
		DarkBlue,
		Load,
	};

	FWindowsConsoleOutputDevice2* Owner;

	struct NewLogEntry { FString String; ELogVerbosity::Type Verbosity; const class FName Category; double Time; uint16 TextAttribute; };
	struct LogEntry { FString String; ELogVerbosity::Type Verbosity; const class FName Category; double Time; uint16 TextAttribute; uint16 LineCount; };
	FCriticalSection NewLogEntriesCs;
	TArray<NewLogEntry> NewLogEntries;
	TArray<NewLogEntry> TempLogEntries;
	TArray<FString> IncludeFilter;
	TArray<FString> ExcludeFilter;
	TArray<ELogVerbosity::Type> IncludeVerbosity;
	TArray<ELogVerbosity::Type> ExcludeVerbosity;
	TRingBuffer<LogEntry> Log;
	FDateTime StartDateTime;
	double StartTime;
	double DpiScale = 1.0;
	int32 AddedEntryLogVirtualIndex = -1;
	HICON Icon;
	HFONT Font;
	HFONT LogFont;
	HWND MainHwnd;
	HWND LogHwnd;
	HWND LogScrollHwnd;
	HWND CommandHwnd;
	HWND RunCommandHwnd;
	HWND IncludeFilterHwnd;
	HWND ExcludeFilterHwnd;
	HWND ClearLogButtonHwnd;
	HWND CheckpointButtonHwnd;
	HWND TipHwnd;
	POINT TipPosition;
	WNDPROC LogHwndWndProcPtr;
	WNDPROC CommandHwndWndProcPtr;
	COLORREF BorderColor;
	COLORREF TextColor;
	COLORREF BackgroundColor;
	COLORREF EditBackgroundColor;
	COLORREF StatusBackgroundColor[2];
	COLORREF ButtonColor;
	COLORREF ButtonHighlightColor;
	COLORREF ButtonPressedColor;
	COLORREF ThumbColor;
	COLORREF ScrollBackgroundColor;
	HBRUSH TextBrush;
	HBRUSH BackgroundBrush;
	HBRUSH LogBackgroundBrush;
	HBRUSH EditBackgroundBrush;
	HBRUSH StatusBackgroundBrush[2];
	HBRUSH ButtonBrush;
	HBRUSH ButtonHighlightBrush;
	HBRUSH ButtonPressedBrush;
	HBRUSH ThumbBrush;
	HBRUSH ScrollBackgroundBrush;
	HBRUSH StatusLightBrush[3];
	HPEN BorderPen;
	HPEN WindowBorderPen;
	HPEN TextPen;
	HPEN NoPen;
	int32 LogIndexOffset = 0;
	int32 LogFontWidth = 0;
	int32 LogFontHeight = 0;
	int32 LogScrollGrabPos = -1;
	int32 NcButtonHot = -1;
	int32 NcButtonDown = -1;
	int32 RightClickedItem = -1;
	FString SelectedWord;
	FString SelectedCategory;
	ELogVerbosity::Type SelectedVerbosity;

	struct Activity { HWND NameHwnd = 0; HWND StatusHwnd = 0; FString Name; FString Status; int32 Light = 0; int32 SortValue; bool bStatusDirty = false; bool bAlignLeft = false; };
	struct ActivityModification { FString Name; FString Status; int32 Light = 0; int32 SortValue; bool bAlignLeft = false; bool bRemove = false; };
	TArray<Activity> Activities;
	TMap<FString, int> IdToActivityIndex;
	FCriticalSection ActivityModificationsCs;
	TSortedMap<int, ActivityModification> ActivityModifications;
	int32 ActivitiesTotalHeight = 0;
	HANDLE DirtyEvent;
	
	HANDLE Thread;
	bool bAutoScrollLog = true;
	bool bSuspendAddingEntries = false;

	int32 CheckpointIndex = 0;

	bool bIsVisible = false;
	bool bExiting = false;
	int32 ConsoleWidth = 160;
	int32 ConsoleHeight = 4000;
	int32 ConsolePosX = 0;
	int32 ConsolePosY = 0;
	EConsoleColor ConsoleColor = EConsoleColor::DarkGray;
	int32 ConsoleShowDateTime = 0; // 0 show nothing, 1 show only time, 2 show date and time
	FString ConsoleIncludeFilterStr;
	FString ConsoleExcludeFilterStr;

	FString ConsoleTitle;
	bool bConsoleExpanded = false;
	bool bConsoleShowVerbosity = false;
};

FName AppStatusTrackerName("AppStatusTracker");

FWindowsConsoleOutputDevice2::FWindowsConsoleOutputDevice2()
	: OverrideColorSet(false)
	, Window(nullptr)
{
#if !UE_BUILD_SHIPPING
	auto ParseColorStr = [](const FString& ColorStr) -> const TCHAR*
	{
		const TCHAR* ReturnVal = nullptr;

		if (ColorStr == TEXTVIEW("Black"))
		{
			ReturnVal = COLOR_BLACK;
		}
		else if (ColorStr == TEXTVIEW("DarkRed"))
		{
			ReturnVal = COLOR_DARK_RED;
		}
		else if (ColorStr == TEXTVIEW("DarkGreen"))
		{
			ReturnVal = COLOR_DARK_GREEN;
		}
		else if (ColorStr == TEXTVIEW("DarkBlue"))
		{
			ReturnVal = COLOR_DARK_BLUE;
		}
		else if (ColorStr == TEXTVIEW("DarkYellow"))
		{
			ReturnVal = COLOR_DARK_YELLOW;
		}
		else if (ColorStr == TEXTVIEW("DarkCyan"))
		{
			ReturnVal = COLOR_DARK_CYAN;
		}
		else if (ColorStr == TEXTVIEW("DarkPurple"))
		{
			ReturnVal = COLOR_DARK_PURPLE;
		}
		else if (ColorStr == TEXTVIEW("Gray"))
		{
			ReturnVal = COLOR_DARK_WHITE;
		}
		else if (ColorStr == TEXTVIEW("Red"))
		{
			ReturnVal = COLOR_RED;
		}
		else if (ColorStr == TEXTVIEW("Green"))
		{
			ReturnVal = COLOR_GREEN;
		}
		else if (ColorStr == TEXTVIEW("Blue"))
		{
			ReturnVal = COLOR_BLUE;
		}
		else if (ColorStr == TEXTVIEW("Yellow"))
		{
			ReturnVal = COLOR_YELLOW;
		}
		else if (ColorStr == TEXTVIEW("Cyan"))
		{
			ReturnVal = COLOR_CYAN;
		}
		else if (ColorStr == TEXTVIEW("Purple"))
		{
			ReturnVal = COLOR_PURPLE;
		}
		else if (ColorStr == TEXTVIEW("White"))
		{
			ReturnVal = COLOR_WHITE;
		}

		return ReturnVal;
	};

	FString HighlightsStr;

	// -LogHighlights="LogNet Cyan, LogTemp Green"
	if (FParse::Value(FCommandLine::Get(), TEXT("LogHighlights="), HighlightsStr))
	{
		TArray<FString> HighlightsList;

		HighlightsStr.ParseIntoArray(HighlightsList, TEXT(","));

		for (const FString& CurHighlightEntry : HighlightsList)
		{
			TArray<FString> CategoryAndColor;

			if (CurHighlightEntry.TrimStartAndEnd().ParseIntoArray(CategoryAndColor, TEXT(" ")) && CategoryAndColor.Num() == 2)
			{
				const TCHAR* ColorStr = ParseColorStr(CategoryAndColor[1].TrimStartAndEnd());

				if (ColorStr != nullptr)
				{
					FLogHighlight& NewEntry = LogHighlights.AddDefaulted_GetRef();

					NewEntry.Category = FName(CategoryAndColor[0]);
					NewEntry.Color = ColorStr;
				}
			}
		}
	}

	FString StringHighlights;

	// -LogStringHighlights="UNetConnection::Close=Purple, NotifyAcceptingConnection accepted from=DarkGreen"
	if (FParse::Value(FCommandLine::Get(), TEXT("LogStringHighlights="), StringHighlights))
	{
		TArray<FString> StringHighlightsList;

		StringHighlights.ParseIntoArray(StringHighlightsList, TEXT(","));

		for (const FString& CurStringHighlightEntry : StringHighlightsList)
		{
			TArray<FString> StringAndColor;

			if (CurStringHighlightEntry.ParseIntoArray(StringAndColor, TEXT("=")) && StringAndColor.Num() == 2)
			{
				const TCHAR* ColorStr = ParseColorStr(StringAndColor[1].TrimStartAndEnd());

				if (ColorStr != nullptr)
				{
					FLogStringHighlight& NewEntry = LogStringHighlights.AddDefaulted_GetRef();

					NewEntry.SearchString = StringAndColor[0].GetCharArray();
					NewEntry.Color = ColorStr;
				}
			}
		}
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("SuppressConsoleOutputVerboseLogging")))
	{
		AllowedLogVerbosity = ELogVerbosity::Log;
	}
#endif
}

FWindowsConsoleOutputDevice2::~FWindowsConsoleOutputDevice2()
{
	delete Window;
}

void FWindowsConsoleOutputDevice2::SaveToINI()
{
	WindowRWLock.WriteLock();
	if (!Window)
	{
		WindowRWLock.WriteUnlock();
		return;
	}
	int32 ConsoleWidth = Window->FromScale(Window->ConsoleWidth);
	int32 ConsoleHeight = Window->FromScale(Window->ConsoleHeight);
	int32 ConsolePosX = Window->FromScale(Window->ConsolePosX);
	int32 ConsolePosY = Window->FromScale(Window->ConsolePosY);
	int32 ConsoleColor = (int32)Window->ConsoleColor;
	int32 ConsoleShowDateTime = Window->ConsoleShowDateTime;
	FString ConsoleIncludeFilterStr = Window->ConsoleIncludeFilterStr;
	FString ConsoleExcludeFilterStr = Window->ConsoleExcludeFilterStr;
	bool bConsoleExpanded = Window->bConsoleExpanded;
	bool bConsoleShowVerbosity = Window->bConsoleShowVerbosity;
	WindowRWLock.WriteUnlock();

	FString Filename = GetConfigFilename();
	const TCHAR* Selection = TEXT("ConsoleWindows");
	if (IsRunningDedicatedServer())
		Selection = TEXT("ServerConsoleWindows");
	else if (IsRunningGame())
		Selection = TEXT("GameConsoleWindows");

	FConfigCacheIni Config(EConfigCacheType::DiskBacked);

	// Workaround to file not saving. Now requires file to exist before calling LoadFile to save
	if (!IFileManager::Get().FileExists(*Filename))
	{
		FFileHelper::SaveStringToFile(FStringView(), *Filename);
	}

	Config.LoadFile(Filename);

	Config.SetInt(Selection, TEXT("ConsoleX"), ConsolePosX, Filename);
	Config.SetInt(Selection, TEXT("ConsoleY"), ConsolePosY, Filename);

	Config.SetInt(Selection, TEXT("ConsoleWidth"), ConsoleWidth, Filename);
	Config.SetInt(Selection, TEXT("ConsoleHeight"), ConsoleHeight, Filename);

	Config.SetInt(Selection, TEXT("ConsoleColor"), ConsoleColor, Filename);

	Config.SetBool(Selection, TEXT("ConsoleExpanded"), bConsoleExpanded, Filename);
	Config.SetBool(Selection, TEXT("ConsoleShowVerbosity"), bConsoleShowVerbosity, Filename);

	Config.SetInt(Selection, TEXT("ConsoleShowDateTime"), ConsoleShowDateTime, Filename);

	Config.SetString(Selection, TEXT("IncludeFilter"), *ConsoleIncludeFilterStr, Filename);
	Config.SetString(Selection, TEXT("ExcludeFilter"), *ConsoleExcludeFilterStr, Filename);

	Config.SetArray(Selection, TEXT("CommandHistory"), CommandHistory, Filename);

	Config.Flush(false, Filename);
}

const FString& FWindowsConsoleOutputDevice2::GetConfigFilename()
{
	static FString Filename = FPaths::EngineSavedDir() / TEXT("Config") / TEXT("DebugConsole.ini");
	return Filename;
}

void FWindowsConsoleOutputDevice2::Show( bool ShowWindow )
{
	if (ShowWindow)
	{
		check(IsInGameThread());

		int32 ConsoleWidth = 1000;
		int32 ConsoleHeight = 700;
		int32 ConsolePosX = 0;
		int32 ConsolePosY = 0;
		int32 ConsoleColor = 0;
		int32 ConsoleShowDateTime = 0;
		FString ConsoleIncludeFilterStr;
		FString ConsoleExcludeFilterStr;

		bool bConsoleExpanded = false;
		bool bConsoleShowVerbosity = false;
		bool bHasX = false;
		bool bHasY = false;

		FString Filename = GetConfigFilename();

		const TCHAR* Selection = TEXT("ConsoleWindows");
		if (IsRunningDedicatedServer())
			Selection = TEXT("ServerConsoleWindows");
		else if (IsRunningGame())
			Selection = TEXT("GameConsoleWindows");

		FConfigCacheIni Config(EConfigCacheType::Temporary);
		Config.LoadFile(Filename);
		Config.GetInt(Selection, TEXT("ConsoleWidth"), ConsoleWidth, Filename);
		Config.GetInt(Selection, TEXT("ConsoleHeight"), ConsoleHeight, Filename);
		bHasX = Config.GetInt(Selection, TEXT("ConsoleX"), ConsolePosX, Filename);
		bHasY = Config.GetInt(Selection, TEXT("ConsoleY"), ConsolePosY, Filename);
		Config.GetInt(Selection, TEXT("ConsoleColor"), ConsoleColor, Filename);
		Config.GetBool(Selection, TEXT("ConsoleExpanded"), bConsoleExpanded, Filename);
		Config.GetBool(Selection, TEXT("ConsoleShowVerbosity"), bConsoleShowVerbosity, Filename);
		Config.GetInt(Selection, TEXT("ConsoleShowDateTime"), ConsoleShowDateTime, Filename);
		Config.GetString(Selection, TEXT("IncludeFilter"), ConsoleIncludeFilterStr, Filename);
		Config.GetString(Selection, TEXT("ExcludeFilter"), ConsoleExcludeFilterStr, Filename);
		Config.GetArray(Selection, TEXT("CommandHistory"), CommandHistory, Filename);
		CommandHistoryCursor = CommandHistory.Num();

		if (!FParse::Value(FCommandLine::Get(), TEXT("ConsoleX="), ConsolePosX) && !bHasX)
		{
			ConsolePosX = CW_USEDEFAULT;
		}

		if (!FParse::Value(FCommandLine::Get(), TEXT("ConsoleY="), ConsolePosY) && !bHasY)
		{
			ConsolePosY = CW_USEDEFAULT;
		}

		FParse::Value(FCommandLine::Get(), TEXT("ConsoleIncludeFilter="), ConsoleIncludeFilterStr);

		FString ConsoleTitle;

		if (!FParse::Value(FCommandLine::Get(), TEXT("ConsoleTitle="), ConsoleTitle))
		{
			// Setting text so they are easily identifyable in taskbar
			const TCHAR* ConsoleType = TEXT("Unreal");
			if (IsRunningDedicatedServer())
				ConsoleType = TEXT("Server");
			else if (IsRunningGame())
				ConsoleType = TEXT("Client");
			ConsoleTitle = FString::Printf(TEXT("%s Console (%s) - %s  (F1 for help)"), ConsoleType, FApp::GetProjectName(), FPlatformProcess::ExecutablePath());
		}

		FDisplayMetrics DisplayMetrics;
		FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);

		// Make sure that the positions specified by INI/CMDLINE are proper
		static const int32 ActualConsoleWidth = ConsoleWidth;
		static const int32 ActualConsoleHeight = ConsoleHeight;

		static const int32 ActualScreenWidth = DisplayMetrics.VirtualDisplayRect.Right - DisplayMetrics.VirtualDisplayRect.Left;
		static const int32 ActualScreenHeight = DisplayMetrics.VirtualDisplayRect.Bottom - DisplayMetrics.VirtualDisplayRect.Top;

		static const int32 RightPadding = FMath::Max(50, FMath::Min((ActualConsoleWidth / 2), ActualScreenWidth / 2));
		static const int32 BottomPadding = FMath::Max(50, FMath::Min((ActualConsoleHeight / 2), ActualScreenHeight / 2));
				
		if (ConsolePosX != CW_USEDEFAULT)
			ConsolePosX = FMath::Min(FMath::Max(ConsolePosX, DisplayMetrics.VirtualDisplayRect.Left), DisplayMetrics.VirtualDisplayRect.Right - RightPadding);
		if (ConsolePosY != CW_USEDEFAULT)
			ConsolePosY = FMath::Min(FMath::Max(ConsolePosY, DisplayMetrics.VirtualDisplayRect.Top), DisplayMetrics.VirtualDisplayRect.Bottom - BottomPadding);

		FWriteScopeLock lock(WindowRWLock);

		bool bFirstCall = !Window;
		if (bFirstCall)
		{
			Window = new FConsoleWindow(this);
		}

		Window->ConsolePosX = ConsolePosX;
		Window->ConsolePosY = ConsolePosY;
		Window->ConsoleWidth = ConsoleWidth;
		Window->ConsoleHeight = ConsoleHeight;
		Window->ConsoleColor = (FConsoleWindow::EConsoleColor)ConsoleColor;
		Window->ConsoleShowDateTime = ConsoleShowDateTime;
		Window->ConsoleIncludeFilterStr = ConsoleIncludeFilterStr;
		Window->ConsoleExcludeFilterStr = ConsoleExcludeFilterStr;

		Window->ConsoleTitle = ConsoleTitle;
		Window->bConsoleExpanded = bConsoleExpanded;
		Window->bConsoleShowVerbosity = bConsoleShowVerbosity;
		Window->bIsVisible = true;

		if (bFirstCall)
		{
			Window->Start();
		}

	}
	else if (Window)
	{
		SaveToINI();
		Window->bIsVisible = false;
	}
}

bool FWindowsConsoleOutputDevice2::IsShown()
{
	return Window && Window->bIsVisible;
}

void FWindowsConsoleOutputDevice2::Serialize( const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category, const double Time )
{
	using namespace UE::String;
	
	if ((Verbosity & ELogVerbosity::VerbosityMask) > AllowedLogVerbosity)
		return;

	const double RealTime = Time == -1.0f ? FPlatformTime::Seconds() - GStartTime : Time;

	if (Verbosity == ELogVerbosity::SetColor)
	{
		SetColor( Data );
		OverrideColorSet = FCString::Strcmp(COLOR_NONE, Data) != 0;
	}
	else
	{
		bool bNeedToResetColor = false;
		if( !OverrideColorSet )
		{
			if( Verbosity == ELogVerbosity::Error )
			{
				SetColor( COLOR_RED );
				bNeedToResetColor = true;
			}
			else if( Verbosity == ELogVerbosity::Warning )
			{
				SetColor( COLOR_YELLOW );
				bNeedToResetColor = true;
			}
#if !UE_BUILD_SHIPPING
			else
			{
				if (LogHighlights.Num() > 0)
				{
					if (const FLogHighlight* CurHighlight = LogHighlights.FindByKey(Category))
					{
						SetColor(CurHighlight->Color);
						bNeedToResetColor = true;
					}
				}

				if (LogStringHighlights.Num() > 0)
				{
					FStringView DataView = TStringView(Data);

					for (const FLogStringHighlight& CurStringHighlight : LogStringHighlights)
					{
						if (FindFirst(DataView, CurStringHighlight.SearchString, ESearchCase::IgnoreCase) != INDEX_NONE)
						{
							SetColor(CurStringHighlight.Color);
							bNeedToResetColor = true;

							break;
						}
					}
				}
			}
#endif
		}
		{
			FReadScopeLock lock(WindowRWLock);
			if (Window)
			{
				Window->AddLogEntry(Data, Verbosity, Category, RealTime, TextAttribute);
			}
		}

		if (bNeedToResetColor)
		{
			SetColor( COLOR_NONE );
		}
	}
}

void FWindowsConsoleOutputDevice2::Serialize( const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category )
{
	Serialize( Data, Verbosity, Category, -1.0 );
}

void FWindowsConsoleOutputDevice2::SetColor( const TCHAR* Color )
{
	// here we can change the color of the text to display, it's in the format:
	// ForegroundRed | ForegroundGreen | ForegroundBlue | ForegroundBright | BackgroundRed | BackgroundGreen | BackgroundBlue | BackgroundBright
	// where each value is either 0 or 1 (can leave off trailing 0's), so 
	// blue on bright yellow is "00101101" and red on black is "1"
	// An empty string reverts to the normal gray on black
	if (FCString::Stricmp(Color, TEXT("")) == 0)
	{
		TextAttribute = FOREGROUND_BLUE | FOREGROUND_GREEN | FOREGROUND_RED;
	}
	else
	{
		// turn the string into a bunch of 0's and 1's
		TCHAR String[9];
		FMemory::Memset(String, 0, sizeof(TCHAR) * UE_ARRAY_COUNT(String));
		FCString::Strncpy(String, Color, UE_ARRAY_COUNT(String));
		for (TCHAR* S = String; *S; S++)
		{
			*S -= '0';
		}
		// make the color
		TextAttribute = 
			(String[0] ? FOREGROUND_RED			: 0) | 
			(String[1] ? FOREGROUND_GREEN		: 0) | 
			(String[2] ? FOREGROUND_BLUE		: 0) | 
			(String[3] ? FOREGROUND_INTENSITY	: 0) | 
			(String[4] ? BACKGROUND_RED			: 0) | 
			(String[5] ? BACKGROUND_GREEN		: 0) | 
			(String[6] ? BACKGROUND_BLUE		: 0) | 
			(String[7] ? BACKGROUND_INTENSITY	: 0);
	}
}

bool FWindowsConsoleOutputDevice2::IsAttached()
{
	return Window != nullptr;
}

bool FWindowsConsoleOutputDevice2::CanBeUsedOnAnyThread() const
{
	return true;
}

bool FWindowsConsoleOutputDevice2::GetPreviousCommand(FString& OutCommand)
{
	if (CommandHistoryCursor > 0)
	{
		CommandHistoryCursor--;

		if (CommandHistoryCursor < CommandHistory.Num())
		{
			OutCommand = CommandHistory[CommandHistoryCursor];
			return true;
		}
	}
	return false;
}

bool FWindowsConsoleOutputDevice2::GetNextCommand(FString& OutCommand)
{
	if (CommandHistoryCursor < (CommandHistory.Num() - 1))
	{
		CommandHistoryCursor++;
		OutCommand = CommandHistory[CommandHistoryCursor];
		return true;
	}
	return false;
}

void FWindowsConsoleOutputDevice2::AddCommand(const FString& InCommand)
{	
	if (CommandHistoryCursor >= 0 && CommandHistoryCursor < CommandHistory.Num() && CommandHistory[CommandHistoryCursor] == InCommand)
	{
		// The command was probably executed from the command history. No need to store again.
		return;
	}

	CommandHistoryCursor = CommandHistory.Add(InCommand);
	SaveToINI();
}

bool FWindowsConsoleOutputDevice2::GetLastCommand(FString& OutCommand)
{
	if (CommandHistoryCursor >= 0 && CommandHistoryCursor < CommandHistory.Num())
	{
		OutCommand = CommandHistory[CommandHistoryCursor];
		return true;
	}
	return false;
}
