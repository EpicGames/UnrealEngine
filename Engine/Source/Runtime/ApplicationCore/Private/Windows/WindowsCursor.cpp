// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsCursor.h"
#include "Misc/AssertionMacros.h"
#include "Containers/UnrealString.h"
#include "Misc/Paths.h"
#include "Misc/CoreMisc.h"
#include "Math/Vector2D.h"
#include "Math/Color.h"
#include "Windows/WindowsWindow.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/FileHelper.h"
#include "Tasks/Pipe.h"

#include "Windows/AllowWindowsPlatformTypes.h"
	#include <Ole2.h>
	#include <oleidl.h>
#include "Windows/HideWindowsPlatformTypes.h"

static bool bUseInvisibleCursorForNoneCursorType = false;
FAutoConsoleVariableRef CVarUseInvisibleCursorForNoneCursorType(
	TEXT("WindowsCursor.UseInvisibleCursorForNoneCursorType"),
	bUseInvisibleCursorForNoneCursorType,
	TEXT("If enabled, sets the platform HCursor to a transparent cursor instead of null when the mouse cursor type to None."),
	ECVF_Default);

static TAutoConsoleVariable<bool> CVarDynamicLockToggle(
	TEXT("WindowsCursor.DynamicLockToggle"),
	true,
	TEXT("If enabled, lock the cursor automatically if our window is in the foreground."),
	ECVF_ReadOnly);

FWindowsCursor::FWindowsCursor()
{
	// Load up cursors that we'll be using
	for( int32 CurCursorIndex = 0; CurCursorIndex < EMouseCursor::TotalCursorCount; ++CurCursorIndex )
	{
		CursorHandles[ CurCursorIndex ] = NULL;
		CursorOverrideHandles[ CurCursorIndex ] = NULL;

		HCURSOR CursorHandle = NULL;
		switch( CurCursorIndex )
		{
		case EMouseCursor::None:
			if (FPlatformMisc::IsRemoteSession() || bUseInvisibleCursorForNoneCursorType)
			{
				// during remote sessions we rely on constantly resetting the mouse cursor position for infinite mouse deltas while dragging (WM_INPUT doesnt work over rdp)
				// this hack here is to avoid having a hidden cursor which prevents SetCursorPos from working.  This is just a completely transparent cursor yet "visible" cursor to work around that
				CursorHandle = LoadCursorFromFile((LPCTSTR) * (FString(FPlatformProcess::BaseDir()) / FString::Printf(TEXT("%s/Slate/Cursor/invisible.cur"), *FPaths::EngineContentDir())));
			}
			break;
		case EMouseCursor::Custom:
			// The mouse cursor will not be visible when None is used
			break;

		case EMouseCursor::Default:
			CursorHandle = ::LoadCursor( NULL, IDC_ARROW );
			break;

		case EMouseCursor::TextEditBeam:
			CursorHandle = ::LoadCursor( NULL, IDC_IBEAM );
			break;

		case EMouseCursor::ResizeLeftRight:
			CursorHandle = ::LoadCursor( NULL, IDC_SIZEWE );
			break;

		case EMouseCursor::ResizeUpDown:
			CursorHandle = ::LoadCursor( NULL, IDC_SIZENS );
			break;

		case EMouseCursor::ResizeSouthEast:
			CursorHandle = ::LoadCursor( NULL, IDC_SIZENWSE );
			break;

		case EMouseCursor::ResizeSouthWest:
			CursorHandle = ::LoadCursor( NULL, IDC_SIZENESW );
			break;

		case EMouseCursor::CardinalCross:
			CursorHandle = ::LoadCursor(NULL, IDC_SIZEALL);
			break;

		case EMouseCursor::Crosshairs:
			CursorHandle = ::LoadCursor( NULL, IDC_CROSS );
			break;

		case EMouseCursor::Hand:
			CursorHandle = ::LoadCursor( NULL, IDC_HAND );
			break;

		case EMouseCursor::GrabHand:
			CursorHandle = LoadCursorFromFile((LPCTSTR)*(FString( FPlatformProcess::BaseDir() ) / FString::Printf( TEXT("%sEditor/Slate/Cursor/grabhand.cur"), *FPaths::EngineContentDir() )));
			if (CursorHandle == NULL)
			{
				// Failed to load file, fall back
				CursorHandle = ::LoadCursor( NULL, IDC_HAND );
			}
			break;

		case EMouseCursor::GrabHandClosed:
			CursorHandle = LoadCursorFromFile((LPCTSTR)*(FString( FPlatformProcess::BaseDir() ) / FString::Printf( TEXT("%sEditor/Slate/Cursor/grabhand_closed.cur"), *FPaths::EngineContentDir() )));
			if (CursorHandle == NULL)
			{
				// Failed to load file, fall back
				CursorHandle = ::LoadCursor( NULL, IDC_HAND );
			}
			break;

		case EMouseCursor::SlashedCircle:
			CursorHandle = ::LoadCursor(NULL, IDC_NO);
			break;

		case EMouseCursor::EyeDropper:
			CursorHandle = LoadCursorFromFile((LPCTSTR)*(FString( FPlatformProcess::BaseDir() ) / FString::Printf( TEXT("%sEditor/Slate/Icons/eyedropper.cur"), *FPaths::EngineContentDir() )));
			break;

			// NOTE: For custom app cursors, use:
			//		CursorHandle = ::LoadCursor( InstanceHandle, (LPCWSTR)MY_RESOURCE_ID );

		default:
			// Unrecognized cursor type!
			check( 0 );
			break;
		}

		CursorHandles[ CurCursorIndex ] = CursorHandle;
	}

	// Set the default cursor
	SetType( EMouseCursor::Default );
}

FWindowsCursor::~FWindowsCursor()
{
	// Release cursors
	// NOTE: Shared cursors will automatically be destroyed when the application is destroyed.
	//       For dynamically created cursors, use DestroyCursor
	for( int32 CurCursorIndex = 0; CurCursorIndex < EMouseCursor::TotalCursorCount; ++CurCursorIndex )
	{
		switch( CurCursorIndex )
		{
		case EMouseCursor::None:
		case EMouseCursor::Default:
		case EMouseCursor::TextEditBeam:
		case EMouseCursor::ResizeLeftRight:
		case EMouseCursor::ResizeUpDown:
		case EMouseCursor::ResizeSouthEast:
		case EMouseCursor::ResizeSouthWest:
		case EMouseCursor::CardinalCross:
		case EMouseCursor::Crosshairs:
		case EMouseCursor::Hand:
		case EMouseCursor::GrabHand:
		case EMouseCursor::GrabHandClosed:
		case EMouseCursor::SlashedCircle:
		case EMouseCursor::EyeDropper:
		case EMouseCursor::Custom:
			// Standard shared cursors don't need to be destroyed
			break;

		default:
			// Unrecognized cursor type!
			check( 0 );
			break;
		}
	}
}

void* FWindowsCursor::CreateCursorFromFile(const FString& InPathToCursorWithoutExtension, FVector2D HotSpot)
{
	const FString AniCursor = InPathToCursorWithoutExtension + TEXT(".ani");
	const FString CurCursor = InPathToCursorWithoutExtension + TEXT(".cur");

	TArray<uint8> CursorFileData;
	if (FFileHelper::LoadFileToArray(CursorFileData, *AniCursor, FILEREAD_Silent) || FFileHelper::LoadFileToArray(CursorFileData, *CurCursor, FILEREAD_Silent))
	{
		//TODO Would be nice to find a way to do this that doesn't involve the temp file copy.

		// The cursors may be in a pak file, if that's the case we need to write it to a temporary file
		// and then load that file as the cursor.  It's a workaround because there doesn't appear to be
		// a good way to load a cursor from anything other than a loose file or a resource.
		FString TempCursorFile = FPaths::CreateTempFilename(FPlatformProcess::UserTempDir(), TEXT("Cursor-"), TEXT(".temp"));
		if (FFileHelper::SaveArrayToFile(CursorFileData, *TempCursorFile))
		{
			HCURSOR CursorHandle = (HCURSOR) LoadImage(NULL,
				*TempCursorFile,
				IMAGE_CURSOR,
				0,
				0,
				LR_LOADFROMFILE);

			IFileManager::Get().Delete(*TempCursorFile);

			return CursorHandle;
		}
	}

	return nullptr;
}

void* FWindowsCursor::CreateCursorFromRGBABuffer(const FColor* Pixels, int32 Width, int32 Height, FVector2D InHotSpot)
{
	TArray<FColor> BGRAPixels;
	BGRAPixels.AddUninitialized(Width * Height);

	TArray<uint8> MaskPixels;
	MaskPixels.AddUninitialized(Width * Height);

	for (int32 Index = 0; Index < BGRAPixels.Num(); Index++)
	{
		const FColor& SrcPixel = Pixels[Index];
		BGRAPixels[Index] = FColor(SrcPixel.B, SrcPixel.G, SrcPixel.R, SrcPixel.A);
		MaskPixels[Index] = 255;
	}

	// The bitmap created is already in BGRA format, so we can just hand over the buffer.
	HBITMAP CursorColor = ::CreateBitmap(Width, Height, 1, 32, BGRAPixels.GetData());
	// Just create a dummy mask, we're making a full 32bit bitmap with support for alpha, so no need to worry about the mask.
	HBITMAP CursorMask = ::CreateBitmap(Width, Height, 1, 8, MaskPixels.GetData());

	ICONINFO IconInfo = { 0 };
	IconInfo.fIcon = 0;
	IconInfo.xHotspot = IntCastChecked<DWORD>(FMath::RoundToInt(InHotSpot.X * Width));
	IconInfo.yHotspot = IntCastChecked<DWORD>(FMath::RoundToInt(InHotSpot.Y * Height));
	IconInfo.hbmColor = CursorColor;
	IconInfo.hbmMask = CursorColor;

	HCURSOR CursorHandle = ::CreateIconIndirect(&IconInfo);

	::DeleteObject(CursorColor);
	::DeleteObject(CursorMask);

	return CursorHandle;
}

FVector2D FWindowsCursor::GetPosition() const
{
	POINT CursorPos;
	return ::GetCursorPos(&CursorPos) ? FVector2D(CursorPos.x, CursorPos.y) : FVector2D::ZeroVector;
}

void FWindowsCursor::SetPosition(const int32 X, const int32 Y)
{
	::SetCursorPos(X, Y);
}

void FWindowsCursor::SetType( const EMouseCursor::Type InNewCursor )
{
	// NOTE: Watch out for contention with FWindowsViewport::UpdateMouseCursor
	checkf( InNewCursor < EMouseCursor::TotalCursorCount, TEXT("Invalid cursor(%d) supplied"), (int)InNewCursor );
	CurrentType = InNewCursor;

	if (CursorOverrideHandles[InNewCursor])
	{
		::SetCursor(CursorOverrideHandles[InNewCursor]);
	}
	else
	{
		::SetCursor(CursorHandles[InNewCursor]);
	}
}

void FWindowsCursor::GetSize( int32& Width, int32& Height ) const
{
	//TODO this is wrong, this should query the size of the cursor on the platform.

	Width = 16;
	Height = 16;
}

void FWindowsCursor::Show( bool bShow )
{
	if( bShow )
	{
		// Show mouse cursor. Each time ShowCursor(true) is called an internal value is incremented so we 
		// call ShowCursor until the cursor is actually shown (>= 0 value returned by showcursor)
		while ( ::ShowCursor(true)<0 );
	}
	else
	{		// Disable the cursor.  Wait until its actually disabled.
		while ( ::ShowCursor(false)>=0 );
	}
}

static void HandleDynamicClipTask(const TOptional<RECT>& InClipBounds)
{
	if (InClipBounds.IsSet())
	{
		static const DWORD ProcessId = GetCurrentProcessId();
		const HWND hWnd = GetForegroundWindow();
		DWORD idProcess = 0;
		if (GetWindowThreadProcessId(hWnd, &idProcess) && (idProcess == ProcessId))
		{
			RECT WindowRect;
			if (GetWindowRect(hWnd, &WindowRect))
			{
				const RECT& ClipRect = InClipBounds.GetValue();
				const bool bAllowClip =
					(FMath::Max(WindowRect.left, ClipRect.left) < FMath::Min(WindowRect.right, ClipRect.right)) &&
					(FMath::Max(WindowRect.top, ClipRect.top) < FMath::Min(WindowRect.bottom, ClipRect.bottom));
				if (bAllowClip)
				{
					::ClipCursor(&InClipBounds.GetValue());
				}
			}
		}
	}
	else
	{
		::ClipCursor(nullptr);
	}
}

static TOptional<RECT> ClipBounds;
static UE::Tasks::FPipe ClipPipe{ UE_SOURCE_LOCATION };
static void HandleDynamicClip()
{
	ClipPipe.Launch(
		UE_SOURCE_LOCATION,
		[InClipBounds = ClipBounds] { HandleDynamicClipTask(InClipBounds); },
		UE::Tasks::ETaskPriority::BackgroundLow);
}

void FWindowsCursor::Tick()
{
	if (CVarDynamicLockToggle.GetValueOnGameThread() && ClipBounds.IsSet())
	{
		HandleDynamicClip();
	}
}

void FWindowsCursor::Lock( const RECT* const Bounds )
{
	if (CVarDynamicLockToggle.GetValueOnGameThread())
	{
		if (Bounds)
		{
			ClipBounds.Emplace(*Bounds);
		}
		else
		{
			ClipBounds.Reset();
		}
		HandleDynamicClip();
	}
	else
	{
		::ClipCursor(Bounds);
	}
}

void FWindowsCursor::SetTypeShape(EMouseCursor::Type InCursorType, void* InCursorHandle)
{
	checkf(InCursorType < EMouseCursor::TotalCursorCount, TEXT("Invalid cursor(%d) supplied"), (int)InCursorType);

	HCURSOR CursorHandle = (HCURSOR)InCursorHandle;
	CursorOverrideHandles[InCursorType] = CursorHandle;

	if (CurrentType == InCursorType)
	{
		SetType(CurrentType);
	}
}
