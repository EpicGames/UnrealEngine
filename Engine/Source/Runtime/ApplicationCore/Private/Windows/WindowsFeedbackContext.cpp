// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsFeedbackContext.h"

#include "HAL/ThreadHeartBeat.h"
#include "Internationalization/Internationalization.h"
#include "Misc/App.h"
#include "Windows/WindowsHWrapper.h"

bool FWindowsFeedbackContext::YesNof(const FText& Question)
{
	if ((GIsClient || GIsEditor) && !GIsSilent && !FApp::IsUnattended())
	{
		FSlowHeartBeatScope SuspendHeartBeat;
		return ::MessageBox(nullptr, Question.ToString().GetCharArray().GetData(), *NSLOCTEXT("Core", "Question", "Question").ToString(), MB_YESNO | MB_TASKMODAL) == IDYES;
	}
	return false;
}
