// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsPlatformApplicationMisc.h"
#include "Windows/WindowsApplication.h"
#include "Windows/WindowsApplicationErrorOutputDevice.h"
#include "Windows/WindowsConsoleOutputDevice.h"
#include "Windows/WindowsConsoleOutputDevice2.h"
#include "Windows/WindowsRemoteConsoleOutputDevice.h"
#include "Windows/WindowsFeedbackContext.h"
#include "HAL/FeedbackContextAnsi.h"
#include "Misc/App.h"
#include "Math/Color.h"
#include "Misc/ScopeExit.h"
#include "Modules/ModuleManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Windows/WindowsD3D.h"
#include "Windows/WindowsPlatformOutputDevices.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "Templates/RefCounting.h"
#include "Null/NullPlatformApplicationMisc.h"

#if WITH_EDITOR
#include "Misc/MessageDialog.h"
#include "Windows/WindowsRedistributableValidation.h"
#endif

// Resource includes.
#include "Runtime/Launch/Resources/Windows/resource.h"

THIRD_PARTY_INCLUDES_START
#include "Windows/AllowWindowsPlatformTypes.h"
#include <shellscalingapi.h>
#include "Windows/HideWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_END

#define LOCTEXT_NAMESPACE "WindowsPlatformApplicationMisc"

void FWindowsPlatformApplicationMisc::PreInit()
{
	FApp::SetHasFocusFunction(&FWindowsPlatformApplicationMisc::IsThisApplicationForeground);

#if WITH_EDITOR
	// Note: We currently exclude Wine from visual c++ redistributable validation
	if (!FWindowsPlatformMisc::IsWine())
	{
		auto IsDllVersionValid = [](const TCHAR* ExecDirectory, const TCHAR* Name, VersionInfo& OutVersionInfo) -> bool
			{
				const FString FilePath = FPaths::Combine(ExecDirectory, Name);
				if (!FilePath.IsEmpty())
				{
					const uint64 FileVersion = FWindowsPlatformMisc::GetFileVersion(FilePath);
					if (FileVersion != 0)
					{
						OutVersionInfo = VersionInfo{ FileVersion };

						return IsVersionValid(OutVersionInfo, MinRedistVersion);
					}
				}

				return false;
			};

		// Reproducing verifications done in BootstrapPackagedGame.cpp
		bool bHasValidRedistributable = false;
		VersionInfo CurrentVersionInfo = {};

		const FString SystemRoot = FWindowsPlatformMisc::GetEnvironmentVariable(TEXT("SystemRoot"));
		const FString System32Path = FPaths::Combine(SystemRoot, TEXT("system32"));

		if (IsDllVersionValid(*System32Path, TEXT("msvcp140_2.dll"), CurrentVersionInfo) &&
			IsDllVersionValid(*System32Path, TEXT("vcruntime140_1.dll"), CurrentVersionInfo))
		{
			bHasValidRedistributable = true;
		}
		else if (IsDllVersionValid(TEXT(""), TEXT("msvcp140_2.dll"), CurrentVersionInfo) &&
			IsDllVersionValid(TEXT(""), TEXT("vcruntime140_1.dll"), CurrentVersionInfo))
		{
			bHasValidRedistributable = true;
		}

		if (!bHasValidRedistributable)
		{
			FFormatNamedArguments Arguments;

			if (CurrentVersionInfo.IsSet())
			{
				FFormatNamedArguments SubArguments;
				SubArguments.Add(TEXT("Current"), FText::AsCultureInvariant(
					FString::Printf(TEXT("%lu.%lu.%lu.%lu"),
						static_cast<unsigned long>(CurrentVersionInfo.Major),
						static_cast<unsigned long>(CurrentVersionInfo.Minor),
						static_cast<unsigned long>(CurrentVersionInfo.Bld),
						static_cast<unsigned long>(CurrentVersionInfo.Rbld)
					)
				));

				Arguments.Add(TEXT("Warning"), FText::Format(LOCTEXT("FWindowsPlatformApplicationMisc_Outdated", "Visual C++ redistributable version {Current} is outdated."), SubArguments));
			}
			else
			{
				Arguments.Add(TEXT("Warning"), LOCTEXT("FWindowsPlatformApplicationMisc_Invalid", "Invalid Visual C++ redistributable."));
			}

			Arguments.Add(TEXT("Version"), FText::AsCultureInvariant(
				FString::Printf(TEXT("%lu.%lu.%lu.%lu"),
					static_cast<unsigned long>(MinRedistVersion.Major),
					static_cast<unsigned long>(MinRedistVersion.Minor),
					static_cast<unsigned long>(MinRedistVersion.Bld),
					static_cast<unsigned long>(MinRedistVersion.Rbld)
				)
			));
			Arguments.Add(TEXT("Arch"), FText::AsCultureInvariant(FWindowsPlatformMisc::GetHostArchitecture()));

			const FText MessageText = FText::Format(LOCTEXT("FWindowsPlatformApplicationMisc_RedistValidation", "{Warning}\n\nPlease install version {Version} (or above) from Engine\\Extras\\Redist\\en-us\\vc_redist.{Arch}.exe."), Arguments);
			const FText MessageTitle(LOCTEXT("FWindowsPlatformApplicationMisc_Title", "Warning"));

			// Note: Message dialog is automatically disabled in unattended mode, see implementation.
			FMessageDialog::Open(EAppMsgType::Ok, MessageText, MessageTitle);

			UE_LOGF(LogWindows, Error, "%ls", *MessageText.ToString());
		}
	}
#endif	//WITH_EDITOR
}

void FWindowsPlatformApplicationMisc::LoadStartupModules()
{
#if !UE_SERVER
	FModuleManager::Get().LoadModule(TEXT("HeadMountedDisplay"));
#endif // !UE_SERVER

#if WITH_EDITOR
	FModuleManager::Get().LoadModule(TEXT("SourceCodeAccess"));
#endif	//WITH_EDITOR
}

class FOutputDeviceConsole* FWindowsPlatformApplicationMisc::CreateConsoleOutputDevice()
{
	// this is a slightly different kind of singleton that gives ownership to the caller and should not be called more than once

#if WITH_REMOTEWIN_CONSOLE
	FString RemoteConsoleHost;
	if (FParse::Value(FCommandLine::Get(), TEXT("RemoteConsoleHost="), RemoteConsoleHost))
		return new FPlaceholder_WindowsRemoteConsoleOutputDevice(RemoteConsoleHost);
	else
#endif

	if (FParse::Param(FCommandLine::Get(), TEXT("NewConsole")))
		return new FWindowsConsoleOutputDevice2();
	else
		return new FWindowsConsoleOutputDevice();
}

class FOutputDeviceError* FWindowsPlatformApplicationMisc::GetErrorOutputDevice()
{
	static FWindowsApplicationErrorOutputDevice Singleton;
	return &Singleton;
}

class FFeedbackContext* FWindowsPlatformApplicationMisc::GetFeedbackContext()
{
#if WITH_EDITOR
	static FWindowsFeedbackContext Singleton;
	return &Singleton;
#else
	return FPlatformOutputDevices::GetFeedbackContext();
#endif
}

GenericApplication* FWindowsPlatformApplicationMisc::CreateApplication()
{
	if (FParse::Param(FCommandLine::Get(), TEXT("RenderOffScreen")))
	{
		return FNullPlatformApplicationMisc::CreateApplication();
	}

	HICON AppIconHandle = LoadIcon( hInstance, MAKEINTRESOURCE( GetAppIcon() ) );
	if( AppIconHandle == NULL )
	{
		AppIconHandle = LoadIcon( (HINSTANCE)NULL, IDI_APPLICATION ); 
	}

	return FWindowsApplication::CreateWindowsApplication( hInstance, AppIconHandle );
}

void FWindowsPlatformApplicationMisc::RequestMinimize()
{
	::ShowWindow(::GetActiveWindow(), SW_MINIMIZE);
}

bool FWindowsPlatformApplicationMisc::IsThisApplicationForeground()
{
	uint32 ForegroundProcess;
	::GetWindowThreadProcessId(GetForegroundWindow(), (::DWORD *)&ForegroundProcess);
	return (ForegroundProcess == GetCurrentProcessId());
}

int32 FWindowsPlatformApplicationMisc::GetAppIcon()
{
	return IDICON_UEGame;
}

static void WinPumpMessages()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(WinPumpMessages);
	{
		MSG Msg;
		while( PeekMessage(&Msg,NULL,0,0,PM_REMOVE) )
		{
			TranslateMessage( &Msg );
			DispatchMessage( &Msg );
		}
	}
}


void FWindowsPlatformApplicationMisc::PumpMessages(bool bFromMainLoop)
{
	const bool bSetPumpingMessages = !GPumpingMessages;
	if (bSetPumpingMessages)
	{
		GPumpingMessages = true;
	}

	ON_SCOPE_EXIT
	{
		if (bSetPumpingMessages)
		{
			GPumpingMessages = false;
		}
	};

	if (!bFromMainLoop)
	{
		FPlatformMisc::PumpMessagesOutsideMainLoop();
		return;
	}

	GPumpingMessagesOutsideOfMainLoop = false;
	WinPumpMessages();
	FWindowsApplication::StaticPostPumpMessages();

	// Determine if application has focus
	bool bHasFocus = FApp::HasFocus();
	static bool bHadFocus = false;

#if !UE_SERVER
	// For non-editor clients, record if the active window is in focus
	if( bHadFocus != bHasFocus )
	{
		FGenericCrashContext::SetEngineData(TEXT("Platform.AppHasFocus"), bHasFocus ? TEXT("true") : TEXT("false"));
	}
#endif

	bHadFocus = bHasFocus;

	// if its our window, allow sound, otherwise apply multiplier
	FApp::SetVolumeMultiplier( bHasFocus ? 1.0f : FApp::GetUnfocusedVolumeMultiplier() );
}

void FWindowsPlatformApplicationMisc::PreventScreenSaver()
{
	INPUT Input = { 0 };
	Input.type = INPUT_MOUSE;
	Input.mi.dx = 0;
	Input.mi.dy = 0;	
	Input.mi.mouseData = 0;
	Input.mi.dwFlags = MOUSEEVENTF_MOVE;
	Input.mi.time = 0;
	Input.mi.dwExtraInfo = 0; 	
	SendInput(1,&Input,sizeof(INPUT));
}

FLinearColor FWindowsPlatformApplicationMisc::GetScreenPixelColor(const FVector2D& InScreenPos, float /*InGamma*/)
{
	HDC TempDC = GetDC(HWND_DESKTOP);
	COLORREF PixelColorRef = GetPixel(TempDC, (int)InScreenPos.X, (int)InScreenPos.Y);

	ReleaseDC(HWND_DESKTOP, TempDC);

	FColor sRGBScreenColor(
		(uint8)(PixelColorRef & 0xFF),
		(uint8)((PixelColorRef & 0xFF00) >> 8),
		(uint8)((PixelColorRef & 0xFF0000) >> 16),
		255);

	// Assume the screen color is coming in as sRGB space
	return FLinearColor(sRGBScreenColor);
}

void FWindowsPlatformApplicationMisc::SetHighDPIMode()
{
	if (IsHighDPIAwarenessEnabled())
	{
		// Enable HighDPIMode for an application running unattended.
		bool bAllowHighDpiWhenUnattended = false;
		GConfig->GetBool(TEXT("/Script/Engine.UserInterfaceSettings"), TEXT("bAllowHighDpiWhenUnattended"), bAllowHighDpiWhenUnattended, GEngineIni);

		if (!IsRunningCommandlet() && (!FApp::IsUnattended() || bAllowHighDpiWhenUnattended))
		{
			PROCESS_DPI_AWARENESS CurrentAwareness = PROCESS_DPI_UNAWARE;

			GetProcessDpiAwareness(nullptr, &CurrentAwareness);

			if (CurrentAwareness != PROCESS_PER_MONITOR_DPI_AWARE)
			{
				UE_LOGF(LogInit, Log, "Setting process to per monitor DPI aware");
				HRESULT Hr = SetProcessDpiAwareness(PROCESS_PER_MONITOR_DPI_AWARE); // PROCESS_PER_MONITOR_DPI_AWARE_VALUE
																					// We dont care about this warning if we are in any kind of headless mode
				if (Hr != S_OK)
				{
					UE_LOGF(LogInit, Warning, "SetProcessDpiAwareness failed.  Error code %x", Hr);
				}
			}
		}
	}
}

bool FWindowsPlatformApplicationMisc::GetWindowTitleMatchingText(const TCHAR* TitleStartsWith, FString& OutTitle)
{
	bool bWasFound = false;
	WCHAR Buffer[8192];
	// Get the first window so we can start walking the window chain
	HWND hWnd = FindWindowW(NULL,NULL);
	if (hWnd != NULL)
	{
		size_t TitleStartsWithLen = _tcslen(TitleStartsWith);
		do
		{
			GetWindowText(hWnd,Buffer,8192);
			// If this matches, then grab the full text
			if (_tcsnccmp(TitleStartsWith, Buffer, TitleStartsWithLen) == 0)
			{
				OutTitle = Buffer;
				hWnd = NULL;
				bWasFound = true;
			}
			else
			{
				// Get the next window to interrogate
				hWnd = GetWindow(hWnd, GW_HWNDNEXT);
			}
		}
		while (hWnd != NULL);
	}
	return bWasFound;
}

int32 FWindowsPlatformApplicationMisc::GetMonitorDPI(const FMonitorInfo& MonitorInfo)
{
	int32 DisplayDPI = USER_DEFAULT_SCREEN_DPI;

	if (IsHighDPIAwarenessEnabled())
	{
		RECT MonitorDim;
		MonitorDim.left = MonitorInfo.DisplayRect.Left;
		MonitorDim.top = MonitorInfo.DisplayRect.Top;
		MonitorDim.right = MonitorInfo.DisplayRect.Right;
		MonitorDim.bottom = MonitorInfo.DisplayRect.Bottom;

		HMONITOR Monitor = MonitorFromRect(&MonitorDim, MONITOR_DEFAULTTONEAREST);
		if (Monitor)
		{
			DisplayDPI = GetMonitorDPI(Monitor);
		}
	}

	return DisplayDPI;
}

int32 FWindowsPlatformApplicationMisc::GetMonitorDPI(void* NativeHandle)
{
	int32 DisplayDPI = USER_DEFAULT_SCREEN_DPI;

	if (IsHighDPIAwarenessEnabled())
	{
		HMONITOR Monitor = static_cast<HMONITOR>(NativeHandle);
		uint32 DPIX = 0;
		uint32 DPIY = 0;
		if (SUCCEEDED(GetDpiForMonitor(Monitor, MDT_EFFECTIVE_DPI, &DPIX, &DPIY)))
		{
			DisplayDPI = DPIX;
		}
	}

	return DisplayDPI;
}

// Looks for an adapter with the most dedicated video memory
FWindowsPlatformApplicationMisc::FGPUInfo FWindowsPlatformApplicationMisc::GetBestGPUInfo()
{
	const FWindowsGPUInfo Info = FWindowsD3D::GetGPUInfoByDedicatedMemory();
	return FGPUInfo{ Info.VendorId, Info.DeviceId, Info.DedicatedVideoMemory };
}

float FWindowsPlatformApplicationMisc::GetDPIScaleFactorAtPoint(float X, float Y)
{
	float Scale = 1.0f;

	if (IsHighDPIAwarenessEnabled())
	{
		POINT Position = { static_cast<LONG>(X), static_cast<LONG>(Y) };
		HMONITOR Monitor = MonitorFromPoint(Position, MONITOR_DEFAULTTONEAREST);
		if (Monitor)
		{
			uint32 DPIX = 0;
			uint32 DPIY = 0;
			if (SUCCEEDED(GetDpiForMonitor(Monitor, MDT_EFFECTIVE_DPI, &DPIX, &DPIY)))
			{
				Scale = (float)DPIX / USER_DEFAULT_SCREEN_DPI;
			}
		}
	}

	return Scale;
}

// Disabling optimizations helps to reduce the frequency of OpenClipboard failing with error code 0. It still happens
// though only with really large text buffers and we worked around this by changing the editor to use an intermediate
// text buffer for internal operations.
UE_DISABLE_OPTIMIZATION_SHIP

void FWindowsPlatformApplicationMisc::ClipboardCopy(const TCHAR* Str)
{
	if( OpenClipboard(GetActiveWindow()) )
	{
		verify(EmptyClipboard());
		HGLOBAL GlobalMem;
		int32 StrLen = FCString::Strlen(Str);
		GlobalMem = GlobalAlloc( GMEM_MOVEABLE, sizeof(TCHAR)*(StrLen+1) );
		check(GlobalMem);
		TCHAR* Data = (TCHAR*) GlobalLock( GlobalMem );
		FCString::Strncpy( Data, Str, (StrLen+1) );
		GlobalUnlock( GlobalMem );
		if( SetClipboardData( CF_UNICODETEXT, GlobalMem ) == NULL )
			UE_LOGF(LogWindows, Fatal,"SetClipboardData failed with error code %i", (uint32)GetLastError() );
		if (!CloseClipboard())
		{
			UE_LOGF(LogWindows, Warning, "CloseClipboard failed with error code %i", (uint32)GetLastError());
		}
	}
	else
	{
		UE_LOGF(LogWindows, Warning, "OpenClipboard failed with error code %i", (uint32)GetLastError());
	}
}

void FWindowsPlatformApplicationMisc::ClipboardPaste(class FString& Result)
{
	if( OpenClipboard(GetActiveWindow()) )
	{
		HGLOBAL GlobalMem = NULL;
		bool Unicode = 0;
		GlobalMem = GetClipboardData( CF_UNICODETEXT );
		Unicode = 1;
		if( !GlobalMem )
		{
			GlobalMem = GetClipboardData( CF_TEXT );
			Unicode = 0;
		}
		if( !GlobalMem )
		{
			Result = TEXT("");
		}
		else
		{
			void* Data = GlobalLock( GlobalMem );
			check( Data );	
			if( Unicode )
				Result = (TCHAR*) Data;
			else
			{
				ANSICHAR* ACh = (ANSICHAR*) Data;
				int32 i;
				for( i=0; ACh[i]; i++ );
				TArray<TCHAR> Ch;
				Ch.AddUninitialized(i+1);
				for( i=0; i<Ch.Num(); i++ )
					Ch[i]=CharCast<TCHAR>(ACh[i]);
				Result.GetCharArray() = MoveTemp(Ch);
			}
			GlobalUnlock( GlobalMem );
		}
		if (!CloseClipboard())
		{
			UE_LOGF(LogWindows, Warning, "CloseClipboard failed with error code %i", (uint32)GetLastError());
		}
	}
	else 
	{
		Result=TEXT("");
		UE_LOGF(LogWindows, Warning, "OpenClipboard failed with error code %i", (uint32)GetLastError());
	}
}

UE_ENABLE_OPTIMIZATION_SHIP
#undef LOCTEXT_NAMESPACE
