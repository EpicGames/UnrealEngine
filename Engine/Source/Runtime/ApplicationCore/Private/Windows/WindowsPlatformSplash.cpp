// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsPlatformSplash.h"
#include "AutoRTFM.h"
#include "HAL/PlatformMemory.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/AssertionMacros.h"
#include "Math/UnrealMathUtility.h"
#include "HAL/UnrealMemory.h"
#include "CoreGlobals.h"
#include "Misc/Parse.h"
#include "Misc/EngineVersionBase.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "Misc/ScopeLock.h"
#include "Misc/CommandLine.h"
#include "Internationalization/Text.h"
#include "Internationalization/Internationalization.h"
#include "Stats/Stats.h"
#include "Misc/App.h"
#include "Misc/EngineVersion.h"
#include "Misc/ScopeExit.h"
#include "Windows/WindowsApplication.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Misc/EngineBuildSettings.h"

THIRD_PARTY_INCLUDES_START
#pragma warning(push)
#pragma warning(disable:4005)
#include <dwmapi.h>
#include <strsafe.h>
#include <wincodec.h>
#pragma warning(pop)
THIRD_PARTY_INCLUDES_END

#pragma comment( lib, "windowscodecs.lib" )

#ifndef UE_WINDOWS_SPLASH_USE_TEXT_OUTLINE
#define UE_WINDOWS_SPLASH_USE_TEXT_OUTLINE (1)
#endif//UE_WINDOWS_SPLASH_USE_TEXT_OUTLINE 

#if !defined(UE_WINDOWS_SPLASH_ENABLE_DRAG)
#define UE_WINDOWS_SPLASH_ENABLE_DRAG WITH_EDITOR
#endif

static TAutoConsoleVariable<bool> CVarWindowsApplicationShowSplash(
	TEXT("WindowsApplication.ShowSplash"),
	true,
	TEXT("Whether to show the splash screen on Windows (default = true).")
);

/**
 * Splash screen functions and static globals
 */

static HANDLE GSplashScreenThread = NULL;
static HWND GSplashScreenWnd = NULL; 
static HWND GSplashScreenGuard = NULL; 
static FString GSplashScreenFileNames[ SplashImageType::NumImageTypes ];
static HBITMAP GSplashScreenBitmaps[ SplashImageType::NumImageTypes ];
static FIntPoint GSplashScreenPos[ SplashImageType::NumImageTypes ];
static FText GSplashScreenAppName;
static FText GSplashScreenText[ SplashTextType::NumTextTypes ];
static RECT GSplashScreenTextRects[ SplashTextType::NumTextTypes ];
static ESplashTextSizeType GSplashScreenTextSizes[ SplashTextType::NumTextTypes ];
static ESplashTextBrightnessType GSplashScreenTextBrightness[ SplashTextType::NumTextTypes ];
static ESplashTextFlags GSplashScreenTextFlags[ SplashTextType::NumTextTypes ];
static HFONT GSplashScreenSmallTextFontHandle = NULL;
static HFONT GSplashScreenNormalTextFontHandle = NULL;
static HFONT GSplashScreenTitleTextFontHandle = NULL;
static FCriticalSection GSplashScreenSynchronizationObject;
static HANDLE GSplashWindowCreationEvent = INVALID_HANDLE_VALUE;


/**
 * Window's proc for splash screen
 */
LRESULT CALLBACK SplashScreenWindowProc(HWND hWnd, uint32 message, WPARAM wParam, LPARAM lParam)
{
	HDC hdc;
	PAINTSTRUCT ps;

	switch( message )
	{
		case WM_PAINT:
			{
				// We can continue to receive messages for a period after we've torn down the bitmap resource
				if (!GSplashScreenBitmaps[SplashImageType::Background])
				{
					return 0;
				}

				hdc = BeginPaint(hWnd, &ps);

				{
					// Take a critical section since another thread may be trying to set a splash text or image
					FScopeLock ScopeLock( &GSplashScreenSynchronizationObject );

					// Draw splash bitmap
					for (int32 i = 0; i < SplashImageType::NumImageTypes; ++i)
					{
						if (HBITMAP BitmapHandle = GSplashScreenBitmaps[i])
						{
							const FIntPoint Pos = GSplashScreenPos[i];
							DrawState(hdc, DSS_NORMAL, NULL, (LPARAM)BitmapHandle, 0, Pos.X, Pos.Y, 0, 0, DST_BITMAP);
						}
					}

					// Draw splash text
					for( int32 CurTypeIndex = 0; CurTypeIndex < SplashTextType::NumTextTypes; ++CurTypeIndex )
					{
						const FText& SplashText = GSplashScreenText[ CurTypeIndex ];
						const RECT& TextRect = GSplashScreenTextRects[ CurTypeIndex ];

						if( !SplashText.IsEmpty() )
						{
							if ( GSplashScreenTextSizes[CurTypeIndex] == ESplashTextSizeType::Normal )
							{
								SelectObject( hdc, GSplashScreenNormalTextFontHandle );
							}
							else if ( GSplashScreenTextSizes[CurTypeIndex] == ESplashTextSizeType::Title )
							{
								SelectObject( hdc, GSplashScreenTitleTextFontHandle );
							}
							else
							{
								SelectObject( hdc, GSplashScreenSmallTextFontHandle );
							}

							// Alignment
							SetTextAlign( hdc, TA_LEFT | TA_TOP | TA_NOUPDATECP );

							SetBkColor( hdc, 0x00000000 );
							SetBkMode( hdc, TRANSPARENT );

							RECT ClientRect;
							GetClientRect( hWnd, &ClientRect );

							FString String = SplashText.ToString();
							const ESplashTextFlags TextFlags = GSplashScreenTextFlags[CurTypeIndex];

#if UE_WINDOWS_SPLASH_USE_TEXT_OUTLINE
							// Draw background text passes
							const int32 NumBGPasses = 8;
							for( int32 CurBGPass = 0; CurBGPass < NumBGPasses; ++CurBGPass )
							{
								int32 BGXOffset, BGYOffset;
								switch( CurBGPass )
								{
									default:
									case 0:	BGXOffset = -1; BGYOffset =  0; break;
									case 1:	BGXOffset = -1; BGYOffset = -1; break;
									case 2:	BGXOffset =  0; BGYOffset = -1; break;
									case 3:	BGXOffset =  1; BGYOffset = -1; break;
									case 4:	BGXOffset =  1; BGYOffset =  0; break;
									case 5:	BGXOffset =  1; BGYOffset =  1; break;
									case 6:	BGXOffset =  0; BGYOffset =  1; break;
									case 7:	BGXOffset = -1; BGYOffset =  1; break;
								}

								SetTextColor( hdc, 0x00000000 );

								RECT OffsetRect = RECT(
									TextRect.left + BGXOffset,
									TextRect.top + BGYOffset,
									TextRect.right + BGXOffset,
									TextRect.bottom + BGYOffset
								);

								if (EnumHasAllFlags(TextFlags, ESplashTextFlags::WordWrap))
								{
									DrawTextW(hdc, *String, String.Len(), &OffsetRect, DT_WORDBREAK | DT_LEFT | DT_TOP | DT_NOPREFIX);
								}
								else
								{
									TextOut(hdc, OffsetRect.left, OffsetRect.top, *String, String.Len());
								}
							}
#endif // UE_WINDOWS_SPLASH_USE_TEXT_OUTLINE
							
							// Draw foreground text pass
							if( GSplashScreenTextBrightness[CurTypeIndex] == ESplashTextBrightnessType::Low )
							{
								SetTextColor( hdc, RGB(160, 160, 160) );
							}
							else if( GSplashScreenTextBrightness[CurTypeIndex] == ESplashTextBrightnessType::Medium )
							{
								SetTextColor( hdc, RGB(180, 180, 180) );
							}
							else
							{
								SetTextColor(hdc, RGB(255, 255, 255));
							}

							if (EnumHasAllFlags(TextFlags, ESplashTextFlags::WordWrap))
							{
								RECT DrawRect = TextRect;
								DrawTextW(hdc, *String, String.Len(), &DrawRect, DT_WORDBREAK | DT_LEFT | DT_TOP);
							}
							else
							{
								TextOut(hdc, TextRect.left, TextRect.top, *String, String.Len());
							}
						}
					}
				}

				EndPaint(hWnd, &ps);
			}
			break;

		case WM_DESTROY:
			PostQuitMessage(0);
			break;

#if UE_WINDOWS_SPLASH_ENABLE_DRAG
		case WM_NCHITTEST:
			{
				// Report client area as non-client area to allow dragging the splashscreen around.
				LRESULT Result = DefWindowProc(hWnd, message, wParam, lParam);
				if (Result == HTCLIENT)
				{
					Result = HTCAPTION;
				}
				return Result;
			}
#endif

		default:
			return DefWindowProc(hWnd, message, wParam, lParam);
	}

	return 0;
}

/**
 * Helper function to load the splash screen bitmap
 * This replaces the old win32 api call to ::LoadBitmap which couldn't handle more modern BMP formats containing
 *  - colour space information or newer format extensions.
 * This code is largely taken from the WicViewerGDI sample provided by Microsoft on MSDN.
 */
HBITMAP LoadSplashBitmap(SplashImageType::Type Type)
{
	if (GSplashScreenFileNames[Type].IsEmpty())
	{
		return nullptr;
	}

	HRESULT hr = CoInitialize(NULL);

	// The factory pointer
	IWICImagingFactory *Factory = NULL;

	// Create the COM imaging factory
	hr = CoCreateInstance(
		CLSID_WICImagingFactory,
		NULL,
		CLSCTX_INPROC_SERVER,
		IID_PPV_ARGS(&Factory)
		);

	// Decode the source image to IWICBitmapSource
	IWICBitmapDecoder *Decoder = NULL;

	// Create a decoder
	hr = Factory->CreateDecoderFromFilename(
		(LPCTSTR)*GSplashScreenFileNames[Type], // Image to be decoded
		NULL,                            // Do not prefer a particular vendor
		GENERIC_READ,                    // Desired read access to the file
		WICDecodeMetadataCacheOnDemand,  // Cache metadata when needed
		&Decoder                        // Pointer to the decoder
		);

	IWICBitmapFrameDecode *Frame = NULL;

	// Retrieve the first frame of the image from the decoder
	if (SUCCEEDED(hr))
	{
		hr = Decoder->GetFrame(0, &Frame);
	}

	// Retrieve IWICBitmapSource from the frame
	IWICBitmapSource *OriginalBitmapSource = NULL;
	if (SUCCEEDED(hr))
	{
		hr = Frame->QueryInterface( IID_IWICBitmapSource, reinterpret_cast<void **>(&OriginalBitmapSource));
	}

	IWICBitmapSource *ToRenderBitmapSource = NULL;

	// convert the pixel format
	if (SUCCEEDED(hr))
	{
		IWICFormatConverter *Converter = NULL;

		hr = Factory->CreateFormatConverter(&Converter);

		// Format convert to 32bppBGR
		if (SUCCEEDED(hr))
		{
			hr = Converter->Initialize(
				Frame,                          // Input bitmap to convert
				GUID_WICPixelFormat32bppBGR,     // Destination pixel format
				WICBitmapDitherTypeNone,         // Specified dither patterm
				NULL,                            // Specify a particular palette 
				0.f,                             // Alpha threshold
				WICBitmapPaletteTypeCustom       // Palette translation type
				);

			// Store the converted bitmap if successful
			if (SUCCEEDED(hr))
			{
				hr = Converter->QueryInterface(IID_PPV_ARGS(&ToRenderBitmapSource));
			}
		}

		Converter->Release();
	}

	// Create a DIB from the converted IWICBitmapSource
	HBITMAP hDIBBitmap = 0;
	if (SUCCEEDED(hr))
	{
		// Get image attributes and check for valid image
		UINT width = 0;
		UINT height = 0;

		void *ImageBits = NULL;
	
		// Check BitmapSource format
		WICPixelFormatGUID pixelFormat;
		hr = ToRenderBitmapSource->GetPixelFormat(&pixelFormat);

		if (SUCCEEDED(hr))
		{
			hr = (pixelFormat == GUID_WICPixelFormat32bppBGR) ? S_OK : E_FAIL;
		}

		if (SUCCEEDED(hr))
		{
			hr = ToRenderBitmapSource->GetSize(&width, &height);
		}

		// Create a DIB section based on Bitmap Info
		// BITMAPINFO Struct must first be setup before a DIB can be created.
		// Note that the height is negative for top-down bitmaps
		if (SUCCEEDED(hr))
		{
			BITMAPINFO bminfo;
			ZeroMemory(&bminfo, sizeof(bminfo));
			bminfo.bmiHeader.biSize         = sizeof(BITMAPINFOHEADER);
			bminfo.bmiHeader.biWidth        = width;
			bminfo.bmiHeader.biHeight       = -(LONG)height;
			bminfo.bmiHeader.biPlanes       = 1;
			bminfo.bmiHeader.biBitCount     = 32;
			bminfo.bmiHeader.biCompression  = BI_RGB;

			// Get a DC for the full screen
			HDC hdcScreen = GetDC(NULL);

			hr = hdcScreen ? S_OK : E_FAIL;

			// Release the previously allocated bitmap 
			if (SUCCEEDED(hr))
			{
				if (hDIBBitmap)
				{
					ensure(DeleteObject(hDIBBitmap));
				}

				hDIBBitmap = CreateDIBSection(hdcScreen, &bminfo, DIB_RGB_COLORS, &ImageBits, NULL, 0);

				ReleaseDC(NULL, hdcScreen);

				hr = hDIBBitmap ? S_OK : E_FAIL;
			}
		}

		UINT cbStride = 0;
		if (SUCCEEDED(hr))
		{
			// Size of a scan line represented in bytes: 4 bytes each pixel
			hr = UIntMult(width, sizeof(DWORD), &cbStride);
		}
	
		UINT cbImage = 0;
		if (SUCCEEDED(hr))
		{
			// Size of the image, represented in bytes
			hr = UIntMult(cbStride, height, &cbImage);
		}

		// Extract the image into the HBITMAP    
		if (SUCCEEDED(hr) && ToRenderBitmapSource)
		{
			hr = ToRenderBitmapSource->CopyPixels(
				NULL,
				cbStride,
				cbImage, 
				reinterpret_cast<BYTE *> (ImageBits));
		}

		// Image Extraction failed, clear allocated memory
		if (FAILED(hr) && hDIBBitmap)
		{
			ensure(DeleteObject(hDIBBitmap));
			hDIBBitmap = NULL;
		}
	}

	if ( OriginalBitmapSource )
	{
		OriginalBitmapSource->Release();
	}

	if ( ToRenderBitmapSource )
	{
		ToRenderBitmapSource->Release();
	}

	if ( Decoder )
	{
		Decoder->Release();
	}

	if ( Frame )
	{
		Frame->Release();
	}

	if ( Factory )
	{
		Factory->Release();
	}

	return hDIBBitmap;
}

/**
 * Splash screen thread entry function
 */
uint32 WINAPI StartSplashScreenThread( LPVOID unused )
{
	WNDCLASS wc;
	wc.style       = CS_HREDRAW | CS_VREDRAW; 
	wc.lpfnWndProc = (WNDPROC) SplashScreenWindowProc; 
	wc.cbClsExtra  = 0; 
	wc.cbWndExtra  = 0; 
	wc.hInstance   = hInstance; 

	wc.hIcon       = LoadIcon(hInstance, MAKEINTRESOURCE(FPlatformApplicationMisc::GetAppIcon()));
	if(wc.hIcon == NULL)
	{
		wc.hIcon   = LoadIcon((HINSTANCE) NULL, IDI_APPLICATION); 
	}

	wc.hCursor     = LoadCursor((HINSTANCE) NULL, IDC_ARROW); 
	wc.hbrBackground = (HBRUSH)GetStockObject(WHITE_BRUSH);
	wc.lpszMenuName  = NULL;
	wc.lpszClassName = TEXT("SplashScreenClass"); 

	BITMAP bm;
	FMemory::Memzero(bm);

	const bool bAllowFading = false;
	{
		// Ensure that once this scope exits, we signal the main thread that the code constructing the splash window was executed (successfully or not)
		// so the main thread can read GSplashScreenWnd value and act properly to resume the execution.
		ON_SCOPE_EXIT { ::SetEvent(GSplashWindowCreationEvent); };
	
		if(!RegisterClass(&wc)) 
		{
			return 0; 
		} 

		// Load splash screen image, display it and handle all window's messages
		for (int32 i = 0; i < SplashImageType::NumImageTypes; ++i)
		{
			GSplashScreenBitmaps[i] = LoadSplashBitmap(static_cast<SplashImageType::Type>(i));
		}

		if (GSplashScreenBitmaps[SplashImageType::Background])
		{
			GetObjectW(GSplashScreenBitmaps[SplashImageType::Background], sizeof(bm), &bm);
			const int32 WindowWidth = bm.bmWidth;
			const int32 WindowHeight = bm.bmHeight;
			int32 ScreenPosX = (GetSystemMetrics(SM_CXSCREEN) - WindowWidth) / 2;
			int32 ScreenPosY = (GetSystemMetrics(SM_CYSCREEN) - WindowHeight) / 2;

			// Force the editor splash screen to show up in the taskbar and alt-tab lists
			uint32 dwWindowStyle = GIsEditor ? WS_EX_APPWINDOW : WS_EX_TOOLWINDOW;
			if( bAllowFading )
			{
				dwWindowStyle |= WS_EX_LAYERED;
			}

			GSplashScreenWnd = CreateWindowEx(
				dwWindowStyle,
				wc.lpszClassName, 
				TEXT("SplashScreen"),
				WS_POPUP,
				ScreenPosX,
				ScreenPosY,
				WindowWidth,
				WindowHeight,
				(HWND) NULL,
				(HMENU) NULL,
				hInstance,
				(LPVOID) NULL);

#if defined(NTDDI_WIN10_CO) && WDK_NTDDI_VERSION >= NTDDI_WIN10_CO
			// Enable W11 window style (drop shadow, thin border, slightly rounded)
			constexpr DWM_WINDOW_CORNER_PREFERENCE Corners = DWMWCP_ROUND;
			DwmSetWindowAttribute(GSplashScreenWnd, DWMWA_WINDOW_CORNER_PREFERENCE, &Corners, sizeof(Corners));
#endif
		}
	}

	if (GSplashScreenWnd) // also implies GSplashScreenBitmap is not null.
	{
		check(GSplashScreenBitmaps[SplashImageType::Background]);
		if (bAllowFading)
		{
			// Set window to fully transparent to start out
			SetLayeredWindowAttributes( GSplashScreenWnd, 0, 0, LWA_ALPHA );
		}

		// Setup font
		{
			HFONT SystemFontHandle = ( HFONT )GetStockObject( DEFAULT_GUI_FONT );

			// Create small font
			{
				LOGFONT MyFont;
				FMemory::Memzero( &MyFont, sizeof( MyFont ) );
				GetObjectW( SystemFontHandle, sizeof( MyFont ), &MyFont );
				MyFont.lfHeight = 11;
				MyFont.lfQuality = CLEARTYPE_QUALITY;
				//StringCchCopy(MyFont.lfFaceName, LF_FACESIZE, TEXT("Roboto"));
				GSplashScreenSmallTextFontHandle = CreateFontIndirect( &MyFont );
				if( GSplashScreenSmallTextFontHandle == NULL )
				{
					// Couldn't create font, so just use a system font
					GSplashScreenSmallTextFontHandle = SystemFontHandle;
				}
			}

			// Create normal font
			{
				LOGFONT MyFont;
				FMemory::Memzero( &MyFont, sizeof( MyFont ) );
				GetObjectW( SystemFontHandle, sizeof( MyFont ), &MyFont );
				MyFont.lfHeight = 12;
				MyFont.lfQuality = CLEARTYPE_QUALITY;
			//	StringCchCopy(MyFont.lfFaceName, LF_FACESIZE, TEXT("Roboto"));
				GSplashScreenNormalTextFontHandle = CreateFontIndirect( &MyFont );
				if( GSplashScreenNormalTextFontHandle == NULL )
				{
					// Couldn't create font, so just use a system font
					GSplashScreenNormalTextFontHandle = SystemFontHandle;
				}
			}

			// Create title font
			{
				LOGFONT MyFont;
				FMemory::Memzero(&MyFont, sizeof( MyFont ));
				GetObjectW(SystemFontHandle, sizeof( MyFont ), &MyFont);
				MyFont.lfHeight = GIsEditor ? 18 : 28;
				MyFont.lfWeight = FW_BOLD;
				MyFont.lfQuality = CLEARTYPE_QUALITY;
			//	StringCchCopy(MyFont.lfFaceName, LF_FACESIZE, TEXT("Roboto"));
				GSplashScreenTitleTextFontHandle = CreateFontIndirect(&MyFont);
				if ( GSplashScreenTitleTextFontHandle == NULL )
				{
					// Couldn't create font, so just use a system font
					GSplashScreenTitleTextFontHandle = SystemFontHandle;
				}
			}
		}

		// Setup bounds for game name

		if (GIsEditor)
		{
			GSplashScreenTextRects[SplashTextType::GameName].top = bm.bmHeight - 60;
			GSplashScreenTextRects[SplashTextType::GameName].bottom = bm.bmHeight - 45;
			GSplashScreenTextRects[SplashTextType::GameName].left = 10;
			GSplashScreenTextRects[SplashTextType::GameName].right = bm.bmWidth - 20;
		}
		else
		{
			GSplashScreenTextRects[SplashTextType::GameName].top = bm.bmHeight - 45;
			GSplashScreenTextRects[SplashTextType::GameName].bottom = bm.bmHeight - 6;
			GSplashScreenTextRects[SplashTextType::GameName].left = 10;
			GSplashScreenTextRects[SplashTextType::GameName].right = bm.bmWidth - 20;
		}

		GSplashScreenTextSizes[ SplashTextType::GameName ] = ESplashTextSizeType::Title;
		GSplashScreenTextBrightness[ SplashTextType::GameName ] = ESplashTextBrightnessType::High;
		
		// Setup bounds for version info text 1
		GSplashScreenTextRects[ SplashTextType::VersionInfo1 ].top = bm.bmHeight - 40;
		GSplashScreenTextRects[ SplashTextType::VersionInfo1 ].bottom = bm.bmHeight - 20;
		GSplashScreenTextRects[ SplashTextType::VersionInfo1 ].left = 10;
		GSplashScreenTextRects[ SplashTextType::VersionInfo1 ].right = bm.bmWidth - 20;
		GSplashScreenTextSizes[ SplashTextType::VersionInfo1 ] = ESplashTextSizeType::Normal;
		GSplashScreenTextBrightness[ SplashTextType::VersionInfo1 ] = ESplashTextBrightnessType::Low;

		// Setup bounds for copyright info text
		GSplashScreenTextRects[ SplashTextType::CopyrightInfo ].top = bm.bmHeight - 16;
		GSplashScreenTextRects[ SplashTextType::CopyrightInfo ].bottom = bm.bmHeight - 6;
		GSplashScreenTextRects[ SplashTextType::CopyrightInfo ].left = bm.bmWidth - 180;
		GSplashScreenTextRects[ SplashTextType::CopyrightInfo ].right = bm.bmWidth - 20;
		GSplashScreenTextSizes[ SplashTextType::CopyrightInfo ] = ESplashTextSizeType::Small;
		GSplashScreenTextBrightness[ SplashTextType::CopyrightInfo ] = ESplashTextBrightnessType::Low;

		// Setup bounds for startup progress text
		GSplashScreenTextRects[ SplashTextType::StartupProgress ].top = bm.bmHeight - 20;
		GSplashScreenTextRects[ SplashTextType::StartupProgress ].bottom = bm.bmHeight;
		GSplashScreenTextRects[ SplashTextType::StartupProgress ].left = 10;
		GSplashScreenTextRects[ SplashTextType::StartupProgress ].right = bm.bmWidth - 20;
		GSplashScreenTextSizes[ SplashTextType::StartupProgress ] = ESplashTextSizeType::Normal;
		GSplashScreenTextBrightness[ SplashTextType::StartupProgress ] = ESplashTextBrightnessType::Low;

		SetWindowText(GSplashScreenWnd, *GSplashScreenAppName.ToString());
		ShowWindow(GSplashScreenWnd, SW_SHOW); 
		UpdateWindow(GSplashScreenWnd); 
		 
		const double FadeStartTime = FPlatformTime::Seconds();
		const float FadeDuration = 0.2f;
		int32 CurrentOpacityByte = 0;

		MSG message;
		bool bIsSplashFinished = false;
		while (!bIsSplashFinished)
		{
			if( PeekMessage(&message, NULL, 0, 0, PM_REMOVE) )
			{
				TranslateMessage(&message);
				DispatchMessage(&message);

				if( message.message == WM_QUIT )
				{
					bIsSplashFinished = true;
				}
			}

			// Update window opacity
			CA_SUPPRESS(6239)
			if( bAllowFading && CurrentOpacityByte < 255 )
			{
				// Set window to fully transparent to start out
				const float TimeSinceFadeStart = (float)( FPlatformTime::Seconds() - FadeStartTime );
				const float FadeAmount = FMath::Clamp( TimeSinceFadeStart / FadeDuration, 0.0f, 1.0f );
				const int32 NewOpacityByte = (int32)(255 * FadeAmount);
				if( NewOpacityByte != CurrentOpacityByte )
				{
					CurrentOpacityByte = NewOpacityByte;
					SetLayeredWindowAttributes( GSplashScreenWnd, 0, (BYTE)CurrentOpacityByte, LWA_ALPHA );
				}

				// We're still fading, but still yield a timeslice
				FPlatformProcess::Sleep( 0.0f );
			}
			else
			{
				// Give up some time
				FPlatformProcess::Sleep( 1.0f / 60.0f );
			}
		}

		for (int32 i = 0; i < SplashImageType::NumImageTypes; ++i)
		{
			if (GSplashScreenBitmaps[i])
			{
				ensure(DeleteObject(GSplashScreenBitmaps[i]));
				GSplashScreenBitmaps[i] = NULL;
			}
		}
	}

	UnregisterClass(wc.lpszClassName, hInstance);
	return 0;
}

/**
 * Sets the text displayed on the splash screen (for startup/loading progress)
 *
 * @param	InType		Type of text to change
 * @param	InText		Text to display
 */
static void StartSetSplashText( const SplashTextType::Type InType, const TCHAR* InText )
{
	// If we've already been set don't init the string with the default
	if (GSplashScreenText[InType].IsEmpty())
	{
		// Only allow copyright text displayed while loading the game.  Editor displays all.
		GSplashScreenText[InType] = FText::FromString(InText);
	}
}

void FWindowsPlatformSplash::Show()
{
	if(!GSplashScreenThread && FParse::Param(FCommandLine::Get(),TEXT("NOSPLASH")) != true && CVarWindowsApplicationShowSplash->GetBool())
	{
		const FText GameName = FText::FromString( FApp::GetProjectName() );

		const TCHAR* SplashImage = GIsEditor ?  TEXT("EdSplash") : TEXT("Splash");

		// make sure a splash was found
		FString SplashPath;
		bool IsCustom;
		if ( GetSplashPath(SplashImage, SplashPath, IsCustom ) == true )
		{
			// Don't set the game name if the splash screen is custom.
		
			// In the editor, we'll display loading info
			if( GIsEditor )
			{
				// Set initial startup progress info
				{
					StartSetSplashText( SplashTextType::StartupProgress,
						*NSLOCTEXT("UnrealEd", "SplashScreen_InitialStartupProgress", "Loading..." ).ToString() );
				}

				// Set version info
				{
					const FText Version = FText::FromString( FEngineVersion::Current().ToString( FEngineBuildSettings::IsPerforceBuild() ? EVersionComponent::Branch : EVersionComponent::Patch ) );

					FText AppName;
					FText VersionInfo = FText::Format(NSLOCTEXT("UnrealEd", "UnrealEdTitleWithVersion_F", "Unreal Editor {0}"), Version);
					if( GameName.IsEmpty() )
					{
						AppName = NSLOCTEXT( "UnrealEd", "UnrealEdTitleNoGameName_F", "Unreal Editor" );
					}
					else
					{
						AppName = FText::Format( NSLOCTEXT( "UnrealEd", "UnrealEdTitle_F", "Unreal Editor - {0}" ), GameName );
					}

					StartSetSplashText(SplashTextType::GameName, *AppName.ToString());
					StartSetSplashText(SplashTextType::VersionInfo1, *VersionInfo.ToString());

					// Change the window text (which will be displayed in the taskbar)
					GSplashScreenAppName = AppName;
				}

				// Display copyright information in editor splash screen
				{
					const FString CopyrightInfo = NSLOCTEXT( "UnrealEd", "SplashScreen_CopyrightInfo", "Copyright \u00a9   Epic Games, Inc.   All rights reserved." ).ToString();
					StartSetSplashText( SplashTextType::CopyrightInfo, *CopyrightInfo );
				}
			}
			else if(!IsCustom)
			{
				StartSetSplashText(SplashTextType::GameName, *GameName.ToString());
			}
			// Spawn a window to receive the Z-order swap when the splashscreen is destroyed.
			// This will prevent the main window from being sent to the background when the splash window closes.
			GSplashScreenGuard = CreateWindow(
				TEXT("STATIC"), 
				TEXT("SplashScreenGuard"),
				0,
				0,
				0,
				0,
				0,
				HWND_MESSAGE,
				(HMENU) NULL,
				hInstance,
				(LPVOID) NULL); 

			if (GSplashScreenGuard)
			{
				ShowWindow(GSplashScreenGuard, SW_SHOW); 
			}

			GSplashWindowCreationEvent = CreateEvent(NULL, FALSE, FALSE, NULL);
			GSplashScreenFileNames[SplashImageType::Background] = MoveTemp(SplashPath);
			DWORD ThreadID = 0;
			GSplashScreenThread = CreateThread(NULL, 128 * 1024, (LPTHREAD_START_ROUTINE)StartSplashScreenThread, (LPVOID)NULL, STACK_SIZE_PARAM_IS_A_RESERVATION, &ThreadID);
#if	STATS
			AutoRTFM::UnreachableIfClosed("TODO: Could remove if we marked `FWindowsPlatformSplash` as `AUTORTFM_DISABLE`!");
			FStartupMessages::Get().AddThreadMetadata( FName( "SplashScreenThread" ), ThreadID );
#endif // STATS
		}
	}
}

void FWindowsPlatformSplash::Hide()
{
	if(GSplashScreenThread)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FWindowsPlatformSplash::Hide);

		// The call to Hide() can be executed on the main thread before GSplashScreenWnd is created on the splash screen thread. Wait until the
		// the thread had a chance to create the splash screen window and then notify the thread back with a WM_DESTROY message that will end
		// the loop in that thread and prevent deadlocking the engine.
		WaitForSingleObject(GSplashWindowCreationEvent, INFINITE);
		if (GSplashScreenWnd)
		{
			// Send message to splash screen window to close itself
			PostMessageW(GSplashScreenWnd, WM_CLOSE, 0, 0);
		}

		// Wait for splash screen thread to finish
		WaitForSingleObject(GSplashScreenThread, INFINITE);

		// Clean up
		CloseHandle(GSplashScreenThread);
		GSplashScreenThread = NULL;
		GSplashScreenWnd = NULL;

		// Close the Z-Order guard window
		if ( GSplashScreenGuard )
		{
			PostMessageW(GSplashScreenGuard, WM_DESTROY, 0, 0);
			GSplashScreenGuard = NULL;
		}
	}
}


bool FWindowsPlatformSplash::IsShown()
{
	return (GSplashScreenThread != nullptr);
}

void FWindowsPlatformSplash::RegisterUserSplashText(const SplashTextType::Type InType, const FPlatformRect& InRect,
	const ESplashTextSizeType InSize, const ESplashTextBrightnessType InBrightness, const ESplashTextFlags InFlags)
{
	check(InType >= SplashTextType::Type::User1);
	check(!GSplashScreenThread);

	GSplashScreenTextRects[InType] = RECT(InRect.Left, InRect.Top, InRect.Right, InRect.Bottom);
	GSplashScreenTextSizes[InType] = InSize;
	GSplashScreenTextBrightness[InType] = InBrightness;
	GSplashScreenTextFlags[InType] = InFlags;
}

void FWindowsPlatformSplash::RegisterUserSplashImage(const SplashImageType::Type InType, const FIntPoint& InPos,
	const TCHAR* InFilename)
{
	check(InType >= SplashImageType::User1);
	check(!GSplashScreenThread);

	bool bCustom;
	GetSplashPath(InFilename, GSplashScreenFileNames[InType], bCustom, false);
	GSplashScreenPos[InType] = InPos;
}

void FWindowsPlatformSplash::SetProgress(int ProgressPercent)
{
	extern FWindowsApplication* WindowsApplication;

	if (ProgressPercent == 100)
	{
		WindowsApplication->GetTaskbarList()->SetProgressState(GSplashScreenWnd, ETaskbarProgressState::NoProgress);
	}
	else
	{
		WindowsApplication->GetTaskbarList()->SetProgressValue(GSplashScreenWnd, ProgressPercent, 100);
	}
}

void FWindowsPlatformSplash::SetSplashText( const SplashTextType::Type InType, const FText& InText )
{
	// We only want to bother drawing startup progress in the editor, since this information is
	// not interesting to an end-user (also, it's not usually localized properly.)
	if( GSplashScreenThread )
	{
		// Only allow copyright text displayed while loading the game.  Editor displays all.
		if( InType == SplashTextType::CopyrightInfo || GIsEditor )
		{
			// Take a critical section since the splash thread may already be repainting using this text
			FScopeLock ScopeLock(&GSplashScreenSynchronizationObject);

			bool bWasUpdated = false;
			{
				// Update splash text
				if (!InText.IdenticalTo(GSplashScreenText[InType], ETextIdenticalModeFlags::LexicalCompareInvariants))
				{
					GSplashScreenText[ InType ] = InText;
					bWasUpdated = true;
				}
			}

			if( bWasUpdated )
			{
				// Repaint the window
				const BOOL bErase = false;
				InvalidateRect( GSplashScreenWnd, &GSplashScreenTextRects[ InType ], bErase );
			}
		}
	}
	else // We haven't started drawing yet we can set text ahead of time to show
	{
		// Update splash text
		GSplashScreenText[InType] = InText;
	}
}

void FWindowsPlatformSplash::SetSplashImage(const SplashImageType::Type InType, const TCHAR* InFilename)
{
	if (GSplashScreenThread)
	{
		const HBITMAP ImageHandle = LoadSplashBitmap(InType);
		if (!ImageHandle)
		{
			return;
		}

		// Take a critical section since the splash thread may already be repainting using this image
		FScopeLock ScopeLock(&GSplashScreenSynchronizationObject);

        if (GSplashScreenBitmaps[InType])
        {
            ensure(DeleteObject(GSplashScreenBitmaps[InType]));
        }

		GSplashScreenBitmaps[InType] = ImageHandle;

		SIZE Size;
		if (GetBitmapDimensionEx(ImageHandle, &Size))
		{
			// Repaint the window
			RECT Rect;
			Rect.left = GSplashScreenPos[InType].X;
			Rect.top = GSplashScreenPos[InType].Y;
			Rect.right = GSplashScreenPos[InType].X + Size.cx;
			Rect.bottom = GSplashScreenPos[InType].Y + Size.cy;

			constexpr BOOL bErase = false;
			InvalidateRect(GSplashScreenWnd, &Rect, bErase);
		}
	}
	else // No drawing is running yet so we can just update the path here
	{
		bool bCustom;
		GetSplashPath(InFilename, GSplashScreenFileNames[InType], bCustom, false);
	}
}

APPLICATIONCORE_API HWND GetSplashScreenWindowHandle()
{
	return GSplashScreenWnd;
}

#include "Windows/HideWindowsPlatformTypes.h"
