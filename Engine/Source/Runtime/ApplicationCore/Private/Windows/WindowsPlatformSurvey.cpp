// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsPlatformSurvey.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformProcess.h"
#include "Logging/LogMacros.h"
#include "HAL/PlatformTime.h"
#include "Math/UnrealMathUtility.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UnrealTemplate.h"
#include "HAL/FileManager.h"
#include "Containers/UnrealString.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "CoreGlobals.h"
#include "GenericPlatform/GenericApplication.h"

#define USING_WINSAT_API	1
#define USING_POWRPROF		1

#include "Windows/AllowWindowsPlatformTypes.h"
	#include <ShlObj.h>
#if USING_WINSAT_API
	#include <winsatcominterfacei.h>
#endif
#if USING_POWRPROF
	#include <powrprof.h>
	#pragma comment( lib, "PowrProf.lib" )
#endif

THIRD_PARTY_INCLUDES_START
	#include <subauth.h>
THIRD_PARTY_INCLUDES_END

#include "SynthBenchmark.h"

#ifndef PROCESSOR_POWER_INFORMATION
typedef struct _PROCESSOR_POWER_INFORMATION {  
	ULONG Number;
	ULONG MaxMhz;
	ULONG CurrentMhz;
	ULONG MhzLimit;
	ULONG MaxIdleState;
	ULONG CurrentIdleState;
} PROCESSOR_POWER_INFORMATION, *PPROCESSOR_POWER_INFORMATION;
#endif

void GetOSVersionLabels(const SYSTEM_INFO& SystemInfo, FHardwareSurveyResults& OutResults);
void WriteFStringToResults(TCHAR* OutBuffer, const FString& InString);

bool FWindowsPlatformSurvey::GetSurveyResults( FHardwareSurveyResults& OutResults, bool bWait )
{
	FMemory::Memset(&OutResults, 0, sizeof(FHardwareSurveyResults));
	WriteFStringToResults(OutResults.Platform, TEXT("Windows"));
	
	// Get memory
	FPlatformMemoryStats PlatformMemoryStats = FPlatformMemory::GetStats();
	OutResults.MemoryMB = PlatformMemoryStats.TotalPhysicalGB * 1024;

	// Identify Display Devices
	FDisplayMetrics DisplayMetrics;
	FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);

	OutResults.DisplayCount = FMath::Min(DisplayMetrics.MonitorInfo.Num(), FHardwareSurveyResults::MaxDisplayCount);

	for (uint32 DisplayIndex = 0; DisplayIndex < OutResults.DisplayCount; ++DisplayIndex)
	{
		FMonitorInfo& Info = DisplayMetrics.MonitorInfo[DisplayIndex];
		OutResults.Displays[DisplayIndex].CurrentModeHeight = Info.NativeHeight;
		OutResults.Displays[DisplayIndex].CurrentModeWidth = Info.NativeWidth;
	}

	// Get system info
	SYSTEM_INFO SystemInfo;
	GetNativeSystemInfo(&SystemInfo);

	// Get CPU count from SystemInfo
	OutResults.CPUCount = SystemInfo.dwNumberOfProcessors;

	ISynthBenchmark::Get().Run(OutResults.SynthBenchmark, true, 5.f);

	FString RHIName;
	ISynthBenchmark::Get().GetRHIInfo(OutResults.RHIAdapter, RHIName);
	WriteFStringToResults(OutResults.RenderingAPI, RHIName);

	// Get CPU speed
	if (OutResults.CPUCount > 0)
	{
#if USING_POWRPROF
		PROCESSOR_POWER_INFORMATION* PowerInfo = new PROCESSOR_POWER_INFORMATION[OutResults.CPUCount];
		if(PowerInfo != NULL)
		{
			uint32 PowerInfoSize = sizeof(PROCESSOR_POWER_INFORMATION) * OutResults.CPUCount;
			FMemory::Memset(PowerInfo, 0, PowerInfoSize);

			NTSTATUS NTStatus = CallNtPowerInformation(ProcessorInformation, NULL, 0L, (PVOID)PowerInfo, PowerInfoSize);
			if (NT_SUCCESS(NTStatus))
			{
				OutResults.CPUClockGHz = 0.001f * (float)PowerInfo[0].MaxMhz;
			}
			else
			{
				OutResults.ErrorCount++;
				WriteFStringToResults(OutResults.LastSurveyError, TEXT("CallNtPowerInformation() failed to get processor power info"));
				WriteFStringToResults(OutResults.LastSurveyErrorDetail, FString::Printf(TEXT("NTSTATUS: 0x%0x"), (uint32)NTStatus));
			}
			delete [] PowerInfo;
		}
		else
		{
			OutResults.ErrorCount++;
			WriteFStringToResults(OutResults.LastSurveyError, TEXT("FWindowsPlatformSurvey::TickSurveyHardware() failed to get processor count"));
			WriteFStringToResults(OutResults.LastSurveyErrorDetail, TEXT(""));
		}
#endif
	}
	else
	{
		OutResults.ErrorCount++;
		WriteFStringToResults(OutResults.LastSurveyError, TEXT("FWindowsPlatformSurvey::TickSurveyHardware() failed to get processor count from GetSystemInfo()"));
		WriteFStringToResults(OutResults.LastSurveyErrorDetail, TEXT(""));
	}

	// Get CPU brand
	FString CPUBrand = FWindowsPlatformMisc::GetCPUVendor();
	WriteFStringToResults(OutResults.CPUBrand, CPUBrand);
	if (CPUBrand.Len() == 0)
	{
		OutResults.ErrorCount++;
		WriteFStringToResults(OutResults.LastSurveyError, TEXT("FWindowsPlatformSurvey::TickSurveyHardware() failed to get processor brand from FWindowsPlatformMisc::GetCPUVendor()"));
		WriteFStringToResults(OutResults.LastSurveyErrorDetail, TEXT(""));
	}

	// Get CPU name
	FString CPUName = FWindowsPlatformMisc::GetCPUBrand();
	WriteFStringToResults(OutResults.CPUNameString, CPUName);
	if (CPUName.Len() == 0)
	{
		OutResults.ErrorCount++;
		WriteFStringToResults(OutResults.LastSurveyError, TEXT("FWindowsPlatformSurvey::TickSurveyHardware() failed to get processor name from FWindowsPlatformMisc::GetCPUBrand()"));
		WriteFStringToResults(OutResults.LastSurveyErrorDetail, TEXT(""));
	}

	// Get CPU info
	OutResults.CPUInfo = FWindowsPlatformMisc::GetCPUInfo();

	// get HDD details
	OutResults.HardDriveGB = -1;
	OutResults.HardDriveFreeMB = -1;
	
	uint64 TotalDiskSpace = 0;
	uint64 FreeDiskSpace = 0;
	if (FPlatformMisc::GetDiskTotalAndFreeSpace(FPlatformProcess::BaseDir(), TotalDiskSpace, FreeDiskSpace))
	{
		OutResults.HardDriveGB = (uint32)(TotalDiskSpace / uint64(1024 * 1024 * 1024));
		OutResults.HardDriveFreeMB = (uint32)(FreeDiskSpace / uint64(1024 * 1024));
	}
	else
	{
		uint32 ErrorCode = FPlatformMisc::GetLastError();
		UE_LOGF(LogWindows, Warning, "FWindowsPlatformSurvey::TickSurveyHardware() failed to get Unreal Engine root-folder drive size from Win32" );
		OutResults.ErrorCount++;
		WriteFStringToResults(OutResults.LastSurveyError, TEXT("GetDiskFreeSpaceEx() failed"));
		WriteFStringToResults(OutResults.LastSurveyErrorDetail, FString::Printf(TEXT("ErrorCode: 0x%0x"), ErrorCode));
	}

	// OS info
	GetOSVersionLabels(SystemInfo, OutResults);
	OutResults.OSBits = 64;

	// OS language
	LCID DefaultLocale = GetSystemDefaultLCID();
	const int32 MaxLocaleStringLength = 9;
	TCHAR LangBuffer[MaxLocaleStringLength];
	int LangReturn = GetLocaleInfo(DefaultLocale, LOCALE_SISO639LANGNAME, LangBuffer, UE_ARRAY_COUNT(LangBuffer));
	TCHAR CountryBuffer[MaxLocaleStringLength];
	int CountryReturn = GetLocaleInfo(DefaultLocale, LOCALE_SISO3166CTRYNAME, CountryBuffer, UE_ARRAY_COUNT(CountryBuffer));

	if (LangReturn == 0 || CountryReturn == 0)
	{
		uint32 ErrorCode = FPlatformMisc::GetLastError();
		UE_LOGF(LogWindows, Warning, "FWindowsPlatformSurvey::TickSurveyHardware() failed to get locale info from Win32" );
		OutResults.ErrorCount++;
		WriteFStringToResults(OutResults.LastSurveyError, TEXT("GetLocaleInfo() failed"));
		WriteFStringToResults(OutResults.LastSurveyErrorDetail, FString::Printf(TEXT("ErrorCode: 0x%0x"), ErrorCode));
	}
	else
	{
		WriteFStringToResults(OutResults.OSLanguage, FString::Printf(TEXT("%s-%s"), LangBuffer, CountryBuffer));
	}

#if USING_WINSAT_API
	// Get an instance to the most recent formal WinSAT assessmenet.
	IQueryRecentWinSATAssessment* Assessment;
	HRESULT COMResult = CoCreateInstance(
		__uuidof(CQueryWinSAT),
		NULL,
		CLSCTX_INPROC_SERVER,
		__uuidof(IQueryRecentWinSATAssessment),
		(void**)&Assessment);

	if (FAILED(COMResult))
	{
		UE_LOGF(LogWindows, Warning, "FWindowsPlatformSurvey::TickSurveyHardware() failed to get query interface from WinSAT API" );
		OutResults.ErrorCount++;
		WriteFStringToResults(OutResults.LastSurveyError, TEXT("CoCreateInstance() failed to get WinSAT"));
		WriteFStringToResults(OutResults.LastSurveyErrorDetail, FString::Printf(TEXT("HRESULT: 0x%0x"), (uint32)COMResult));

	}
	else
	{
		// Get the summary information for the WinSAT assessment.
		IProvideWinSATResultsInfo* WinSATResults = NULL;
		COMResult = Assessment->get_Info(&WinSATResults);
		if (FAILED(COMResult))
		{
			UE_LOGF(LogWindows, Error, "FWindowsPlatformSurvey::TickSurveyHardware() failed to get assessment results from WinSAT API" );
			OutResults.ErrorCount++;
			WriteFStringToResults(OutResults.LastSurveyError, TEXT("get_Info() failed to get WinSAT assessment results"));
			WriteFStringToResults(OutResults.LastSurveyErrorDetail, FString::Printf(TEXT("HRESULT: 0x%0x"), (uint32)COMResult));

		}
		else
		{
			// Get the state of the assessment.
			WINSAT_ASSESSMENT_STATE WinSATState;
			COMResult = WinSATResults->get_AssessmentState(&WinSATState);
			if (FAILED(COMResult))
			{
				UE_LOGF(LogWindows, Error, "FWindowsPlatformSurvey::TickSurveyHardware() failed to get assessment state from WinSAT API" );
				OutResults.ErrorCount++;
				WriteFStringToResults(OutResults.LastSurveyError, TEXT("get_AssessmentState() failed to get WinSAT assessment state"));
				WriteFStringToResults(OutResults.LastSurveyErrorDetail, FString::Printf(TEXT("HRESULT: 0x%0x"), (uint32)COMResult));
			}
			else
			{
				// Examine the assessment state
				bool bAssessmentAvailable = false;
				switch(WinSATState)
				{
				case WINSAT_ASSESSMENT_STATE_VALID:
					bAssessmentAvailable = true;
					break;

				case WINSAT_ASSESSMENT_STATE_INCOHERENT_WITH_HARDWARE:
					UE_LOGF(LogWindows, Log, "FWindowsPlatformSurvey::TickSurveyHardware() WinSAT assessment state is out-of-date. Unable to examine some hardware metrics. Run the Windows Experience Index Assessment." );
					OutResults.ErrorCount++;
					WriteFStringToResults(OutResults.LastSurveyError, TEXT("WinSAT assessment out-of-date. Using old results."));
					WriteFStringToResults(OutResults.LastSurveyErrorDetail, TEXT(""));
					bAssessmentAvailable = true;
					break;

				case WINSAT_ASSESSMENT_STATE_NOT_AVAILABLE:
					UE_LOGF(LogWindows, Log, "FWindowsPlatformSurvey::TickSurveyHardware() WinSAT assessment unavailable. Unable to examine some hardware metrics. Run the Windows Experience Index Assessment." );
					OutResults.ErrorCount++;
					WriteFStringToResults(OutResults.LastSurveyError, TEXT("WinSAT assessment unavailable. User hasn't run Windows Experience Index Assessment."));
					WriteFStringToResults(OutResults.LastSurveyErrorDetail, TEXT(""));
					break;

				default:
					UE_LOGF(LogWindows, Warning, "FWindowsPlatformSurvey::TickSurveyHardware() WinSAT assessment data was invalid." );
					OutResults.ErrorCount++;
					WriteFStringToResults(OutResults.LastSurveyError, TEXT("WinSAT assessment state unknown"));
					WriteFStringToResults(OutResults.LastSurveyErrorDetail, FString::Printf(TEXT("WinSATState: %d"), (int32)WinSATState));
				}

				// Get the index scores from the results
				if (bAssessmentAvailable)
				{
					if (!GetSubComponentIndex(WinSATResults, OutResults, WINSAT_ASSESSMENT_MEMORY, OutResults.RAMPerformanceIndex))
					{
						UE_LOGF(LogWindows, Error, "FWindowsPlatformSurvey::TickSurveyHardware() failed to get MEMORY score from WinSAT API." );
					}

					if (!GetSubComponentIndex(WinSATResults, OutResults, WINSAT_ASSESSMENT_CPU, OutResults.CPUPerformanceIndex))
					{
						UE_LOGF(LogWindows, Error, "FWindowsPlatformSurvey::TickSurveyHardware() failed to get CPU score from WinSAT API." );
					}

					float GPU3DScoreIndex = 0.0f;
					if (!GetSubComponentIndex(WinSATResults, OutResults, WINSAT_ASSESSMENT_D3D, GPU3DScoreIndex))
					{
						UE_LOGF(LogWindows, Error, "FWindowsPlatformSurvey::TickSurveyHardware() failed to get D3D score from WinSAT API." );
					}

					float GPUDesktopScoreIndex = 0.0f;
					if (!GetSubComponentIndex(WinSATResults, OutResults, WINSAT_ASSESSMENT_D3D, GPUDesktopScoreIndex))
					{
						UE_LOGF(LogWindows, Error, "FWindowsPlatformSurvey::TickSurveyHardware() failed to get GRAPHICS score from WinSAT API." );
					}

					OutResults.GPUPerformanceIndex = 0.5f * (GPU3DScoreIndex + GPUDesktopScoreIndex);
				}
			}
				
			WinSATResults->Release();
		}
	}

	if (Assessment)
	{
		Assessment->Release();
	}
#endif	// #if USING_WINSAT_API

	// Get system power info to determine whether we're running on a laptop or desktop computer
	OutResults.bIsLaptopComputer = false;
#if USING_POWRPROF
	SYSTEM_POWER_CAPABILITIES PowerCaps;
	NTSTATUS NTStatus = CallNtPowerInformation(SystemPowerCapabilities, NULL, 0L, (PVOID)&PowerCaps, sizeof(SYSTEM_POWER_CAPABILITIES));
	if (NT_SUCCESS(NTStatus))
	{
		OutResults.bIsLaptopComputer = PowerCaps.SystemBatteriesPresent && !PowerCaps.BatteriesAreShortTerm;
	}
	else
	{
		UE_LOGF(LogWindows, Warning, "FWindowsPlatformSurvey::TickSurveyHardware() failed to get system power capabilities. Assuming desktop PC." );
		OutResults.ErrorCount++;
		WriteFStringToResults(OutResults.LastSurveyError, TEXT("CallNtPowerInformation() failed to get system power capabilities"));
		WriteFStringToResults(OutResults.LastSurveyErrorDetail, FString::Printf(TEXT("NTSTATUS: 0x%0x"), (uint32)NTStatus));
	}
#endif	// #if USING_POWRPROF

	// Get remote desktop session status
	OutResults.bIsRemoteSession = GetSystemMetrics(SM_REMOTESESSION) != 0;

	return true;
}

bool FWindowsPlatformSurvey::GetSubComponentIndex( IProvideWinSATResultsInfo* WinSATResults, FHardwareSurveyResults& OutSurveyResults, int32 SubComponent, float& OutSubComponentIndex ) 
{
	bool bSuccess = false;

#if USING_WINSAT_API
	IProvideWinSATAssessmentInfo* AssessmentInfo = NULL;
	HRESULT COMResult = WinSATResults->GetAssessmentInfo((WINSAT_ASSESSMENT_TYPE)SubComponent, &AssessmentInfo);
	if (FAILED(COMResult))
	{
		UE_LOGF(LogWindows, Log, "FWindowsPlatformSurvey::GetSubComponentIndex() failed to get assessment info for a sub-component from WinSAT API." );
		OutSurveyResults.ErrorCount++;
		WriteFStringToResults(OutSurveyResults.LastPerformanceIndexError, FString::Printf(TEXT("GetAssessmentInfo() failed to get WinSAT assessment for sub-component %d"), SubComponent));
		WriteFStringToResults(OutSurveyResults.LastPerformanceIndexErrorDetail, FString::Printf(TEXT("HRESULT: 0x%0x"), (uint32)COMResult));
	}
	else
	{
		OutSubComponentIndex = 0.0f;
		COMResult = AssessmentInfo->get_Score(&OutSubComponentIndex);
		if (FAILED(COMResult))
		{
			UE_LOGF(LogWindows, Log, "FWindowsPlatformSurvey::GetSubComponentIndex() failed to get sub-component score from WinSAT API." );
			OutSurveyResults.ErrorCount++;
			WriteFStringToResults(OutSurveyResults.LastPerformanceIndexError, FString::Printf(TEXT("get_Score() failed to get WinSAT WIE score for sub-component %d"), SubComponent));
			WriteFStringToResults(OutSurveyResults.LastPerformanceIndexErrorDetail, FString::Printf(TEXT("HRESULT: 0x%0x"), (uint32)COMResult));
		}
		else
		{
			bSuccess = true;
		}

		AssessmentInfo->Release();
	}
#endif

	return bSuccess;
}

void GetOSVersionLabels(const SYSTEM_INFO& SystemInfo, FHardwareSurveyResults& OutResults)
{
	FString OSVersionLabel;
	FString OSSubVersionLabel;
	const int32 ErrorCode = FWindowsOSVersionHelper::GetOSVersions( OSVersionLabel, OSSubVersionLabel );

	if( ErrorCode & FWindowsOSVersionHelper::ERROR_GETPRODUCTINFO_FAILED )
	{
		OutResults.ErrorCount++;
		WriteFStringToResults( OutResults.LastSurveyError, TEXT( "Failed to get GetProductInfo() function from GetProcAddress()." ) );
		WriteFStringToResults( OutResults.LastSurveyErrorDetail, TEXT( "" ) );
	}

	if( ErrorCode & FWindowsOSVersionHelper::ERROR_UNKNOWNVERSION )
	{
		OSVERSIONINFOEX OsVersionInfo = {0};
		OsVersionInfo.dwOSVersionInfoSize = sizeof( OSVERSIONINFOEX );
#ifdef __clang__
#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wdeprecated-declarations"
#else
#pragma warning(push)
#pragma warning(disable : 4996) // 'function' was declared deprecated
#endif
		CA_SUPPRESS(28159)
		GetVersionEx( (LPOSVERSIONINFO)&OsVersionInfo );
#ifdef __clang__
#pragma clang diagnostic pop
#else
#pragma warning(pop)
#endif

		UE_LOGF( LogWindows, Warning, "FWindowsPlatformSurvey::GetOSVersionLabel() unknown Windows version info from GetVersionEx()" );
		OutResults.ErrorCount++;
		WriteFStringToResults( OutResults.LastSurveyError, TEXT( "GetVersionEx() returned unknown version" ) );
		WriteFStringToResults( OutResults.LastSurveyErrorDetail, FString::Printf( TEXT( "dwMajorVersion: %u  dwMinorVersion: %u" ), (uint32)OsVersionInfo.dwMajorVersion, (uint32)OsVersionInfo.dwMinorVersion ) );
	}

	if( ErrorCode & FWindowsOSVersionHelper::ERROR_GETVERSIONEX_FAILED )
	{
		const uint32 LastError = FPlatformMisc::GetLastError();
		UE_LOGF( LogWindows, Warning, "FWindowsPlatformSurvey::GetOSVersionLabel() failed to get Windows version info from GetVersionEx()" );
		OutResults.ErrorCount++;
		WriteFStringToResults( OutResults.LastSurveyError, TEXT( "GetVersionEx() failed" ) );
		WriteFStringToResults( OutResults.LastSurveyErrorDetail, FString::Printf( TEXT( "ErrorCode: 0x%0x" ), LastError ) );
	}

	WriteFStringToResults( OutResults.OSVersion, OSVersionLabel );
	WriteFStringToResults( OutResults.OSSubVersion, OSSubVersionLabel );
}

bool FWindowsPlatformSurvey::GetLineFollowing(const FString& Token, const TArray<FString>& InLines, FString& OutString, int32 NthHit)
{
	int32 HitIdx = 0;
	for (int32 LineIdx = 0; LineIdx < InLines.Num(); LineIdx++)
	{
		const FString& Line = InLines[LineIdx];

		int32 SubStrIdx =  Line.Find(Token);
		if (0 <= SubStrIdx && NthHit == HitIdx++)
		{
			OutString = Line.RightChop(SubStrIdx + Token.Len());
			return 0 < OutString.Len();
		}
	}
	return false;
}

void WriteFStringToResults(TCHAR* OutBuffer, const FString& InString)
{
	FMemory::Memset( OutBuffer, 0, sizeof(TCHAR) * FHardwareSurveyResults::MaxStringLength );
	TCHAR* Cursor = OutBuffer;
	for (int32 i = 0; i < FMath::Min(InString.Len(), FHardwareSurveyResults::MaxStringLength - 1); i++)
	{
		*Cursor++ = InString[i];
	}
}

bool FWindowsPlatformSurvey::GetNamedSection(FString SectionName, const TArray<FString>& InLines, TArray<FString>& OutSectionLines)
{
	OutSectionLines.Empty();
	int32 SectionStartLine = -1;
	int32 LineIdx = 0;
	for (; LineIdx < InLines.Num(); LineIdx++)
	{
		if (LineIdx < InLines.Num() - 2)
		{
			const FString& StartLine = InLines[LineIdx];

			if (StartLine.StartsWith(TEXT("---")))
			{
				const FString& EndLine = InLines[LineIdx+2];

				if (EndLine.StartsWith(TEXT("---")))
				{
					if (0 <= SectionStartLine)
					{
						break;
					}
					else
					{
						const FString& SectionLine = InLines[LineIdx+1];

						if (SectionLine.StartsWith(SectionName))
						{
							SectionStartLine = LineIdx + 3;
						}
					}
				}
			}
		}
	}

	if (0 <= SectionStartLine)
	{
		for (int32 CopyLineIdx = SectionStartLine; CopyLineIdx < LineIdx; CopyLineIdx++)
		{
			OutSectionLines.Add(InLines[CopyLineIdx]);
		}
	}

	return 0 < OutSectionLines.Num();
}

#include "Windows/HideWindowsPlatformTypes.h"