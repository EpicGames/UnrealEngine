// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsRemoteConsoleOutputDevice.h"

#if WITH_REMOTEWIN_CONSOLE

#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "Misc/OutputDeviceHelper.h"
#include "Containers/SpscQueue.h"
#include "Containers/Utf8String.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Event.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include <WinSock2.h>
#include <ws2tcpip.h>
#include "Microsoft/HideMicrosoftPlatformTypes.h"


class FRemoteConsoleRunnable : public FRunnable
{
public:
	FRemoteConsoleRunnable( const TArray<FString>& RemoteAddresses, uint32 RemotePort)
	{
		WakeEvent = FPlatformProcess::GetSynchEventFromPool(false);
		Socket = INVALID_SOCKET;

		WSADATA WSAData;
		WSAStartup(0x0101, &WSAData);

		// convert the given addresses to winsock ipv4 addresses
		for (const FString& RemoteAddress : RemoteAddresses)
		{
			sockaddr_storage& SockAddr = Addresses.AddZeroed_GetRef();
			sockaddr_in* SockAddrIP4 = (sockaddr_in*)&SockAddr;
			SockAddrIP4->sin_family = AF_INET;
			SockAddrIP4->sin_port = htons((u_short)RemotePort);
			InetPtonW(AF_INET, *RemoteAddress, &SockAddrIP4->sin_addr);
		}
	}


	virtual ~FRemoteConsoleRunnable()
	{
		WSACleanup();

		// note: this class is stored in GScopedLogConsole so there is no need to return WakeEvent to the pool as the pool is destroyed before us
	}


	virtual uint32 Run() override
	{
		// connect to the host PC
		if (!Connect())
		{
			UE_LOGF( LogWindows, Error, "Failed to connect to a WinRemoteConsoleDevice socket: %d", WSAGetLastError());
			return 1;
		}

		// keep pumping log messages
		while (!IsEngineExitRequested() && Socket != INVALID_SOCKET)
		{
			// wait for any new serialized log messages, waking periodically to send a heartbeat
			if (WakeEvent->Wait(FTimespan::FromSeconds(5)))
			{		
				FUtf8String Line;
				while (Lines.Dequeue(Line))
				{
					SendString(Line);
				}
			}
			else
			{
				// send a heartbeat to let UAT know that we are still running
				SendString(FUtf8String(TEXT("remotewin_heartbeat\n")));
			}
		}

		// clean up
		closesocket(Socket);
		Socket = INVALID_SOCKET;
		return 0;
	}


	virtual void Stop() override
	{
		WakeEvent->Trigger();
		closesocket(Socket);
		Socket = INVALID_SOCKET;
	}


	void Serialize( const TCHAR* Line )
	{
		if (Socket != INVALID_SOCKET)
		{
			Lines.Enqueue(FUtf8String(Line));
			WakeEvent->Trigger();
		}
	}

private:
	bool Connect()
	{
		static const int32 ConnectionTimeoutSeconds = 5;

		// find an address to connect to
		bool bIsConnected = false;
		for (sockaddr_storage& SockAddr : Addresses)
		{
			Socket = socket(AF_INET, SOCK_STREAM, IPPROTO_TCP);
			if (Socket == INVALID_SOCKET)
			{
				continue;
			}

			unsigned long NonBlockingMode = 1;
			ioctlsocket(Socket, FIONBIO, &NonBlockingMode);

			if (connect(Socket, (sockaddr*)&SockAddr, sizeof(sockaddr_in)) == 0)
			{
				bIsConnected = true;
				break;
			}

			if (WSAGetLastError() == WSAEWOULDBLOCK)
			{
				fd_set FdSet = { 1, { Socket }, };
				TIMEVAL TimeVal { ConnectionTimeoutSeconds, 0};
				int Res = select(0, nullptr, &FdSet, nullptr, &TimeVal);
				if (Res > 0)
				{
					bIsConnected = true;
					break;
				}
			}

			closesocket(Socket);
			Socket = INVALID_SOCKET;
		}

		// don't need the addresses any more
		Addresses.Reset();
	
		return bIsConnected;
	}


	void SendString( const FUtf8String& Line )
	{
		uint32 BytesToSend = Line.Len();
		const char* DataPtr = (const char*)*Line;

		while (BytesToSend > 0)
		{
			int32 BytesSent = send(Socket, DataPtr, BytesToSend, 0);
			if (BytesSent == SOCKET_ERROR)
			{
				return;
			}
			BytesToSend -= BytesSent;
			DataPtr += BytesSent;
		}
	}

	TArray<sockaddr_storage> Addresses;
	TSpscQueue<FUtf8String> Lines;
	FEvent* WakeEvent;
	SOCKET Socket;
};




FPlaceholder_WindowsRemoteConsoleOutputDevice::FPlaceholder_WindowsRemoteConsoleOutputDevice( const FString& RemoteConsoleHost )
	: Runnable(nullptr)
	, Thread(nullptr)
{	
	FString AddressesString;
	FString PortString;
	if (RemoteConsoleHost.Split(TEXT(":"), &AddressesString, &PortString))
	{
		TArray<FString> RemoteAddresses;
		AddressesString.ParseIntoArray(RemoteAddresses, TEXT("+"));

		if (RemoteAddresses.Num() > 0)
		{
			uint32 RemotePort = FCString::Atoi(*PortString);

			Runnable = new FRemoteConsoleRunnable(RemoteAddresses, RemotePort);
			Thread = FRunnableThread::Create( Runnable, TEXT("WinRemoteConsoleDevice"), 0, EThreadPriority::TPri_BelowNormal );
		}
	}

	UE_CLOGF( Thread == nullptr, LogWindows, Error, "Failed to create WinRemoteConsoleDevice");
}


FPlaceholder_WindowsRemoteConsoleOutputDevice::~FPlaceholder_WindowsRemoteConsoleOutputDevice()
{
	if( Thread )
	{
		Thread->Kill();
		delete Thread;

		delete Runnable;
		Runnable = nullptr;
	}
}


void FPlaceholder_WindowsRemoteConsoleOutputDevice::Serialize( const TCHAR* Msg, ELogVerbosity::Type Verbosity, const class FName& Category )
{
	if (Runnable != nullptr)
	{
		FString LogMessage = FOutputDeviceHelper::FormatLogLine(Verbosity, Category, Msg, GPrintLogTimes) + LINE_TERMINATOR;
		Runnable->Serialize(*LogMessage);
	}
}


#endif // WITH_REMOTEWIN_CONSOLE
