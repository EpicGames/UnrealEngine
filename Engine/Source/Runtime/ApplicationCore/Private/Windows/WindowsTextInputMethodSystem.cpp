// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsTextInputMethodSystem.h"
#include "CoreGlobals.h"
#include "Containers/UnrealString.h"
#include "Math/Vector2D.h"
#include "Logging/LogCategory.h"
#include "GenericPlatform/GenericWindow.h"
#include "Stats/Stats.h"

DEFINE_LOG_CATEGORY_STATIC(LogWindowsTextInputMethodSystem, Log, All);

DECLARE_STATS_GROUP(TEXT("IME Windows"), STATGROUP_IMEWindows, STATCAT_Advanced);

DECLARE_CYCLE_STAT(TEXT("IME ApplyDefaults"), STAT_IMEWindowsApplyDefaults, STATGROUP_IMEWindows);
DECLARE_CYCLE_STAT(TEXT("IME RegisterContext"), STAT_IMEWindowsRegisterContext, STATGROUP_IMEWindows);
DECLARE_CYCLE_STAT(TEXT("IME UnregisterContext"), STAT_IMEWindowsUnregisterContext, STATGROUP_IMEWindows);
DECLARE_CYCLE_STAT(TEXT("IME ActivateContext"), STAT_IMEWindowsActivateContext, STATGROUP_IMEWindows);
DECLARE_CYCLE_STAT(TEXT("IME DeactivateContext"), STAT_IMEWindowsDeactivateContext, STATGROUP_IMEWindows);
DECLARE_CYCLE_STAT(TEXT("IME OnIMEActivationStateChanged"), STAT_IMEWindowsOnIMEActivationStateChanged, STATGROUP_IMEWindows);
DECLARE_CYCLE_STAT(TEXT("IME ProcessMessage"), STAT_IMEWindowsProcessMessage, STATGROUP_IMEWindows);

#include "Windows/AllowWindowsPlatformTypes.h"


namespace
{
	FString GetIMMStringAsFString(HIMC IMMContext, const DWORD StringType)
	{
		// Get the internal buffer of the string, we're going to use it as scratch space
		FString OutString;
		TArray<TCHAR, FString::AllocatorType>& OutStringBuffer = OutString.GetCharArray();
				
		// Work out the maximum size required and resize the buffer so it can hold enough data
		const LONG StringNeededSizeBytes = ::ImmGetCompositionString(IMMContext, StringType, nullptr, 0);
		const int32 StringNeededSizeTCHARs = static_cast<int32>(StringNeededSizeBytes) / sizeof(TCHAR);
		OutStringBuffer.SetNumUninitialized(StringNeededSizeTCHARs + 1); // +1 for null

		// Get directly into the string buffer, and then null terminate the FString
		::ImmGetCompositionString(IMMContext, StringType, OutStringBuffer.GetData(), StringNeededSizeBytes);
		OutStringBuffer[StringNeededSizeTCHARs] = 0;

		return OutString;
	}

	FString GetLocaleInfoAsFString(LCID lcid)
	{
		// Get the internal buffer of the string, we're going to use it as scratch space
		FString OutString;
		TArray<TCHAR, FString::AllocatorType>& OutStringBuffer = OutString.GetCharArray();

		// Work out the maximum size required and resize the buffer so it can hold enough data
		const int32 StringNeededSize = ::GetLocaleInfo(lcid, LOCALE_SLANGUAGE, nullptr, 0);
		OutStringBuffer.SetNumUninitialized(StringNeededSize); // size already includes null

		// Get directly into the string buffer
		::GetLocaleInfo(lcid, LOCALE_SLANGUAGE, OutStringBuffer.GetData(), StringNeededSize);

		return OutString;
	}

	FString GetHKLDescriptionAsFString(HKL KeyboardLayout)
	{
		const HKL ActiveKeyboardLayout = ::GetKeyboardLayout(0);
		if (ActiveKeyboardLayout != KeyboardLayout)
		{
			// We need to activate this layout, as the IMM functions below ignore their HKL argument
			::ActivateKeyboardLayout(KeyboardLayout, 0);
		}

		// Build a friendly description in the form "{Locale} - {Description} ({Type})"
		FString InputMethodDescription = GetLocaleInfoAsFString(MAKELCID(((UPTRINT)KeyboardLayout & 0xffffffff), SORT_DEFAULT));
		InputMethodDescription += TEXT(" - ");
		if (::ImmGetIMEFileName(KeyboardLayout, nullptr, 0) > 0)
		{
			TArray<TCHAR> DescriptionString;
			const int32 DescriptionLen = ::ImmGetDescription(KeyboardLayout, nullptr, 0);
			DescriptionString.SetNumUninitialized(DescriptionLen + 1); // +1 for null
			::ImmGetDescription(KeyboardLayout, DescriptionString.GetData(), DescriptionLen);
			DescriptionString[DescriptionLen] = 0;

			if (DescriptionString.Num() > 1)
			{
				InputMethodDescription += DescriptionString.GetData();
				InputMethodDescription += TEXT(" ");
			}

			InputMethodDescription += TEXT("(IMM IME)");
		}
		else
		{
			InputMethodDescription += TEXT("(Keyboard)");
		}

		if (ActiveKeyboardLayout != KeyboardLayout)
		{
			// Restore the previous keyboard layout
			::ActivateKeyboardLayout(ActiveKeyboardLayout, 0);
		}

		return InputMethodDescription;
	}

	FString GetTSFInputMethodAsFString(const TF_INPUTPROCESSORPROFILE& TSFProfile, ITfInputProcessorProfiles& TSFInputProcessorProfiles)
	{
		// Build a friendly description in the form "{Locale} - {Description} ({Type})"
		FString InputMethodDescription = GetLocaleInfoAsFString(MAKELCID(TSFProfile.langid, SORT_DEFAULT));
		InputMethodDescription += TEXT(" - ");
		if (TSFProfile.dwProfileType == TF_PROFILETYPE_KEYBOARDLAYOUT)
		{
			InputMethodDescription += TEXT("(Keyboard)");
		}
		else if (TSFProfile.dwProfileType == TF_PROFILETYPE_INPUTPROCESSOR)
		{
			BSTR TSFDescriptionString;
			if (SUCCEEDED(TSFInputProcessorProfiles.GetLanguageProfileDescription(TSFProfile.clsid, TSFProfile.langid, TSFProfile.guidProfile, &TSFDescriptionString)))
			{
				InputMethodDescription += TSFDescriptionString;
				InputMethodDescription += TEXT(" ");
				::SysFreeString(TSFDescriptionString);
			}

			InputMethodDescription += TEXT("(TSF IME)");
		}

		return InputMethodDescription;
	}

	class FTextInputMethodChangeNotifier
		: public ITextInputMethodChangeNotifier
	{
	public:
		FTextInputMethodChangeNotifier(const TComPtr<FTextStoreACP>& InTextStoreACP)
			:	TextStoreACP(InTextStoreACP)
		{}

		virtual ~FTextInputMethodChangeNotifier() {}

		virtual void NotifyLayoutChanged(const ELayoutChangeType ChangeType) override;
		virtual void NotifySelectionChanged() override;
		virtual void NotifyTextChanged(const uint32 BeginIndex, const uint32 OldLength, const uint32 NewLength) override;
		virtual void CancelComposition() override;

	private:
		const TComPtr<FTextStoreACP> TextStoreACP;
	};

	void FTextInputMethodChangeNotifier::NotifyLayoutChanged(const ELayoutChangeType ChangeType)
	{
		if(TextStoreACP->AdviseSinkObject.TextStoreACPSink != nullptr)
		{
			TsLayoutCode LayoutCode = TS_LC_CHANGE;
			switch(ChangeType)
			{
			case ELayoutChangeType::Created:	{ LayoutCode = TS_LC_CREATE; } break;
			case ELayoutChangeType::Changed:	{ LayoutCode = TS_LC_CHANGE; } break;
			case ELayoutChangeType::Destroyed:	{ LayoutCode = TS_LC_DESTROY; } break;
			}
			TextStoreACP->AdviseSinkObject.TextStoreACPSink->OnLayoutChange(LayoutCode, 0);
		}
	}

	void FTextInputMethodChangeNotifier::NotifySelectionChanged()
	{
		if(TextStoreACP->AdviseSinkObject.TextStoreACPSink != nullptr)
		{
			TextStoreACP->AdviseSinkObject.TextStoreACPSink->OnSelectionChange();
		}
	}

	void FTextInputMethodChangeNotifier::NotifyTextChanged(const uint32 BeginIndex, const uint32 OldLength, const uint32 NewLength)
	{
		if(TextStoreACP->AdviseSinkObject.TextStoreACPSink != nullptr)
		{
			TS_TEXTCHANGE TextChange;
			TextChange.acpStart = BeginIndex;
			TextChange.acpOldEnd = BeginIndex + OldLength;
			TextChange.acpNewEnd = BeginIndex + NewLength;
			TextStoreACP->AdviseSinkObject.TextStoreACPSink->OnTextChange(0, &(TextChange));
		}
	}

	void FTextInputMethodChangeNotifier::CancelComposition()
	{
		if(TextStoreACP->TSFContextOwnerCompositionServices != nullptr && TextStoreACP->Composition.TSFCompositionView != nullptr)
		{
			TextStoreACP->TSFContextOwnerCompositionServices->TerminateComposition(TextStoreACP->Composition.TSFCompositionView);
		}
	}
}

STDMETHODIMP FTSFActivationProxy::QueryInterface(REFIID riid, void **ppvObj)
{
	*ppvObj = nullptr;

	if( IsEqualIID(riid, IID_IUnknown) || IsEqualIID(riid, IID_ITfInputProcessorProfileActivationSink) )
	{
		*ppvObj = static_cast<ITfInputProcessorProfileActivationSink*>(this);
	}
	else if( IsEqualIID(riid, IID_ITfActiveLanguageProfileNotifySink) )
	{
		*ppvObj = static_cast<ITfActiveLanguageProfileNotifySink*>(this);
	}

	// Add a reference if we're (conceptually) returning a reference to our self.
	if (*ppvObj)
	{
		AddRef();
	}

	return *ppvObj ? S_OK : E_NOINTERFACE;
}

STDMETHODIMP_(ULONG) FTSFActivationProxy::AddRef()
{
	return ++ReferenceCount;
}

STDMETHODIMP_(ULONG) FTSFActivationProxy::Release()
{
	const ULONG LocalReferenceCount = --ReferenceCount;
	if(!ReferenceCount)
	{
		delete this;
	}
	return LocalReferenceCount;
}

STDAPI FTSFActivationProxy::OnActivated(DWORD dwProfileType, LANGID langid, __RPC__in REFCLSID clsid, __RPC__in REFGUID catid, __RPC__in REFGUID guidProfile, HKL hkl, DWORD dwFlags)
{
	const bool bIsEnabled = !!(dwFlags & TF_IPSINK_FLAG_ACTIVE);
	Owner->OnIMEActivationStateChanged(bIsEnabled);
	return S_OK;
}

STDAPI FTSFActivationProxy::OnActivated(REFCLSID clsid, REFGUID guidProfile, BOOL fActivated)
{
	Owner->OnIMEActivationStateChanged(fActivated == TRUE);
	return S_OK;
}

bool FWindowsTextInputMethodSystem::Initialize()
{
	CurrentAPI = EAPI::Unknown;
	const bool Result = InitializeIMM() && InitializeTSF();

	if(Result)
	{
		// Log all available input methods, to catch issues where crashes are caused by having an IME installed that's never actually activated
		{
			TArray<FString> AvailableInputMethods;
			TSet<HKL> ProcessedKeyboardLayouts;

			// Query TSF-based input methods first, which will include physical keyboards and TSF-based IMEs
			// Most modern Windows input methods use TSF, so this should catch everything except legacy IMEs that only support IMM
			{
				// Create an enumerator for all TSF-based input profiles (langid 0)
				TComPtr<IEnumTfInputProcessorProfiles> TSFEnumInputProcessorProfiles;
				if (SUCCEEDED(TSFInputProcessorProfileManager->EnumProfiles(0, &TSFEnumInputProcessorProfiles)))
				{
					// Enumerate in batches to minimize COM API calls
					ULONG FetchedTSFProfilesCount = 0;
					TF_INPUTPROCESSORPROFILE TSFProfiles[32];
					while (SUCCEEDED(TSFEnumInputProcessorProfiles->Next(UE_ARRAY_COUNT(TSFProfiles), TSFProfiles, &FetchedTSFProfilesCount)) && FetchedTSFProfilesCount > 0)
					{
						for (ULONG FetchedTSFProfileIndex = 0; FetchedTSFProfileIndex < FetchedTSFProfilesCount; ++FetchedTSFProfileIndex)
						{
							const TF_INPUTPROCESSORPROFILE& TSFProfile = TSFProfiles[FetchedTSFProfileIndex];

							// If this is a keyboard layout, mark it as processed even if we'll skip it via the disabled test below
							// This will stop it potentially being reconsidered as a potential IMM-based IME
							if (TSFProfile.dwProfileType == TF_PROFILETYPE_KEYBOARDLAYOUT)
							{
								ProcessedKeyboardLayouts.Add(TSFProfile.hkl);
							}

							// Skip disabled profiles, as these may have just been installed by default
							if (!(TSFProfile.dwFlags & TF_IPP_FLAG_ENABLED))
							{
								continue;
							}

							AvailableInputMethods.Add(GetTSFInputMethodAsFString(TSFProfile, *TSFInputProcessorProfiles));
						}
					}
				}
			}

			// Query HKL inputs second, discarding any for physical keyboards that were already processed as a TSF-based input method
			// That should leave us with a list of legacy IMEs that only support IMM
			{
				// Get all the available keyboard layouts
				const int32 NumKeyboardLayouts = ::GetKeyboardLayoutList(0, nullptr);
				TArray<HKL, TInlineAllocator<4>> KeyboardLayouts;
				KeyboardLayouts.AddZeroed(NumKeyboardLayouts);
				if (::GetKeyboardLayoutList(KeyboardLayouts.Num(), KeyboardLayouts.GetData()) == NumKeyboardLayouts)
				{
					for (HKL KeyboardLayout : KeyboardLayouts)
					{
						// Skip anything already processed
						if (ProcessedKeyboardLayouts.Contains(KeyboardLayout))
						{
							continue;
						}

						// Mark this as processed, as GetKeyboardLayoutList can return duplicates
						ProcessedKeyboardLayouts.Add(KeyboardLayout);

						// Anything left at this point is likely an IMM-based IME
						AvailableInputMethods.Add(GetHKLDescriptionAsFString(KeyboardLayout));
					}
				}
			}

			UE_LOGF(LogWindowsTextInputMethodSystem, Log, "Available input methods:");
			for (const FString& AvailableInputMethod : AvailableInputMethods)
			{
				UE_LOGF(LogWindowsTextInputMethodSystem, Log, "  - %ls.", *AvailableInputMethod);
			}
		}

		// Detect whether we have an IME active, and log the active input method
		{
			// We might already be using an IME if it's set as the default language
			// If so, work out what kind of IME it is
			TF_INPUTPROCESSORPROFILE TSFProfile;
			if (SUCCEEDED(TSFInputProcessorProfileManager->GetActiveProfile(GUID_TFCAT_TIP_KEYBOARD, &TSFProfile)) && TSFProfile.dwProfileType == TF_PROFILETYPE_INPUTPROCESSOR)
			{
				CurrentAPI = EAPI::TSF;
			}
			else if (::ImmGetIMEFileName(::GetKeyboardLayout(0), nullptr, 0) > 0)
			{
				CurrentAPI = EAPI::IMM;
			}

			LogActiveInputMethod();
		}
	}

	return Result;
}

void FWindowsTextInputMethodSystem::LogActiveInputMethod()
{
	FString InputMethodDescription;

	if (CurrentAPI == EAPI::TSF)
	{
		// TSF-based IME
		TF_INPUTPROCESSORPROFILE TSFProfile;
		if (SUCCEEDED(TSFInputProcessorProfileManager->GetActiveProfile(GUID_TFCAT_TIP_KEYBOARD, &TSFProfile)) && TSFProfile.dwProfileType == TF_PROFILETYPE_INPUTPROCESSOR)
		{
			InputMethodDescription = GetTSFInputMethodAsFString(TSFProfile, *TSFInputProcessorProfiles);
		}
	}
	else
	{
		// IMM-based IME, or physical keyboard
		InputMethodDescription = GetHKLDescriptionAsFString(::GetKeyboardLayout(0));
	}

	UE_LOGF(LogWindowsTextInputMethodSystem, Log, "Activated input method: %ls.", InputMethodDescription.IsEmpty() ? TEXT("Unknown") : *InputMethodDescription);
}

bool FWindowsTextInputMethodSystem::InitializeIMM()
{
	UE_LOGF(LogWindowsTextInputMethodSystem, Verbose, "Initializing IMM...");

	IMMContextId = ::ImmCreateContext();
	UpdateIMMProperty(::GetKeyboardLayout(0));

	if (!IMMContextId)
	{
		UE_CLOGF(!GIsBuildMachine, LogWindowsTextInputMethodSystem, Warning, "Initialization failed while creating the IMM context.");
		return false;
	}

	UE_LOGF(LogWindowsTextInputMethodSystem, Verbose, "Initialized IMM!");

	return true;
}

void FWindowsTextInputMethodSystem::UpdateIMMProperty(HKL KeyboardLayoutHandle)
{
	IMMProperties = ::ImmGetProperty(KeyboardLayoutHandle, IGP_PROPERTY);
}

bool FWindowsTextInputMethodSystem::ShouldDrawIMMCompositionString() const
{
	// If the IME doesn't have any kind of special UI and it draws the composition window at the caret, we can draw it ourselves.
	return !(IMMProperties & IME_PROP_SPECIAL_UI) && (IMMProperties & IME_PROP_AT_CARET);
}

void FWindowsTextInputMethodSystem::UpdateIMMWindowPositions(HIMC IMMContext)
{
	if(ActiveContext.IsValid())
	{
		FInternalContext& InternalContext = ContextToInternalContextMap[ActiveContext];

		// Get start of composition area.
		const uint32 BeginIndex = InternalContext.IMMContext.CompositionBeginIndex;
		const uint32 Length = InternalContext.IMMContext.CompositionLength;
		FVector2D Position;
		FVector2D Size;
		ActiveContext->GetTextBounds(BeginIndex, Length, Position, Size);

		// Positions provided to IMM are relative to the window, but we retrieved screen-space coordinates.
		TSharedPtr<FGenericWindow> GenericWindow = ActiveContext->GetWindow();
		HWND WindowHandle = reinterpret_cast<HWND>(GenericWindow->GetOSWindowHandle());
		RECT WindowRect;
		GetWindowRect(WindowHandle, &(WindowRect));
		Position.X -= WindowRect.left;
		Position.Y -= WindowRect.top;

		// Update candidate window position.
		CANDIDATEFORM CandidateForm;
		CandidateForm.dwIndex = 0;
		CandidateForm.dwStyle = CFS_EXCLUDE;
		CandidateForm.ptCurrentPos.x = (LONG)Position.X;
		CandidateForm.ptCurrentPos.y = (LONG)Position.Y;
		CandidateForm.rcArea.left = CandidateForm.ptCurrentPos.x;
		CandidateForm.rcArea.right = CandidateForm.ptCurrentPos.x;
		CandidateForm.rcArea.top = CandidateForm.ptCurrentPos.y;
		CandidateForm.rcArea.bottom = CandidateForm.ptCurrentPos.y + (LONG)Size.Y;
		::ImmSetCandidateWindow(IMMContext, &CandidateForm);

		// Update composition window position.
		COMPOSITIONFORM CompositionForm;
		CompositionForm.dwStyle = CFS_POINT;
		CompositionForm.ptCurrentPos.x = (LONG)Position.X;
		CompositionForm.ptCurrentPos.y = (LONG)(Position.Y + Size.Y);
		::ImmSetCompositionWindow(IMMContext, &CompositionForm);
	}
}

void FWindowsTextInputMethodSystem::BeginIMMComposition()
{
	check(ActiveContext.IsValid());

	FInternalContext& InternalContext = ContextToInternalContextMap[ActiveContext];
				
	InternalContext.IMMContext.IsComposing = true;
	InternalContext.IMMContext.IsDeactivating = false;
	ActiveContext->BeginComposition();

	uint32 SelectionBeginIndex = 0;
	uint32 SelectionLength = 0;
	ITextInputMethodContext::ECaretPosition SelectionCaretPosition = ITextInputMethodContext::ECaretPosition::Ending;
	ActiveContext->GetSelectionRange(SelectionBeginIndex, SelectionLength, SelectionCaretPosition);
				
	// Set the initial composition range based on the start of the current selection
	// We ignore the relative caret position as once you start typing any selected text is 
	// removed before new text is added, so the caret will effectively always be at the start
	InternalContext.IMMContext.CompositionBeginIndex = SelectionBeginIndex;
	InternalContext.IMMContext.CompositionLength = 0;
	ActiveContext->UpdateCompositionRange(InternalContext.IMMContext.CompositionBeginIndex, InternalContext.IMMContext.CompositionLength);
}

void FWindowsTextInputMethodSystem::EndIMMComposition()
{
	check(ActiveContext.IsValid());

	FInternalContext& InternalContext = ContextToInternalContextMap[ActiveContext];
	InternalContext.IMMContext.IsComposing = false;
	InternalContext.IMMContext.IsDeactivating = false;
	ActiveContext->EndComposition();
}

void FWindowsTextInputMethodSystem::CancelIMMComposition()
{
	check(ActiveContext.IsValid());

	UE_LOGF(LogWindowsTextInputMethodSystem, Verbose, "WM_IME_COMPOSITION Composition Canceled");

	FInternalContext& InternalContext = ContextToInternalContextMap[ActiveContext];

	const int32 CurrentCompositionBeginIndex = InternalContext.IMMContext.CompositionBeginIndex;
	const uint32 CurrentCompositionLength = InternalContext.IMMContext.CompositionLength;

	// Clear Composition
	InternalContext.IMMContext.CompositionLength = 0;
	ActiveContext->UpdateCompositionRange(InternalContext.IMMContext.CompositionBeginIndex, 0);
	ActiveContext->SetSelectionRange(InternalContext.IMMContext.CompositionBeginIndex, 0, ITextInputMethodContext::ECaretPosition::Beginning);
	ActiveContext->SetTextInRange(CurrentCompositionBeginIndex, CurrentCompositionLength, TEXT(""));

	EndIMMComposition();
}

bool FWindowsTextInputMethodSystem::InitializeTSF()
{
	UE_LOGF(LogWindowsTextInputMethodSystem, Verbose, "Initializing TSF...");

	HRESULT Result = S_OK;

	// Input Processors
	{
		// Get input processor profiles.
		ITfInputProcessorProfiles* RawPointerTSFInputProcessorProfiles;
		Result = ::CoCreateInstance(CLSID_TF_InputProcessorProfiles, nullptr, CLSCTX_INPROC_SERVER, IID_ITfInputProcessorProfiles, reinterpret_cast<void**>(&(RawPointerTSFInputProcessorProfiles)));
		if(FAILED(Result))
		{
			TCHAR ErrorMsg[1024];
			FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
			UE_LOGF(LogWindowsTextInputMethodSystem, Error, "Initialization failed while creating the TSF input processor profiles. %ls (0x%08x)", ErrorMsg, Result);
			return false;
		}
		TSFInputProcessorProfiles.Attach(RawPointerTSFInputProcessorProfiles);

		// Get input processor profile manager from profiles.
		Result = TSFInputProcessorProfileManager.FromQueryInterface(IID_ITfInputProcessorProfileMgr, TSFInputProcessorProfiles);
		if(FAILED(Result))
		{
			TCHAR ErrorMsg[1024];
			FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
			UE_LOGF(LogWindowsTextInputMethodSystem, Error, "Initialization failed while acquiring the TSF input processor profile manager. %ls (0x%08x)", ErrorMsg, Result);
			TSFInputProcessorProfiles.Reset();
			return false;
		}
	}

	// Thread Manager
	{
		// Create thread manager.
		ITfThreadMgr* RawPointerTSFThreadManager;
		Result = ::CoCreateInstance(CLSID_TF_ThreadMgr, NULL, CLSCTX_INPROC_SERVER, IID_ITfThreadMgr, reinterpret_cast<void**>(&(RawPointerTSFThreadManager)));
		if(FAILED(Result))
		{
			if (!GIsBuildMachine)
			{
				TCHAR ErrorMsg[1024];
				FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
				UE_LOGF(LogWindowsTextInputMethodSystem, Warning, "Initialization failed while creating the TSF thread manager. %ls (0x%08x)", ErrorMsg, Result);
			}
			TSFInputProcessorProfiles.Reset();
			TSFInputProcessorProfileManager.Reset();
			return false;
		}
		TSFThreadManager.Attach(RawPointerTSFThreadManager);

		// Activate thread manager.
		Result = TSFThreadManager->Activate(&(TSFClientId));
		if(FAILED(Result))
		{
			TCHAR ErrorMsg[1024];
			FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
			UE_LOGF(LogWindowsTextInputMethodSystem, Error, "Initialization failed while activating the TSF thread manager. %ls (0x%08x)", ErrorMsg, Result);
			TSFInputProcessorProfiles.Reset();
			TSFInputProcessorProfileManager.Reset();
			TSFThreadManager.Reset();
			return false;
		}

		// Get source from thread manager, needed to install profile processor related sinks.
		TComPtr<ITfSource> TSFSource;
		Result = TSFSource.FromQueryInterface(IID_ITfSource, TSFThreadManager);
		if(FAILED(Result))
		{
			TCHAR ErrorMsg[1024];
			FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
			UE_LOGF(LogWindowsTextInputMethodSystem, Error, "Initialization failed while acquiring the TSF source from TSF thread manager. %ls (0x%08x)", ErrorMsg, Result);
			TSFInputProcessorProfiles.Reset();
			TSFInputProcessorProfileManager.Reset();
			TSFThreadManager.Reset();
			return false;
		}

		TSFActivationProxy = new FTSFActivationProxy(this);

#ifdef __clang__
#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wdeprecated-declarations"
#else
#pragma warning(push)
#pragma warning(disable : 4996) // 'function' was declared deprecated
#endif
		CA_SUPPRESS(28159)
		const DWORD WindowsVersion = ::GetVersion();
#ifdef __clang__
#pragma clang diagnostic pop
#else
#pragma warning(pop)
#endif

		Result = TSFSource->AdviseSink(IID_ITfInputProcessorProfileActivationSink, static_cast<ITfInputProcessorProfileActivationSink*>(TSFActivationProxy), &(TSFActivationProxy->TSFProfileCookie));
		if(FAILED(Result))
		{
			TCHAR ErrorMsg[1024];
			FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
			UE_LOGF(LogWindowsTextInputMethodSystem, Error, "Initialization failed while advising the profile notification sink to the TSF source. %ls (0x%08x)", ErrorMsg, Result);
			TSFInputProcessorProfiles.Reset();
			TSFInputProcessorProfileManager.Reset();
			TSFThreadManager.Reset();
			TSFActivationProxy.Reset();
			return false;
		}
	}

	// Disabled Document Manager
	Result = TSFThreadManager->CreateDocumentMgr(&(TSFDisabledDocumentManager));
	if(FAILED(Result))
	{
		TCHAR ErrorMsg[1024];
		FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
		UE_LOGF(LogWindowsTextInputMethodSystem, Warning, "Initialization failed while creating the TSF document manager. %ls (0x%08x)", ErrorMsg, Result);
		TSFInputProcessorProfiles.Reset();
		TSFInputProcessorProfileManager.Reset();
		TSFThreadManager.Reset();
		TSFActivationProxy.Reset();
		return false;
	}

	// Default the focus to the disabled document manager.
	Result = TSFThreadManager->SetFocus(TSFDisabledDocumentManager);
	if(FAILED(Result))
	{
		TCHAR ErrorMsg[1024];
		FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
		UE_LOGF(LogWindowsTextInputMethodSystem, Error, "Initialization failed while activating the TSF document manager. %ls (0x%08x)", ErrorMsg, Result);
		TSFInputProcessorProfiles.Reset();
		TSFInputProcessorProfileManager.Reset();
		TSFThreadManager.Reset();
		TSFActivationProxy.Reset();
		return false;
	}

	UE_LOGF(LogWindowsTextInputMethodSystem, Verbose, "Initialized TSF!");

	return true;
}

void FWindowsTextInputMethodSystem::Terminate()
{
	HRESULT Result;

	::ImmDestroyContext(IMMContextId);

	// Get source from thread manager, needed to uninstall profile processor related sinks.
	TComPtr<ITfSource> TSFSource;
	Result = TSFSource.FromQueryInterface(IID_ITfSource, TSFThreadManager);
	if(FAILED(Result) || !TSFSource)
	{
		TCHAR ErrorMsg[1024];
		FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
		UE_LOGF(LogWindowsTextInputMethodSystem, Error, "Terminating failed while acquiring the TSF source from the TSF thread manager. %ls (0x%08x)", ErrorMsg, Result);
	}

	if(TSFSource && TSFActivationProxy)
	{
		// Uninstall language notification sink.
		if(TSFActivationProxy->TSFLanguageCookie != TF_INVALID_COOKIE)
		{
			Result = TSFSource->UnadviseSink(TSFActivationProxy->TSFLanguageCookie);
			if(FAILED(Result))
			{
				TCHAR ErrorMsg[1024];
				FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
				UE_LOGF(LogWindowsTextInputMethodSystem, Error, "Terminating failed while unadvising the language notification sink from the TSF source. %ls (0x%08x)", ErrorMsg, Result);
			}
		}

		// Uninstall profile notification sink.
		if(TSFActivationProxy->TSFProfileCookie != TF_INVALID_COOKIE)
		{
			Result = TSFSource->UnadviseSink(TSFActivationProxy->TSFProfileCookie);
			if(FAILED(Result))
			{
				TCHAR ErrorMsg[1024];
				FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
				UE_LOGF(LogWindowsTextInputMethodSystem, Error, "Terminating failed while unadvising the profile notification sink from the TSF source. %ls (0x%08x)", ErrorMsg, Result);
			}
		}
	}
	TSFActivationProxy.Reset();

	Result = TSFThreadManager->Deactivate();
	if(FAILED(Result))
	{
		TCHAR ErrorMsg[1024];
		FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
		UE_LOGF(LogWindowsTextInputMethodSystem, Error, "Terminating failed while deactivating the TSF thread manager. %ls (0x%08x)", ErrorMsg, Result);
	}

	TSFThreadManager.Reset();

	TSFDisabledDocumentManager.Reset();
	TSFInputProcessorProfiles.Reset();
	TSFInputProcessorProfileManager.Reset();
}

void FWindowsTextInputMethodSystem::ClearStaleWindowHandles()
{
	auto KnownWindowsTmp = KnownWindows;
	for(const TWeakPtr<FGenericWindow>& Window : KnownWindowsTmp)
	{
		if(!Window.IsValid())
		{
			KnownWindows.Remove(Window);
		}
	}
}

void FWindowsTextInputMethodSystem::ApplyDefaults(const TSharedRef<FGenericWindow>& InWindow)
{
	UE_LOGF(LogWindowsTextInputMethodSystem, Verbose, "Apply Defaults...");

	SCOPE_CYCLE_COUNTER(STAT_IMEWindowsApplyDefaults);
	ClearStaleWindowHandles();
	KnownWindows.Add(InWindow);

	const HWND Hwnd = reinterpret_cast<HWND>(InWindow->GetOSWindowHandle());

	// Typically we disable the IME when a new window is created, however, we don't want to do that if we have an active context, 
	// as it will immediately disable the IME (which may be undesirable for things like a search suggestions window appearing)
	// In that case, we set the window to use the same IME context as is currently active. If the window actually takes focus
	// away from the currently active IME context, then that will be taken care of in DeactivateContext, and all windows using the
	// IME context will be disabled
	ITfDocumentMgr* TSFDocumentManagerToSet = TSFDisabledDocumentManager;
	HIMC IMMContextToSet = nullptr;
	if(ActiveContext.IsValid())
	{
		const HRESULT Result = TSFThreadManager->GetFocus(&TSFDocumentManagerToSet);
		if (FAILED(Result))
		{
			TSFDocumentManagerToSet = TSFDisabledDocumentManager;

			TCHAR ErrorMsg[1024];
			FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
			UE_LOGF(LogWindowsTextInputMethodSystem, Error, "Getting the active TSF document manager failed, so will fallback to using the disabled document manager. %ls (0x%08x)", ErrorMsg, Result);
		}

		IMMContextToSet = IMMContextId;
	}

	// TSF Implementation
	ITfDocumentMgr* Unused;
	const HRESULT Result = TSFThreadManager->AssociateFocus(Hwnd, TSFDocumentManagerToSet, &Unused);
	if (FAILED(Result))
	{
		TCHAR ErrorMsg[1024];
		FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
		UE_LOGF(LogWindowsTextInputMethodSystem, Error, "Couldn't associate focus between window and TSF document manager, %ls (0x%08x)", ErrorMsg, Result);
	}

	// IMM Implementation
	::ImmAssociateContext(Hwnd, IMMContextToSet);
}

TSharedPtr<ITextInputMethodChangeNotifier> FWindowsTextInputMethodSystem::RegisterContext(const TSharedRef<ITextInputMethodContext>& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_IMEWindowsRegisterContext);
	UE_LOGF(LogWindowsTextInputMethodSystem, Verbose, "Registering context %p...", &(Context.Get()));

	HRESULT Result;

	FInternalContext& InternalContext = ContextToInternalContextMap.Add(Context);

	// TSF Implementation
	TComPtr<FTextStoreACP>& TextStore = InternalContext.TSFContext;
	TextStore.Attach(new FTextStoreACP(Context));

	Result = TSFThreadManager->CreateDocumentMgr(&(TextStore->TSFDocumentManager));
	if(FAILED(Result) || !TextStore->TSFDocumentManager)
	{
		TCHAR ErrorMsg[1024];
		FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
		UE_LOGF(LogWindowsTextInputMethodSystem, Error, "Registering a context failed while creating the TSF document manager. %ls (0x%08x)", ErrorMsg, Result);
		TextStore.Reset();
		return nullptr;
	}

	Result = TextStore->TSFDocumentManager->CreateContext(TSFClientId, 0, static_cast<ITextStoreACP*>(TextStore), &(TextStore->TSFContext), &(TextStore->TSFEditCookie));	
	if(FAILED(Result) || !TextStore->TSFContext)
	{
		TCHAR ErrorMsg[1024];
		FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
		UE_LOGF(LogWindowsTextInputMethodSystem, Error, "Registering a context failed while creating the TSF context. %ls (0x%08x)", ErrorMsg, Result);
		TextStore.Reset();
		return nullptr;
	}

	Result = TextStore->TSFDocumentManager->Push(TextStore->TSFContext);
	if(FAILED(Result))
	{
		TCHAR ErrorMsg[1024];
		FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
		UE_LOGF(LogWindowsTextInputMethodSystem, Error, "Registering a context failed while pushing a TSF context onto the TSF document manager. %ls (0x%08x)", ErrorMsg, Result);
		TextStore.Reset();
		return nullptr;
	}

	Result = TextStore->TSFContextOwnerCompositionServices.FromQueryInterface(IID_ITfContextOwnerCompositionServices, TextStore->TSFContext);
	if(FAILED(Result) || !TextStore->TSFContextOwnerCompositionServices)
	{
		TCHAR ErrorMsg[1024];
		FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
		UE_LOGF(LogWindowsTextInputMethodSystem, Error, "Registering a context failed while getting the TSF context owner composition services. %ls (0x%08x)", ErrorMsg, Result);
		Result = TextStore->TSFDocumentManager->Pop(TF_POPF_ALL);
		if(FAILED(Result))
		{
			FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
			UE_LOGF(LogWindowsTextInputMethodSystem, Error, "Failed to pop a TSF context off from TSF document manager while recovering from failing. %ls (0x%08x)", ErrorMsg, Result);
		}
		TextStore.Reset();
		return nullptr;
	}

	UE_LOGF(LogWindowsTextInputMethodSystem, Verbose, "Registered context %p!", &(Context.Get()));

	return MakeShareable( new FTextInputMethodChangeNotifier(TextStore) );
}

void FWindowsTextInputMethodSystem::UnregisterContext(const TSharedRef<ITextInputMethodContext>& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_IMEWindowsUnregisterContext);
	UE_LOGF(LogWindowsTextInputMethodSystem, Verbose, "Unregistering context %p...", &(Context.Get()));

	HRESULT Result;

	check(ActiveContext != Context);

	check(ContextToInternalContextMap.Contains(Context));
	FInternalContext& InternalContext = ContextToInternalContextMap[Context];

	// TSF Implementation
	TComPtr<FTextStoreACP>& TextStore = InternalContext.TSFContext;

	Result = TextStore->TSFDocumentManager->Pop(TF_POPF_ALL);
	if(FAILED(Result))
	{
		TCHAR ErrorMsg[1024];
		FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
		UE_LOGF(LogWindowsTextInputMethodSystem, Error, "Unregistering a context failed while popping a TSF context off from the TSF document manager. %ls (0x%08x)", ErrorMsg, Result);
	}

	ContextToInternalContextMap.Remove(Context);
	UE_LOGF(LogWindowsTextInputMethodSystem, Verbose, "Unregistered context %p!", &(Context.Get()));
}

void FWindowsTextInputMethodSystem::ActivateContext(const TSharedRef<ITextInputMethodContext>& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_IMEWindowsActivateContext);
	UE_LOGF(LogWindowsTextInputMethodSystem, Verbose, "Activating context %p...", &(Context.Get()));
	HRESULT Result;

	// General Implementation
	ActiveContext = Context;

	check(ContextToInternalContextMap.Contains(Context));
	FInternalContext& InternalContext = ContextToInternalContextMap[Context];

	const TSharedPtr<FGenericWindow> GenericWindow = Context->GetWindow();
	InternalContext.WindowHandle = GenericWindow.IsValid() ? reinterpret_cast<HWND>(GenericWindow->GetOSWindowHandle()) : nullptr;

	if (InternalContext.WindowHandle && ::IsWindowEnabled(InternalContext.WindowHandle))
	{
		// IMM Implementation
		InternalContext.IMMContext.IsComposing = false;
		InternalContext.IMMContext.IsDeactivating = false;
		::ImmAssociateContext(InternalContext.WindowHandle, IMMContextId);

		// TSF Implementation
		TComPtr<FTextStoreACP>& TextStore = InternalContext.TSFContext;
		Result = TSFThreadManager->SetFocus(TextStore->TSFDocumentManager);
		if (FAILED(Result))
		{
			TCHAR ErrorMsg[1024];
			FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
			UE_LOGF(LogWindowsTextInputMethodSystem, Error, "Activating a context failed while setting focus on a TSF document manager. %ls (0x%08x)", ErrorMsg, Result);
		}
		ITfDocumentMgr* Unused;
		Result = TSFThreadManager->AssociateFocus(InternalContext.WindowHandle, TextStore->TSFDocumentManager, &Unused);
		if (FAILED(Result))
		{
			TCHAR ErrorMsg[1024];
			FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
			UE_LOGF(LogWindowsTextInputMethodSystem, Error, "Activating a context failed while associating focus on a TSF document manager. %ls (0x%08x)", ErrorMsg, Result);
		}
		UE_LOGF(LogWindowsTextInputMethodSystem, Verbose, "Activated context %p!", &(Context.Get()));
	}
}

void FWindowsTextInputMethodSystem::DeactivateContext(const TSharedRef<ITextInputMethodContext>& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_IMEWindowsDeactivateContext);
	FInternalContext& InternalContext = ContextToInternalContextMap[Context];

	if (InternalContext.WindowHandle)
	{
		UE_LOGF(LogWindowsTextInputMethodSystem, Verbose, "Deactivating context %p...", &(Context.Get()));

		// IMM Implementation
		HIMC IMMContext = ::ImmGetContext(InternalContext.WindowHandle);
		InternalContext.IMMContext.IsDeactivating = true;
		// Request the composition is completed to ensure that the composition input UI is closed, and that a WM_IME_ENDCOMPOSITION message is sent
		::ImmNotifyIME(IMMContext, NI_COMPOSITIONSTR, CPS_COMPLETE, 0);
		::ImmReleaseContext(InternalContext.WindowHandle, IMMContext);

		// TSF Implementation
		HRESULT Result = TSFThreadManager->SetFocus(TSFDisabledDocumentManager);
		if (FAILED(Result))
		{
			TCHAR ErrorMsg[1024];
			FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Result);
			UE_LOGF(LogWindowsTextInputMethodSystem, Error, "Deactivating a context failed while setting focus on the disabled TSF document manager. %ls (0x%08x)", ErrorMsg, Result);
		}

		// Ensure all known windows are associated with a disabled IME
		ClearStaleWindowHandles();
		for (const TWeakPtr<FGenericWindow>& Window : KnownWindows)
		{
			TSharedPtr<FGenericWindow> WindowPtr = Window.Pin();
			if (WindowPtr.IsValid())
			{
				const HWND Hwnd = reinterpret_cast<HWND>(WindowPtr->GetOSWindowHandle());
				if (Hwnd)
				{
					// TSF Implementation
					ITfDocumentMgr* Unused;
					TSFThreadManager->AssociateFocus(Hwnd, TSFDisabledDocumentManager, &Unused);

					// IMM Implementation
					::ImmAssociateContext(Hwnd, nullptr);
				}
			}
		}
		UE_LOGF(LogWindowsTextInputMethodSystem, Verbose, "Deactivated context %p!", &(Context.Get()));
	}

	// General Implementation
	ActiveContext = nullptr;
}

bool FWindowsTextInputMethodSystem::IsActiveContext(const TSharedRef<ITextInputMethodContext>& Context) const
{
	return ActiveContext == Context;
}

void FWindowsTextInputMethodSystem::OnIMEActivationStateChanged(const bool bIsEnabled)
{
	SCOPE_CYCLE_COUNTER(STAT_IMEWindowsOnIMEActivationStateChanged);
	if(bIsEnabled)
	{
		// It seems that switching away from an IMM based IME doesn't generate a deactivation notification
		//check(CurrentAPI == EAPI::Unknown);

		const HKL KeyboardLayout = ::GetKeyboardLayout(0);

		TF_INPUTPROCESSORPROFILE TSFProfile;
		if(SUCCEEDED(TSFInputProcessorProfileManager->GetActiveProfile(GUID_TFCAT_TIP_KEYBOARD, &TSFProfile)) && TSFProfile.dwProfileType == TF_PROFILETYPE_INPUTPROCESSOR)
		{
			CurrentAPI = EAPI::TSF;
		}
		else if(::ImmGetIMEFileName(KeyboardLayout, nullptr, 0) > 0)
		{
			CurrentAPI = EAPI::IMM;
			UpdateIMMProperty(KeyboardLayout);
		}
		else
		{
			CurrentAPI = EAPI::Unknown;
		}
	}
	else
	{
		// It seems that switching away from an IMM based IME doesn't generate a deactivation notification
		//check(CurrentAPI != EAPI::Unknown);

		CurrentAPI = EAPI::Unknown;
	}

	LogActiveInputMethod();
}

int32 FWindowsTextInputMethodSystem::ProcessMessage(HWND hwnd, uint32 msg, WPARAM wParam, LPARAM lParam)
{
	SCOPE_CYCLE_COUNTER(STAT_IMEWindowsProcessMessage);
	if(CurrentAPI != EAPI::IMM)
	{
		return IntCastChecked<int32>(DefWindowProc(hwnd, msg, wParam, lParam));
	}

	switch(msg)
	{
	case WM_INPUTLANGCHANGEREQUEST:
	case WM_INPUTLANGCHANGE:
		{
			HKL KeyboardLayoutHandle = reinterpret_cast<HKL>(lParam);

			UpdateIMMProperty(KeyboardLayoutHandle);

			return IntCastChecked<int32>(DefWindowProc(hwnd, msg, wParam, lParam));
		}
	case WM_IME_SETCONTEXT:
		{
			if(ActiveContext.IsValid())
			{
				// Disable showing an IME-implemented composition window if we're going to draw the composition string ourselves.
				if(wParam && ShouldDrawIMMCompositionString())
				{
					lParam &= ~ISC_SHOWUICOMPOSITIONWINDOW;
				}

				UE_LOGF(LogWindowsTextInputMethodSystem, Verbose, "Setting IMM context.");
			}

			return IntCastChecked<int32>(DefWindowProc(hwnd, msg, wParam, lParam));
		}
	case WM_IME_NOTIFY:
		{
			return IntCastChecked<int32>(DefWindowProc(hwnd, msg, wParam, lParam));
		}
	case WM_IME_REQUEST:
		{
			return IntCastChecked<int32>(DefWindowProc(hwnd, msg, wParam, lParam));
		}
	case WM_IME_STARTCOMPOSITION:
		{
			if(ActiveContext.IsValid())
			{
				BeginIMMComposition();
				
				UE_LOGF(LogWindowsTextInputMethodSystem, Verbose, "Beginning IMM composition.");
			}

			return IntCastChecked<int32>(DefWindowProc(hwnd, msg, wParam, lParam));
		}
	case WM_IME_COMPOSITION:
		{
			if(ActiveContext.IsValid())
			{
				FInternalContext& InternalContext = ContextToInternalContextMap[ActiveContext];
				
				// Not all IMEs trigger a call of WM_IME_STARTCOMPOSITION
				if(!InternalContext.IMMContext.IsComposing)
				{
					BeginIMMComposition();
				}

				HIMC IMMContext = ::ImmGetContext(hwnd);

				UpdateIMMWindowPositions(IMMContext);

				const bool bHasBeenCanceled = !lParam;
				const bool bHasCompositionStringFlag = !!(lParam & GCS_COMPSTR);
				const bool bHasResultStringFlag = !!(lParam & GCS_RESULTSTR);
				const bool bHasNoMoveCaretFlag = !!(lParam & CS_NOMOVECARET);
				const bool bHasCursorPosFlag = !!(lParam & GCS_CURSORPOS);

				// Canceled, so remove the compositing string
				if(bHasBeenCanceled)
				{
					CancelIMMComposition();
				}

				// Check Result
				if(bHasResultStringFlag)
				{
					// If we're being deactivated, so we need to take the current selection so we can restore it properly
					// otherwise calling SetTextInRange can cause the cursor/selection to jump around
					uint32 SelectionBeginIndex = 0;
					uint32 SelectionLength = 0;
					ITextInputMethodContext::ECaretPosition SelectionCaretPosition = ITextInputMethodContext::ECaretPosition::Ending;
					ActiveContext->GetSelectionRange(SelectionBeginIndex, SelectionLength, SelectionCaretPosition);

					const FString ResultString = GetIMMStringAsFString(IMMContext, GCS_RESULTSTR);
					UE_LOGF(LogWindowsTextInputMethodSystem, Verbose, "WM_IME_COMPOSITION Result String: %ls", *ResultString);

					// Update Result
					ActiveContext->SetTextInRange(InternalContext.IMMContext.CompositionBeginIndex, InternalContext.IMMContext.CompositionLength, ResultString);

					if(InternalContext.IMMContext.IsDeactivating)
					{
						// Restore any previous selection
						ActiveContext->SetSelectionRange(SelectionBeginIndex, SelectionLength, SelectionCaretPosition);
					}
					else
					{
						// Once we get a result, we're done; set the caret to the end of the result and end the current composition
						ActiveContext->SetSelectionRange(InternalContext.IMMContext.CompositionBeginIndex + ResultString.Len(), 0, ITextInputMethodContext::ECaretPosition::Ending);
					}

					EndIMMComposition();
				}

				// Check Composition
				if(bHasCompositionStringFlag)
				{
					const FString CompositionString = GetIMMStringAsFString(IMMContext, GCS_COMPSTR);
					UE_LOGF(LogWindowsTextInputMethodSystem, Verbose, "WM_IME_COMPOSITION Composition String: %ls", *CompositionString);

					// Not all IMEs send a cancel request when you press escape, but instead just set the string to empty
					// We need to cancel out the composition string here to avoid weirdness when you start typing again
					// Don't do this if we have a result string, as we'll have already called EndIMMComposition to finish the composition
					if(CompositionString.Len() == 0 && !bHasResultStringFlag)
					{
						CancelIMMComposition();
					}

					// We've typed a character, so we need to clear out any currently selected text to mimic what happens when you normally type into a text input
					uint32 SelectionBeginIndex = 0;
					uint32 SelectionLength = 0;
					ITextInputMethodContext::ECaretPosition SelectionCaretPosition = ITextInputMethodContext::ECaretPosition::Ending;
					ActiveContext->GetSelectionRange(SelectionBeginIndex, SelectionLength, SelectionCaretPosition);
					if(SelectionLength)
					{
						ActiveContext->SetTextInRange(SelectionBeginIndex, SelectionLength, TEXT(""));
					}

					// If we received a result (handled above) then the previous composition will have been ended, so we need to start a new one now
					// This ensures that each composition ends up as its own distinct undo
					if(!InternalContext.IMMContext.IsComposing)
					{
						BeginIMMComposition();
					}

					const int32 CurrentCompositionBeginIndex = InternalContext.IMMContext.CompositionBeginIndex;
					const uint32 CurrentCompositionLength = InternalContext.IMMContext.CompositionLength;

					// Update Composition Range
					InternalContext.IMMContext.CompositionLength = CompositionString.Len();
					ActiveContext->UpdateCompositionRange(InternalContext.IMMContext.CompositionBeginIndex, InternalContext.IMMContext.CompositionLength);

					// Update Composition
					ActiveContext->SetTextInRange(CurrentCompositionBeginIndex, CurrentCompositionLength, CompositionString);
				}

				// Check Cursor
				if(!bHasNoMoveCaretFlag && bHasCursorPosFlag)
				{
					const LONG CursorPositionResult = ::ImmGetCompositionString(IMMContext, GCS_CURSORPOS, nullptr, 0);
					const int16 CursorPosition = CursorPositionResult & 0xFFFF;

					// Update Cursor
					UE_LOGF(LogWindowsTextInputMethodSystem, Verbose, "WM_IME_COMPOSITION Cursor Position: %d", CursorPosition);
					ActiveContext->SetSelectionRange(InternalContext.IMMContext.CompositionBeginIndex + CursorPosition, 0, ITextInputMethodContext::ECaretPosition::Ending);
				}

				::ImmReleaseContext(hwnd, IMMContext);

				UE_LOGF(LogWindowsTextInputMethodSystem, Verbose, "Updating IMM composition.");
			}

			return IntCastChecked<int32>(DefWindowProc(hwnd, msg, wParam, lParam));
		}
	case WM_IME_ENDCOMPOSITION:
		{
			// On composition end, notify context of the end.
			if(ActiveContext.IsValid())
			{
				EndIMMComposition();
				
				UE_LOGF(LogWindowsTextInputMethodSystem, Verbose, "Ending IMM composition.");
			}

			return IntCastChecked<int32>(DefWindowProc(hwnd, msg, wParam, lParam));
		}
	case WM_IME_CHAR:
		{
			// Suppress sending a WM_CHAR for this WM_IME_CHAR - the composition windows messages will have handled this.
			UE_LOGF(LogWindowsTextInputMethodSystem, Verbose, "Ignoring WM_IME_CHAR message.");
			return 0;
		}
	default:
		{
			UE_LOGF(LogWindowsTextInputMethodSystem, Warning, "Unexpected windows message received for processing.");

			return IntCastChecked<int32>(DefWindowProc(hwnd, msg, wParam, lParam));
		}
	}
}

#include "Windows/HideWindowsPlatformTypes.h"
