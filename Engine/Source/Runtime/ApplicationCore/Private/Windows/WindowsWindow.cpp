// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsWindow.h"

#include "Algo/Transform.h"
#include "Math/UnrealMathUtility.h"
#include "HAL/UnrealMemory.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Math/Vector2D.h"
#include "GenericPlatform/GenericWindowDefinition.h"
#include "Windows/WindowsApplication.h"
#include "HAL/ThreadHeartBeat.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/PlatformProcess.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include <dwmapi.h>
#include <ShlObj.h>
#include "Windows/HideWindowsPlatformTypes.h"

#undef IsZoomed
#undef IsMaximized
#undef IsMinimized

// Flag which guards an embedded engine instance overriding anything other than the first window.
static bool GOverrideNextWnd = true;

FWindowsWindow::~FWindowsWindow()
{
	// NOTE: The HWnd is invalid here!
	//       Doing stuff with HWnds will fail silently.
	//       Use Destroy() instead.
}


const TCHAR FWindowsWindow::AppWindowClass[] = TEXT("UnrealWindow");

// This is a hard-coded constant for Windows' WS_THICKFRAME
// There does not seem to be a way to alter this
static int32 WindowsAeroBorderSize = 8;
static int32 WindowsStandardBorderSize = 4;


TSharedRef< FWindowsWindow > FWindowsWindow::Make()
{
	// First, allocate the new native window object.  This doesn't actually create a native window or anything,
	// we're simply instantiating the object so that we can keep shared references to it.
	return MakeShareable( new FWindowsWindow() );
}

void FWindowsWindow::Initialize( FWindowsApplication* const Application, const TSharedRef< FGenericWindowDefinition >& InDefinition, HINSTANCE InHInstance, const TSharedPtr< FWindowsWindow >& InParent, const bool bShowImmediately )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWindowsWindow::Initialize);

	Definition = InDefinition;
	OwningApplication = Application;

	// Finally, let's initialize the new native window object.  Calling this function will often cause OS
	// window messages to be sent! (such as activation messages)
	uint32 WindowExStyle = 0;
	uint32 WindowStyle = 0;

	RegionWidth = RegionHeight = INDEX_NONE;

	const float XInitialRect = Definition->XDesiredPositionOnScreen;
	const float YInitialRect = Definition->YDesiredPositionOnScreen;

	const float WidthInitial = Definition->WidthDesiredOnScreen;
	const float HeightInitial = Definition->HeightDesiredOnScreen;

	DPIScaleFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(XInitialRect, YInitialRect);

	int32 ClientX = FMath::TruncToInt( XInitialRect );
	int32 ClientY = FMath::TruncToInt( YInitialRect );
	int32 ClientWidth = FMath::TruncToInt( WidthInitial );
	int32 ClientHeight = FMath::TruncToInt( HeightInitial );
	int32 WindowX = ClientX;
	int32 WindowY = ClientY;
	int32 WindowWidth = ClientWidth;
	int32 WindowHeight = ClientHeight;
	const bool bApplicationSupportsPerPixelBlending =
#if ALPHA_BLENDED_WINDOWS
		Application->GetWindowTransparencySupport() == EWindowTransparency::PerPixel;
#else
		false;
#endif

	if( !Definition->HasOSWindowBorder )
	{
		WindowExStyle = WS_EX_WINDOWEDGE;

		if( Definition->TransparencySupport == EWindowTransparency::PerWindow )
		{
			WindowExStyle |= WS_EX_LAYERED;
		}
#if ALPHA_BLENDED_WINDOWS
		else if( Definition->TransparencySupport == EWindowTransparency::PerPixel )
		{
			if( bApplicationSupportsPerPixelBlending )
			{
				WindowExStyle |= WS_EX_COMPOSITED;
			}
		}
#endif

		WindowStyle = WS_POPUP | WS_CLIPCHILDREN | WS_CLIPSIBLINGS;
		if ( Definition->AppearsInTaskbar )
		{
			WindowExStyle |= WS_EX_APPWINDOW;
		}
		else
		{
			WindowExStyle |= WS_EX_TOOLWINDOW;
		}

		if( Definition->IsTopmostWindow )
		{
			// Tool tips are always top most windows
			WindowExStyle |= WS_EX_TOPMOST;
		}

		if ( !Definition->AcceptsInput )
		{
			// Window should never get input
			WindowExStyle |= WS_EX_TRANSPARENT;
		}
	}
	else
	{
		// OS Window border setup
		WindowExStyle = WS_EX_APPWINDOW;
		WindowStyle = WS_OVERLAPPED | WS_SYSMENU | WS_CAPTION;

		if (IsRegularWindow())
		{
			if (Definition->SupportsMaximize)
			{
				WindowStyle |= WS_MAXIMIZEBOX;
			}

			if (Definition->SupportsMinimize)
			{
				WindowStyle |= WS_MINIMIZEBOX;
			}

			if (Definition->HasSizingFrame)
			{
				WindowStyle |= WS_THICKFRAME;
			}
			else
			{
				WindowStyle |= WS_BORDER;
			}
		}
		else
		{
			WindowStyle |= WS_POPUP | WS_BORDER;
		}

		// X,Y, Width, Height defines the top-left pixel of the client area on the screen
		// This adjusts a zero rect to give us the size of the border
		RECT BorderRect = { 0, 0, 0, 0 };
		::AdjustWindowRectEx(&BorderRect, WindowStyle, false, WindowExStyle);

		// Border rect size is negative - see MoveWindowTo
		WindowX += BorderRect.left;
		WindowY += BorderRect.top;

		// Inflate the window size by the OS border
		WindowWidth += BorderRect.right - BorderRect.left;
		WindowHeight += BorderRect.bottom - BorderRect.top;
	}

	// In embedded mode, use the provided window handle instead of creating a new window
	// and store the current client area dimensions for use later.
	if (GUELibraryOverrideSettings.bIsEmbedded && GUELibraryOverrideSettings.WindowHandle && GOverrideNextWnd)
	{
		HWnd = *(HWND*)&GUELibraryOverrideSettings.WindowHandle;
		GOverrideNextWnd = false;

		RECT rect;
		::GetClientRect(HWnd, &rect);
		RegionWidth  = VirtualWidth = rect.right - rect.left;
		RegionHeight = VirtualHeight = rect.bottom - rect.top;

		GUELibraryOverrideSettings.WindowWidth  = RegionWidth;
		GUELibraryOverrideSettings.WindowHeight = RegionHeight;

		return;
	}

	// Creating the Window
	HWnd = CreateWindowEx(
		WindowExStyle,
		AppWindowClass,
		*Definition->Title,
		WindowStyle,
		WindowX, WindowY, 
		WindowWidth, WindowHeight,
		( InParent.IsValid() ) ? static_cast<HWND>( InParent->HWnd ) : NULL,
		NULL, InHInstance, NULL);

	if (HWnd == NULL)
	{
		// Make sure the call to GetLastError happens immediately after CreateWindowEx to capture the proper error context for that API call.
		const uint32 Error = GetLastError();

		FSlowHeartBeatScope SuspendHeartBeat;

		// Get the number of handles along with global and per process GDI and User Objects to give an idea of why the problem is happening.
		// A large number of windows has been known to cause window creation to fail because windows only allows so many.
		DWORD NumHandles = 0;
		GetProcessHandleCount(GetCurrentProcess(), &NumHandles);
		checkf(0, TEXT("Window Creation Failed (Error Code %d). \n  Current Process (Handles: %d, GDI Objects: %d, User Objects: %d)\n  Global (GDI Objects: %d, User Objects: %d)"),
			Error, NumHandles,
			GetGuiResources(GetCurrentProcess(), GR_GDIOBJECTS),
			GetGuiResources(GetCurrentProcess(), GR_USEROBJECTS),
			GetGuiResources(GR_GLOBAL, GR_GDIOBJECTS),
			GetGuiResources(GR_GLOBAL, GR_USEROBJECTS)
			);

		return;
	}

	if ( RegisterTouchWindow( HWnd, 0 ) == false )
	{
		uint32 Error = GetLastError();
		UE_LOGF(LogWindows, Warning, "Register touch input failed!");
	}

	bool bDisableTouchFeedback;
	GConfig->GetBool(TEXT("WindowsApplication.Accessibility"), TEXT("DisableTouchFeedback"), bDisableTouchFeedback, GEngineIni);
	if (bDisableTouchFeedback)
	{
		DisableTouchFeedback();
	}

	VirtualWidth = ClientWidth;
	VirtualHeight = ClientHeight;

	// We call reshape window here because we didn't take into account the non-client area
	// in the initial creation of the window. Slate should only pass client area dimensions.
	// Reshape window may resize the window if the non-client area is encroaching on our
	// desired client area space.
	ReshapeWindow( ClientX, ClientY, ClientWidth, ClientHeight );


	if ( Definition->TransparencySupport == EWindowTransparency::PerWindow )
	{
		SetOpacity( Definition->Opacity );
	}

	// Disable DWM Rendering and Nonclient Area painting if not showing the os window border
	// This prevents the standard windows frame from ever being drawn
	if( !Definition->HasOSWindowBorder )
	{
		const DWMNCRENDERINGPOLICY RenderingPolicy = DWMNCRP_DISABLED;
		verify(SUCCEEDED(DwmSetWindowAttribute(HWnd, DWMWA_NCRENDERING_POLICY, &RenderingPolicy, sizeof(RenderingPolicy))));

		const BOOL bEnableNCPaint = false;
		verify(SUCCEEDED(DwmSetWindowAttribute(HWnd, DWMWA_ALLOW_NCPAINT, &bEnableNCPaint, sizeof(bEnableNCPaint))));

	#if ALPHA_BLENDED_WINDOWS
		if ( bApplicationSupportsPerPixelBlending && Definition->TransparencySupport == EWindowTransparency::PerPixel )
		{
			MARGINS Margins = {-1};
			verify(SUCCEEDED(::DwmExtendFrameIntoClientArea(HWnd, &Margins)));
		}
	#endif
	}

	// No region for non regular windows or windows displaying the os window border
	if ( IsRegularWindow() && !Definition->HasOSWindowBorder )
	{
		WindowStyle |= WS_OVERLAPPED | WS_CAPTION | WS_SYSMENU;
		if ( Definition->SupportsMaximize )
		{
			WindowStyle |= WS_MAXIMIZEBOX;
		}
		if ( Definition->SupportsMinimize )
		{
			WindowStyle |= WS_MINIMIZEBOX;
		}
		if ( Definition->HasSizingFrame )
		{
			WindowStyle |= WS_THICKFRAME;
		}

		verify(SetWindowLong(HWnd, GWL_STYLE, WindowStyle));

		uint32 SetWindowPositionFlags = SWP_NOMOVE | SWP_NOSIZE | SWP_NOZORDER | SWP_FRAMECHANGED;

		if ( Definition->ActivationPolicy == EWindowActivationPolicy::Never )
		{
			SetWindowPositionFlags |= SWP_NOACTIVATE;
		}

		::SetWindowPos(HWnd, nullptr, 0, 0, 0, 0, SetWindowPositionFlags);
		
		// For regular non-game windows delete the close menu from the default system menu. This prevents accidental closing of win32 apps by double clicking by accident on the application icon
		// The overwhelming majority of feedback is that is confusing behavior and we want to prevent this.
		::DeleteMenu(GetSystemMenu(HWnd, false), SC_CLOSE, MF_BYCOMMAND);

		AdjustWindowRegion( ClientWidth, ClientHeight );
	}
	else if (Definition->HasOSWindowBorder)
	{
		if (!Definition->HasCloseButton)
		{
			EnableMenuItem(GetSystemMenu(HWnd, false), SC_CLOSE, MF_GRAYED);
		}
	}

	if ( IsRegularWindow() )
	{
		// Tell OLE that we are opting into drag and drop.
		// Only makes sense for regular windows (windows that last a while.)
		RegisterDragDrop( HWnd, this );
		
		// Listing to clipboard change event
		::AddClipboardFormatListener(HWnd);
	}
}

FWindowsWindow::FWindowsWindow()
	: HWnd(NULL)
	, WindowMode( EWindowMode::Windowed )
	, OLEReferenceCount(0)
	, AspectRatio(1.0f)
	, bIsVisible(false)
	, bIsFirstTimeVisible(true)
	, bInitiallyMinimized(false)
	, bInitiallyMaximized(false)
	, DPIScaleFactor(1.0f)
{
	// PreFullscreenWindowPlacement.length will be set when we save the window placement and then used to check if the structure is valid
	FMemory::Memzero(PreFullscreenWindowPlacement);

	FMemory::Memzero(PreParentMinimizedWindowPlacement);
	PreParentMinimizedWindowPlacement.length = sizeof(WINDOWPLACEMENT);
}

HWND FWindowsWindow::GetHWnd() const
{
	return HWnd;
}

void FWindowsWindow::OnTransparencySupportChanged(EWindowTransparency NewTransparency)
{
#if ALPHA_BLENDED_WINDOWS
	if ( Definition->TransparencySupport == EWindowTransparency::PerPixel )
	{
		const auto Style = GetWindowLong( HWnd, GWL_EXSTYLE );

		if( NewTransparency == EWindowTransparency::PerPixel )
		{
			SetWindowLong( HWnd, GWL_EXSTYLE, Style | WS_EX_COMPOSITED );

			MARGINS Margins = {-1};
			verify(SUCCEEDED(::DwmExtendFrameIntoClientArea(HWnd, &Margins)));
		}
		else
		{
			SetWindowLong( HWnd, GWL_EXSTYLE, Style & ~WS_EX_COMPOSITED );
		}

		// Must call SWP_FRAMECHANGED when updating the style attribute of a window in order to update internal caches (according to MSDN)
		SetWindowPos( HWnd, NULL, 0, 0, 0, 0, SWP_FRAMECHANGED | SWP_NOACTIVATE | SWP_NOMOVE | SWP_NOOWNERZORDER | SWP_NOREDRAW | SWP_NOSIZE | SWP_NOSENDCHANGING | SWP_NOZORDER );
	}
#endif
}

HRGN FWindowsWindow::MakeWindowRegionObject(bool bForQuery) const
{
	int X1, Y1, X2, Y2;
	if ( RegionWidth != INDEX_NONE && RegionHeight != INDEX_NONE )
	{
		const bool bIsBorderless = !GetDefinition().HasOSWindowBorder;
		if (IsMaximized())
		{
			// Maximized windows are forcibly oversized by cxWindowBorders, so the rect we pass to SetWindowRgn will be interpreted as relative to
			// -cxWindowBorders pixels from the screen's corner. We don't want anything being drawn on adjacent monitors, so we need to offset the rgn
			// by this much (unless it is being used for a query against a cursor position that we have already remapped to client-relative).
			int32 BorderSize = bForQuery ? 0 : GetWin32BorderSize();
			X1 = BorderSize;
			Y1 = BorderSize;
			X2 = BorderSize + RegionWidth;
			Y2 = BorderSize + RegionHeight;
		}
		else
		{
			const bool bUseCornerRadius = WindowMode == EWindowMode::Windowed && !bIsBorderless &&
#if ALPHA_BLENDED_WINDOWS
				// Corner radii cause DWM window composition blending to fail, so we always set regions to full size rectangles
				Definition->TransparencySupport != EWindowTransparency::PerPixel &&
#endif
				Definition->CornerRadius > 0;

			if( bUseCornerRadius )
			{
				// CreateRoundRectRgn gives you a duff region that's 1 pixel smaller than you ask for. CreateRectRgn behaves correctly.
				// This can be verified by uncommenting the assert below
				HRGN Region = CreateRoundRectRgn( 0, 0, RegionWidth+1, RegionHeight+1, Definition->CornerRadius, Definition->CornerRadius );
				if (!Region)
				{
					UE_LOGF(LogWindows, Error, "CreateRoundRectRgn for 0,0,%d+1,%d+1,%d,%d failed with error %d",
						RegionWidth, RegionHeight, Definition->CornerRadius, Definition->CornerRadius, GetLastError());
				}

				// Test that a point that should be in the region, is in the region
				// check(!!PtInRegion(Region, RegionWidth-1, RegionHeight/2));
				return Region;
			}
			else
			{
				X1 = 0;
				Y1 = 0;
				X2 = RegionWidth;
				Y2 = RegionHeight;
			}
		}
	}
	else
	{
		RECT rcWnd;
		GetWindowRect( HWnd, &rcWnd );
		X1 = 0;
		Y1 = 0;
		X2 = rcWnd.right - rcWnd.left;
		Y2 = rcWnd.bottom - rcWnd.top;
	}

	HRGN Region = CreateRectRgn(X1, Y1, X2, Y2);
	if (!Region)
	{
		UE_LOGF(LogWindows, Error, "CreateRectRgn failed for %d,%d,%d,%d with error %d", X1, Y1, X2, Y2, GetLastError());
	}
	return Region;
}

void FWindowsWindow::DisableTouchFeedback()
{
	if (void* User32Dll = GetModuleHandle(L"user32.dll"))
	{
		typedef enum tagWINVER602FEEDBACK_TYPE
		{
			WINVER602_FEEDBACK_TOUCH_CONTACTVISUALIZATION = 1,
			WINVER602_FEEDBACK_PEN_BARRELVISUALIZATION = 2,
			WINVER602_FEEDBACK_PEN_TAP = 3,
			WINVER602_FEEDBACK_PEN_DOUBLETAP = 4,
			WINVER602_FEEDBACK_PEN_PRESSANDHOLD = 5,
			WINVER602_FEEDBACK_PEN_RIGHTTAP = 6,
			WINVER602_FEEDBACK_TOUCH_TAP = 7,
			WINVER602_FEEDBACK_TOUCH_DOUBLETAP = 8,
			WINVER602_FEEDBACK_TOUCH_PRESSANDHOLD = 9,
			WINVER602_FEEDBACK_TOUCH_RIGHTTAP = 10,
			WINVER602_FEEDBACK_GESTURE_PRESSANDTAP = 11,
			WINVER602_FEEDBACK_MAX = 0xFFFFFFFF
		} WINVER602FEEDBACK_TYPE;

		typedef BOOL(*SetWindowFeedbackSettingProc)(_In_ HWND hwnd,
												_In_ WINVER602FEEDBACK_TYPE feedback,
												_In_ DWORD dwFlags,
												_In_ UINT32 size,
												_In_reads_bytes_opt_(size) CONST VOID* configuration);

		//Manually fetching SetWindowFeedbackSetting since it is only available in WINVER 0x0602 (Windows 8) and we are currently using WINVER 0x0601 (Windows 7)
		SetWindowFeedbackSettingProc SetWindowFeedbackSetting = (SetWindowFeedbackSettingProc)FPlatformProcess::GetDllExport(User32Dll, TEXT("SetWindowFeedbackSetting"));
		if (SetWindowFeedbackSetting)
		{
			BOOL enabled = 0;
			SetWindowFeedbackSetting(HWnd, WINVER602_FEEDBACK_TOUCH_CONTACTVISUALIZATION, 0, sizeof(enabled), &enabled);
			SetWindowFeedbackSetting(HWnd, WINVER602_FEEDBACK_TOUCH_TAP, 0, sizeof(enabled), &enabled);
			SetWindowFeedbackSetting(HWnd, WINVER602_FEEDBACK_TOUCH_PRESSANDHOLD, 0, sizeof(enabled), &enabled);
		}
	}
}

void FWindowsWindow::AdjustWindowRegion( int32 Width, int32 Height )
{
	RegionWidth = Width;
	RegionHeight = Height;

	const bool bForQuery = false;
	HRGN Region = MakeWindowRegionObject(bForQuery);
	if (!Region)
	{
		return;
	}

	// NOTE: We explicitly don't delete the Region object, because the OS deletes the handle when it no longer needed after
	// a call to SetWindowRgn.
	verify( SetWindowRgn( HWnd, Region, false ) );
}

void FWindowsWindow::ReshapeWindow( int32 NewX, int32 NewY, int32 NewWidth, int32 NewHeight )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWindowsWindow::ReshapeWindow);

	WINDOWINFO WindowInfo;
	FMemory::Memzero( WindowInfo );
	WindowInfo.cbSize = sizeof( WindowInfo );
	::GetWindowInfo( HWnd, &WindowInfo );

	AspectRatio = (float)NewWidth / (float)NewHeight;

	// Don't resize the window if it's been embedded inside another application, the
	// outer application will take care of it.
	if (GUELibraryOverrideSettings.bIsEmbedded && HWnd == *(HWND*)&GUELibraryOverrideSettings.WindowHandle && WindowMode == EWindowMode::Windowed)
	{
		RegionWidth  = VirtualWidth = NewWidth;
		RegionHeight = VirtualHeight = NewHeight;
		return;
	}

	// X,Y, Width, Height defines the top-left pixel of the client area on the screen
	if( Definition->HasOSWindowBorder )
	{
		// This adjusts a zero rect to give us the size of the border
		RECT BorderRect = { 0, 0, 0, 0 };
		::AdjustWindowRectEx(&BorderRect, WindowInfo.dwStyle, false, WindowInfo.dwExStyle);

		// Border rect size is negative - see MoveWindowTo
		NewX += BorderRect.left;
		NewY += BorderRect.top;

		// Inflate the window size by the OS border
		NewWidth += BorderRect.right - BorderRect.left;
		NewHeight += BorderRect.bottom - BorderRect.top;
	}

	// the window position is the requested position
	int32 WindowX = NewX;
	int32 WindowY = NewY;
	
	// If the window size changes often, only grow the window, never shrink it
	const bool bVirtualSizeChanged = NewWidth != VirtualWidth || NewHeight != VirtualHeight;
	VirtualWidth = NewWidth;
	VirtualHeight = NewHeight;

	if( Definition->SizeWillChangeOften )
	{
		// When SizeWillChangeOften is set, we set a minimum width and height window size that we'll keep allocated
		// even when the requested actual window size is smaller.  This just avoids constantly resizing the window
		// and associated GPU buffers, which can be very slow on some platforms.

		const RECT OldWindowRect = WindowInfo.rcWindow;
		const int32 OldWidth = OldWindowRect.right - OldWindowRect.left;
		const int32 OldHeight = OldWindowRect.bottom - OldWindowRect.top;

		const int32 MinRetainedWidth = Definition->ExpectedMaxWidth != INDEX_NONE ? Definition->ExpectedMaxWidth : OldWidth;
		const int32 MinRetainedHeight = Definition->ExpectedMaxHeight != INDEX_NONE ? Definition->ExpectedMaxHeight : OldHeight;

		NewWidth = FMath::Max( NewWidth, FMath::Min( OldWidth, MinRetainedWidth ) );
		NewHeight = FMath::Max( NewHeight, FMath::Min( OldHeight, MinRetainedHeight ) );
	}

	if (IsMaximized())
	{
		Restore();
	}

	// We use SWP_NOSENDCHANGING when in fullscreen mode to prevent Windows limiting our window size to the current resolution, as that 
	// prevents us being able to change to a higher resolution while in fullscreen mode
	::SetWindowPos( HWnd, nullptr, WindowX, WindowY, NewWidth, NewHeight, SWP_NOZORDER | SWP_NOACTIVATE | ((WindowMode == EWindowMode::Fullscreen) ? SWP_NOSENDCHANGING : 0) );
	
	bool bAdjustSizeChange = Definition->SizeWillChangeOften || bVirtualSizeChanged;
	bool bAdjustCorners = Definition->Type != EWindowType::Menu && Definition->CornerRadius > 0;

	if (!Definition->HasOSWindowBorder && (bAdjustSizeChange || bAdjustCorners))
	{
		AdjustWindowRegion( VirtualWidth, VirtualHeight );
	}
}

bool FWindowsWindow::GetFullScreenInfo( int32& X, int32& Y, int32& Width, int32& Height ) const
{
	// Grab current monitor data for sizing
	HMONITOR Monitor = MonitorFromWindow( HWnd, MONITOR_DEFAULTTONEAREST );
	MONITORINFO MonitorInfo;
	MonitorInfo.cbSize = sizeof(MONITORINFO);
	GetMonitorInfo( Monitor, &MonitorInfo );

	X = MonitorInfo.rcMonitor.left;
	Y = MonitorInfo.rcMonitor.top;
	Width = MonitorInfo.rcMonitor.right - X;
	Height = MonitorInfo.rcMonitor.bottom - Y;

	return true;
}

/** Native windows should implement MoveWindowTo by relocating the client area of the platform-specific window to (X,Y). */
void FWindowsWindow::MoveWindowTo( int32 X, int32 Y )
{
	// Slate gives the window position as relative to the client area of a window, so we may need to compensate for the OS border
	if (Definition->HasOSWindowBorder)
	{
		const LONG WindowStyle = ::GetWindowLong(HWnd, GWL_STYLE);
		const LONG WindowExStyle = ::GetWindowLong(HWnd, GWL_EXSTYLE);

		// This adjusts a zero rect to give us the size of the border
		RECT BorderRect = { 0, 0, 0, 0 };
		::AdjustWindowRectEx(&BorderRect, WindowStyle, false, WindowExStyle);

		// Border rect size is negative
		X += BorderRect.left;
		Y += BorderRect.top;
	}

	::SetWindowPos(HWnd, nullptr, X, Y, 0, 0, SWP_NOACTIVATE | SWP_NOSIZE | SWP_NOZORDER);
}

/** Native windows should implement BringToFront by making this window the top-most window (i.e. focused).
 *
 * @param bForce	Forces the window to the top of the Z order, even if that means stealing focus from other windows
 *					In general do not pass true for this.  It is really only useful for some windows, like game windows where not forcing it to the front
 *					could cause mouse capture and mouse lock to happen but without the window visible
 */
void FWindowsWindow::BringToFront( bool bForce )
{
	if ( IsRegularWindow() )
	{
		if (::IsIconic(HWnd))
		{
			::ShowWindow(HWnd, SW_RESTORE);
		}
		else
		{
			::SetActiveWindow(HWnd);
		}
	}
	else
	{
		HWND HWndInsertAfter = HWND_TOP;
		// By default we activate the window or it isn't actually brought to the front 
		uint32 Flags = SWP_NOMOVE | SWP_NOSIZE | SWP_NOOWNERZORDER;

		if( !bForce )
		{
			Flags |= SWP_NOACTIVATE;
		}

		if( Definition->IsTopmostWindow )
		{
			HWndInsertAfter = HWND_TOPMOST;
		}

		::SetWindowPos( HWnd, HWndInsertAfter, 0, 0, 0, 0, Flags );
	}
}

void FWindowsWindow::HACK_ForceToFront()
{
	::SetForegroundWindow(HWnd);
}

/** Native windows should implement this function by asking the OS to destroy OS-specific resource associated with the window (e.g. Win32 window handle) */
void FWindowsWindow::Destroy()
{
	if (OLEReferenceCount > 0 && IsWindow(HWnd))
	{
		HRESULT Result = RevokeDragDrop(HWnd);
		// If we decremented OLEReferenceCount check it for being null (shutdown)
		if (Result == S_OK)
		{
			ensureMsgf(OLEReferenceCount == 0, TEXT("Not all references to window are released, %i left"), OLEReferenceCount);
		}
	}

	::DestroyWindow( HWnd );
	// Null HWnd so callers (e.g. GetOSWindowHandle) can't observe a stale handle after the
	// OS window is gone. Without this, CreateWindowEx can receive the dead handle as a parent
	// and fail with ERROR_INVALID_WINDOW_HANDLE (1400).
	HWnd = nullptr;
}

/** Native window should implement this function by performing the equivalent of the Win32 minimize-to-taskbar operation */
void FWindowsWindow::Minimize()
{
	// Windows window handles the initial show state on the first Show call in order to handle activation policy.
	// So we only call now if that already happened.
	if (!bIsFirstTimeVisible)
	{
		::ShowWindow(HWnd, SW_MINIMIZE);
	}
	else
	{
		bInitiallyMinimized = true;
		bInitiallyMaximized = false;
	}
}

/** Native window should implement this function by performing the equivalent of the Win32 maximize operation */
void FWindowsWindow::Maximize()
{
	// Windows window handles the initial show state on the first Show call in order to handle activation policy.
	// So we only call now if that already happened.
	if (!bIsFirstTimeVisible)
	{
		::ShowWindow(HWnd, SW_MAXIMIZE);
	}
	else
	{
		bInitiallyMaximized = true;
		bInitiallyMinimized = false;
	}
}

/** Native window should implement this function by performing the equivalent of the Win32 maximize operation */
void FWindowsWindow::Restore()
{
	// Windows window handles the initial show state on the first Show call in order to handle activation policy.
	// So we only call now if that already happened.
	if (!bIsFirstTimeVisible)
	{
		::ShowWindow(HWnd, SW_RESTORE);
	}
	else
	{
		bInitiallyMaximized = false;
		bInitiallyMinimized = false;
	}
}

/** Native window should make itself visible */
void FWindowsWindow::Show()
{
	if (!bIsVisible)
	{
		bIsVisible = true;

		// Should the show command include activation?
		// Do not activate windows that do not take input; e.g. tool-tips and cursor decorators
		bool bShouldActivate = false;
		if (Definition->AcceptsInput)
		{
			bShouldActivate = Definition->ActivationPolicy == EWindowActivationPolicy::Always;
			if (bIsFirstTimeVisible && Definition->ActivationPolicy == EWindowActivationPolicy::FirstShown)
			{
				bShouldActivate = true;
			}
		}

		// Map to the relevant ShowWindow command.
		int ShowWindowCommand = bShouldActivate ? SW_SHOW : SW_SHOWNOACTIVATE;
		if (bIsFirstTimeVisible)
		{
			bIsFirstTimeVisible = false;
			if (bInitiallyMinimized)
			{
				ShowWindowCommand = bShouldActivate ? SW_MINIMIZE : SW_SHOWMINNOACTIVE;
			}
			else if (bInitiallyMaximized)
			{
				ShowWindowCommand = bShouldActivate ? SW_SHOWMAXIMIZED : SW_MAXIMIZE;
			}
		}

		::ShowWindow(HWnd, ShowWindowCommand);

		// Turns out SW_SHOWNA doesn't work correctly if the window has never been shown before.  If the window
		// was already maximized, (and hidden) and we're showing it again, SW_SHOWNA would be right.  But it's not right
		// to use SW_SHOWNA when the window has never been shown before!
		// 
		// TODO Add in a more complicated path that involves SW_SHOWNA if we hide windows in their maximized/minimized state.
		//::ShowWindow(HWnd, bShouldActivate ? SW_SHOW : SW_SHOWNA);
	}
}

/** Native window should hide itself */
void FWindowsWindow::Hide()
{
	if (bIsVisible)
	{
		bIsVisible = false;
		::ShowWindow(HWnd, SW_HIDE);
	}
}

/** Toggle native window between fullscreen and normal mode */
void FWindowsWindow::SetWindowMode( EWindowMode::Type NewWindowMode )
{
	if (NewWindowMode != WindowMode)
	{
		EWindowMode::Type PreviousWindowMode = WindowMode;
		WindowMode = NewWindowMode;

		const bool bTrueFullscreen = NewWindowMode == EWindowMode::Fullscreen;

		// Setup Win32 Flags to be used for Fullscreen mode
		LONG WindowStyle = GetWindowLong(HWnd, GWL_STYLE);
		const LONG FullscreenModeStyle = WS_POPUP;

		LONG WindowedModeStyle = WS_OVERLAPPED | WS_SYSMENU | WS_CAPTION;
		if (IsRegularWindow())
		{
			if (Definition->SupportsMaximize)
			{
				WindowedModeStyle |= WS_MAXIMIZEBOX;
			}

			if (Definition->SupportsMinimize)
			{
				WindowedModeStyle |= WS_MINIMIZEBOX;
			}

			if (Definition->HasSizingFrame)
			{
				WindowedModeStyle |= WS_THICKFRAME;
			}
			else
			{
				WindowedModeStyle |= WS_BORDER;
			}
		}
		else
		{
			WindowedModeStyle |= WS_POPUP | WS_BORDER;
		}

		// If we're not in fullscreen, make it so
		if (NewWindowMode == EWindowMode::WindowedFullscreen || NewWindowMode == EWindowMode::Fullscreen)
		{
			if (PreviousWindowMode == EWindowMode::Windowed)
			{
				PreFullscreenWindowPlacement.length = sizeof(WINDOWPLACEMENT);
				::GetWindowPlacement(HWnd, &PreFullscreenWindowPlacement);
			}

			// Setup Win32 flags for fullscreen window
			WindowStyle &= ~WindowedModeStyle;
			WindowStyle |= FullscreenModeStyle;

			SetWindowLong(HWnd, GWL_STYLE, WindowStyle);
			::SetWindowPos(HWnd, nullptr, 0, 0, 0, 0, SWP_NOMOVE | SWP_NOSIZE | SWP_NOZORDER | SWP_FRAMECHANGED);

			if (!bTrueFullscreen)
			{
				// Ensure the window is restored if we are going for WindowedFullscreen
				ShowWindow(HWnd, SW_RESTORE);
			}

			// Get the current window position.
			RECT ClientRect;
			GetClientRect(HWnd, &ClientRect);

			// Grab current monitor data for sizing
			HMONITOR Monitor = MonitorFromWindow( HWnd, MONITOR_DEFAULTTONEAREST );
			MONITORINFO MonitorInfo;
			MonitorInfo.cbSize = sizeof(MONITORINFO);
			GetMonitorInfo( Monitor, &MonitorInfo );

			// Get the target client width to send to ReshapeWindow.
			// Preserve the current res if going to true fullscreen and the monitor supports it and allow the calling code
			// to resize if required.
			// Else, use the monitor's res for windowed fullscreen.
			LONG MonitorWidth  = MonitorInfo.rcMonitor.right - MonitorInfo.rcMonitor.left;
			LONG TargetClientWidth = bTrueFullscreen ?
				FMath::Min(MonitorWidth, ClientRect.right - ClientRect.left) :
				MonitorWidth;

			LONG MonitorHeight = MonitorInfo.rcMonitor.bottom - MonitorInfo.rcMonitor.top;
			LONG TargetClientHeight = bTrueFullscreen ?
				FMath::Min(MonitorHeight, ClientRect.bottom - ClientRect.top) :
				MonitorHeight;


			// Resize and position fullscreen window
			ReshapeWindow(
				MonitorInfo.rcMonitor.left,
				MonitorInfo.rcMonitor.top,
				TargetClientWidth,
				TargetClientHeight);
		}
		else
		{
			// Windowed:

			// Setup Win32 flags for restored window
			WindowStyle &= ~FullscreenModeStyle;
			WindowStyle |= WindowedModeStyle;
			SetWindowLong(HWnd, GWL_STYLE, WindowStyle);
			::SetWindowPos(HWnd, nullptr, 0, 0, 0, 0, SWP_NOMOVE | SWP_NOSIZE | SWP_NOZORDER | SWP_FRAMECHANGED);

			if (PreFullscreenWindowPlacement.length) // Was PreFullscreenWindowPlacement initialized?
			{
				::SetWindowPlacement(HWnd, &PreFullscreenWindowPlacement);
			}

			// Set the icon back again as it seems to get ignored if the application has ever started in full screen mode
			HICON HIcon = (HICON)::GetClassLongPtr( HWnd, GCLP_HICON );
			if (HIcon != nullptr)
			{
				::SendMessageW( HWnd, WM_SETICON, ICON_SMALL, (LPARAM)HIcon );
			}
		}
	}
}

/** @return true if the native window is maximized, false otherwise */
bool FWindowsWindow::IsMaximized() const
{
	bool bIsMaximized = !!::IsZoomed(HWnd);
	return bIsMaximized;
}

bool FWindowsWindow::IsMinimized() const
{
	return !!::IsIconic(HWnd);
}

/** @return true if the native window is visible, false otherwise */
bool FWindowsWindow::IsVisible() const
{
	return bIsVisible;
}

/** Returns the size and location of the window when it is restored */
bool FWindowsWindow::GetRestoredDimensions(int32& X, int32& Y, int32& Width, int32& Height)
{
	WINDOWPLACEMENT WindowPlacement;
	WindowPlacement.length = sizeof( WINDOWPLACEMENT );

	if( ::GetWindowPlacement( HWnd, &WindowPlacement) != 0 )
	{
		// This window may be minimized.  Get the position when it is restored.
		const RECT Restored = WindowPlacement.rcNormalPosition;

		X = Restored.left;
		Y = Restored.top;
		Width = Restored.right - Restored.left;
		Height = Restored.bottom - Restored.top;

		const LONG WindowExStyle = ::GetWindowLong(HWnd, GWL_EXSTYLE);
		if ((WindowExStyle & WS_EX_TOOLWINDOW) == 0)
		{
			// For windows without WS_EX_TOOLWINDOW window style WindowPlacement.rcNormalPosition is in workspace coordinates, so we need to convert the position to screen coordinates
			HMONITOR Monitor = MonitorFromWindow(HWnd, MONITOR_DEFAULTTONEAREST);
			MONITORINFO MonitorInfo;
			MonitorInfo.cbSize = sizeof(MONITORINFO);
			GetMonitorInfo(Monitor, &MonitorInfo);

			X += MonitorInfo.rcWork.left - MonitorInfo.rcMonitor.left;
			Y += MonitorInfo.rcWork.top - MonitorInfo.rcMonitor.top;
		}

		return true;
	}
	else
	{
		return false;
	}
}

bool FWindowsWindow::IsManualManageDPIChanges() const
{
	return bHandleManualDPIChanges;
}

void FWindowsWindow::SetManualManageDPIChanges(const bool bManualDPIChanges)
{
	bHandleManualDPIChanges = bManualDPIChanges;
}

void FWindowsWindow::AdjustCachedSize( FVector2D& Size ) const
{
	if( Definition.IsValid() && Definition->SizeWillChangeOften )
	{
		Size = FVector2D( VirtualWidth, VirtualHeight );
	}
	else if(HWnd)
	{
		RECT ClientRect;
		::GetClientRect(HWnd,&ClientRect);
		Size.X = ClientRect.right - ClientRect.left;
		Size.Y = ClientRect.bottom - ClientRect.top;
	}
}

void FWindowsWindow::OnParentWindowMinimized()
{
	// This function is called from SW_PARENTCLOSING, because there's a bug in Win32 that causes the equivalent SW_PARENTOPENING
	// message to restore in an incorrect state (eg, it will lose the maximized status of the window)
	// To work around this, we cache our window placement here so that we can restore it later (see OnParentWindowRestored)
	::GetWindowPlacement(HWnd, &PreParentMinimizedWindowPlacement);
}

void FWindowsWindow::OnParentWindowRestored()
{
	// This function is called from SW_PARENTOPENING so that we can restore the window placement that was cached in OnParentWindowMinimized
	::SetWindowPlacement(HWnd, &PreParentMinimizedWindowPlacement);
}

/** Sets focus on the native window */
void FWindowsWindow::SetWindowFocus()
{
	if( GetFocus() != HWnd )
	{
		::SetFocus( HWnd );
	}
}

/**
 * Sets the opacity of this window
 *
 * @param	InOpacity	The new window opacity represented as a floating point scalar
 */
void FWindowsWindow::SetOpacity( const float InOpacity )
{
	SetLayeredWindowAttributes( HWnd, 0, (BYTE)FMath::TruncToInt( InOpacity * 255.0f ), LWA_ALPHA );
}

/**
 * Enables or disables this window.  When disabled, a window will receive no input.       
 *
 * @param bEnable	true to enable the window, false to disable it.
 */
void FWindowsWindow::Enable( bool bEnable )
{
	::EnableWindow( HWnd, bEnable );
}

/** @return True if the window is enabled */
bool FWindowsWindow::IsEnabled()
{
	return !!::IsWindowEnabled( HWnd );
}

TArray<FColor> FWindowsWindow::GetWindowPixels(uint32& OutWidth, uint32& OutHeight)
{
	TArray<FColor> Pixels;

	RECT Size;
	if (!GetClientRect(HWnd, &Size))
	{
		return Pixels;
	}

	LONG WindowWidth = Size.right - Size.left;
	LONG WindowHeight = Size.bottom - Size.top;

	HDC WindowDC = GetDC(HWnd);
	if (!WindowDC)
	{
		return Pixels;
	}

	HDC CaptureDC = CreateCompatibleDC(WindowDC);
	if (!CaptureDC)
	{
		ReleaseDC(HWnd, WindowDC);
		return Pixels;
	}

	HBITMAP CaptureBitmap = CreateCompatibleBitmap(WindowDC, WindowWidth, WindowHeight);
	if (!CaptureBitmap)
	{
		ReleaseDC(HWnd, WindowDC);
		DeleteDC(CaptureDC);
		return Pixels;
	}

	if (SelectObject(CaptureDC, CaptureBitmap))
	{
		// Use the undocumented flag PW_RENDERFULLCONTENT (2) which is available since Windows 8.1,
		// and that allows to capture applications that use hardware-accelerated graphics.
		UINT Flags = PW_CLIENTONLY | 2;
		if (PrintWindow(HWnd, CaptureDC, Flags))
		{
			BITMAPINFO BitmapInfo = {};
			BitmapInfo.bmiHeader.biSize = sizeof(BitmapInfo.bmiHeader);
			BitmapInfo.bmiHeader.biWidth = WindowWidth;
			BitmapInfo.bmiHeader.biHeight = -WindowHeight; // Negative height to indicate we want top-to-bottom scanlines.
			BitmapInfo.bmiHeader.biPlanes = 1;
			BitmapInfo.bmiHeader.biBitCount = 32;
			BitmapInfo.bmiHeader.biCompression = BI_RGB;

			// Get bitmap pixels and convert to RGB.
			TArray<RGBQUAD> BitmapPixels;
			BitmapPixels.SetNumUninitialized(WindowWidth * WindowHeight);
			uint32 ActualHeight = GetDIBits(CaptureDC, CaptureBitmap, 0, WindowHeight, BitmapPixels.GetData(), &BitmapInfo, DIB_RGB_COLORS);
			if (ActualHeight > 0)
			{
				Algo::Transform(BitmapPixels, Pixels, [](const RGBQUAD& Pixel) { return FColor(Pixel.rgbRed, Pixel.rgbGreen, Pixel.rgbBlue); });
				OutWidth = WindowWidth;
				OutHeight = ActualHeight;
			}
		}
	}

	ReleaseDC(HWnd, WindowDC);
	DeleteDC(CaptureDC);
	DeleteObject(CaptureBitmap);
	return Pixels;
}

/** @return true if native window exists underneath the coordinates */
bool FWindowsWindow::IsPointInWindow( int32 X, int32 Y ) const
{
	bool Result = false;

	const bool bForQuery = true;
	HRGN Region = MakeWindowRegionObject(bForQuery);
	if (!Region)
	{
		return false;
	}
	Result = !!PtInRegion( Region, X, Y );
	DeleteObject( Region );

	return Result;
}

int32 FWindowsWindow::GetWindowBorderSize() const
{
	if (!GetDefinition().HasOSWindowBorder)
	{
		// Our borderless windows technically have a thick border, but we draw over them to simulate a borderless window.
		return 0;
	}

	return GetWin32BorderSize();
}

int32 FWindowsWindow::GetWin32BorderSize() const
{
	WINDOWINFO WindowInfo;
	FMemory::Memzero( WindowInfo );
	WindowInfo.cbSize = sizeof( WindowInfo );
	::GetWindowInfo( HWnd, &WindowInfo );

	return WindowInfo.cxWindowBorders;
}

int32 FWindowsWindow::GetWindowTitleBarSize() const
{
	return GetSystemMetrics(SM_CYCAPTION);
}

bool FWindowsWindow::IsForegroundWindow() const
{
	return ::GetForegroundWindow() == HWnd;
}

bool FWindowsWindow::SupportsDynamicCursorLock() const
{
	static const TConsoleVariableData<bool>* const CVarDynamicLockToggleData = IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("WindowsCursor.DynamicLockToggle"));
	return CVarDynamicLockToggleData->GetValueOnGameThread();
}

bool FWindowsWindow::IsFullscreenSupported() const
{
#if defined(UE_WINDOWS_FORCE_NONEXCLUSIVE_FULLSCREEN) && UE_WINDOWS_FORCE_NONEXCLUSIVE_FULLSCREEN
	return false;
#else
	// fullscreen not supported when using remote desktop
	return !::GetSystemMetrics(SM_REMOTESESSION);
#endif
}

void FWindowsWindow::SetText( const TCHAR* const Text )
{
	SetWindowText(HWnd, Text);
}

void FWindowsWindow::DrawAttention(const FWindowDrawAttentionParameters& Parameters)
{
	FLASHWINFO FlashInfo;
	FMemory::Memzero(FlashInfo);
	FlashInfo.cbSize = sizeof(FlashInfo);
	FlashInfo.hwnd = HWnd;

	switch (Parameters.RequestType)
	{
	case EWindowDrawAttentionRequestType::UntilActivated:
		FlashInfo.dwFlags = FLASHW_TRAY | FLASHW_TIMERNOFG;
		break;

	case EWindowDrawAttentionRequestType::Stop:
		FlashInfo.dwFlags = FLASHW_STOP;
		break;

	default:
		ensureMsgf(false, TEXT("FWindowsWindow::DrawAttention(): Invalid request type %d"), static_cast<int32>(Parameters.RequestType));
		break;
	}

	FlashWindowEx(&FlashInfo);
}

bool FWindowsWindow::IsRegularWindow() const
{
	return Definition->IsRegularWindow;
}

HRESULT STDCALL FWindowsWindow::QueryInterface( REFIID iid, void ** ppvObject )
{
	if ( IID_IDropTarget == iid || IID_IUnknown == iid )
	{
		AddRef();
		*ppvObject = (IDropTarget*)(this);
		return S_OK;
	}
	else
	{
		*ppvObject = NULL;
		return E_NOINTERFACE;
	}
}

ULONG STDCALL FWindowsWindow::AddRef( void )
{
	// We only do this for correctness checking
	FPlatformAtomics::InterlockedIncrement( &OLEReferenceCount );
	return OLEReferenceCount;
}

ULONG STDCALL FWindowsWindow::Release( void )
{
	// We only do this for correctness checking
	FPlatformAtomics::InterlockedDecrement( &OLEReferenceCount );
	return OLEReferenceCount;
}

FDragDropOLEData DecipherOLEData(IDataObject* DataObjectPointer)
{
	FDragDropOLEData OLEData;

	// Utility to ensure resource release
	struct FOLEResourceGuard
	{
		STGMEDIUM& StorageMedium;
		LPVOID DataPointer;

		FOLEResourceGuard(STGMEDIUM& InStorage)
			: StorageMedium(InStorage)
			, DataPointer(GlobalLock(InStorage.hGlobal))
		{
		}

		~FOLEResourceGuard()
		{
			GlobalUnlock(StorageMedium.hGlobal);
			ReleaseStgMedium(&StorageMedium);
		}
	};

	// Attempt to get plain text or unicode text from the data being dragged in

	FORMATETC FormatEtc_Ansii = { CF_TEXT, 0, DVASPECT_CONTENT, -1, TYMED_HGLOBAL };
	const bool bHaveAnsiText = (DataObjectPointer->QueryGetData(&FormatEtc_Ansii) == S_OK)
		? true
		: false;

	FORMATETC FormatEtc_UNICODE = { CF_UNICODETEXT, 0, DVASPECT_CONTENT, -1, TYMED_HGLOBAL };
	const bool bHaveUnicodeText = (DataObjectPointer->QueryGetData(&FormatEtc_UNICODE) == S_OK)
		? true
		: false;

	FORMATETC FormatEtc_File = { CF_HDROP, 0, DVASPECT_CONTENT, -1, TYMED_HGLOBAL };
	const bool bHaveFiles = (DataObjectPointer->QueryGetData(&FormatEtc_File) == S_OK)
		? true
		: false;

	STGMEDIUM StorageMedium;

	if (bHaveUnicodeText && S_OK == DataObjectPointer->GetData(&FormatEtc_UNICODE, &StorageMedium))
	{
		FOLEResourceGuard ResourceGuard(StorageMedium);
		OLEData.Type |= FDragDropOLEData::Text;
		OLEData.OperationText = static_cast<TCHAR*>(ResourceGuard.DataPointer);
	}
	if (bHaveAnsiText && S_OK == DataObjectPointer->GetData(&FormatEtc_Ansii, &StorageMedium))
	{
		FOLEResourceGuard ResourceGuard(StorageMedium);
		OLEData.Type |= FDragDropOLEData::Text;
		OLEData.OperationText = static_cast<ANSICHAR*>(ResourceGuard.DataPointer);
	}
	if (bHaveFiles && S_OK == DataObjectPointer->GetData(&FormatEtc_File, &StorageMedium))
	{
		OLEData.Type |= FDragDropOLEData::Files;

		FOLEResourceGuard ResourceGuard(StorageMedium);
		const DROPFILES* DropFiles = static_cast<DROPFILES*>(ResourceGuard.DataPointer);

		// pFiles is the offset to the beginning of the file list, in bytes
		LPVOID FileListStart = (BYTE*)ResourceGuard.DataPointer + DropFiles->pFiles;

		if (DropFiles->fWide)
		{
			// Unicode filenames
			// The file list is NULL delimited with an extra NULL character at the end.
			TCHAR* Pos = static_cast<TCHAR*>(FileListStart);
			while (Pos[0] != 0)
			{
				const FString ListElement = FString(Pos);
				OLEData.OperationFilenames.Add(ListElement);
				Pos += ListElement.Len() + 1;
			}
		}
		else
		{
			// Ansi filenames
			// The file list is NULL delimited with an extra NULL character at the end.
			ANSICHAR* Pos = static_cast<ANSICHAR*>(FileListStart);
			while (Pos[0] != 0)
			{
				const FString ListElement = FString(Pos);
				OLEData.OperationFilenames.Add(ListElement);
				Pos += ListElement.Len() + 1;
			}
		}
	}

	return OLEData;
}

HRESULT STDCALL FWindowsWindow::DragEnter( __RPC__in_opt IDataObject *DataObjectPointer, ::DWORD KeyState, POINTL CursorPosition, __RPC__inout ::DWORD *CursorEffect )
{
	// Decipher the OLE data
	const FDragDropOLEData& OLEData = DecipherOLEData(DataObjectPointer);

	if (IsInGameThread())
	{
		return OwningApplication->OnOLEDragEnter(HWnd, OLEData, KeyState, CursorPosition, CursorEffect);
	}
	else
	{
		// The occurred in a non-game thread. We must synchronize to the main thread as Slate is not designed to be multi-threaded.
		// Note that doing this means that we cannot determine the CursorEffect, but this is unfixable at the moment so we will just leave it alone.
		OwningApplication->DeferDragDropOperation( FDeferredWindowsDragDropOperation::MakeDragEnter(HWnd, OLEData, KeyState, CursorPosition) );
		return 0;
	}
}

HRESULT STDCALL FWindowsWindow::DragOver( ::DWORD KeyState, POINTL CursorPosition, __RPC__inout ::DWORD *CursorEffect )
{
	if (IsInGameThread())
	{
		return OwningApplication->OnOLEDragOver(HWnd, KeyState, CursorPosition, CursorEffect);
	}
	else
	{
		// The occurred in a non-game thread. We must synchronize to the main thread as Slate is not designed to be multi-threaded.
		// Note that doing this means that we cannot determine the CursorEffect, but this is unfixable at the moment so we will just leave it alone.
		OwningApplication->DeferDragDropOperation(FDeferredWindowsDragDropOperation::MakeDragOver(HWnd, KeyState, CursorPosition));
		return 0;
	}
}

HRESULT STDCALL FWindowsWindow::DragLeave( void )
{
	if (IsInGameThread())
	{
		return OwningApplication->OnOLEDragOut(HWnd);
	}
	else
	{
		// The occurred in a non-game thread. We must synchronize to the main thread as Slate is not designed to be multi-threaded.
		OwningApplication->DeferDragDropOperation(FDeferredWindowsDragDropOperation::MakeDragLeave(HWnd));
		return 0;
	}
}

HRESULT STDCALL FWindowsWindow::Drop( __RPC__in_opt IDataObject *DataObjectPointer, ::DWORD KeyState, POINTL CursorPosition, __RPC__inout ::DWORD *CursorEffect )
{
	// Decipher the OLE data
	const FDragDropOLEData& OLEData = DecipherOLEData(DataObjectPointer);

	if (IsInGameThread())
	{
		return OwningApplication->OnOLEDrop(HWnd, OLEData, KeyState, CursorPosition, CursorEffect);
	}
	else
	{
		// The occurred in a non-game thread. We must synchronize to the main thread as Slate is not designed to be multi-threaded.
		// Note that doing this means that we cannot determine the CursorEffect, but this is unfixable at the moment so we will just leave it alone.
		OwningApplication->DeferDragDropOperation(FDeferredWindowsDragDropOperation::MakeDrop(HWnd, OLEData, KeyState, CursorPosition));
		return 0;
	}
}
