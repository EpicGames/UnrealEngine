// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericApplication.h"
#include "HAL/IConsoleManager.h"
#include "AndroidWindow.h"
#if USE_ANDROID_JNI
#include "Android/AndroidJavaEnv.h"
#endif

class IModularFeature;

namespace FAndroidAppEntry
{
	void PlatformInit();

	// if the native window handle has changed then the new handle is required.
	void ReInitWindow(const TOptional<FAndroidWindow::FNativeAccessor>& WindowContainer);

	void ReleaseEGL();
	void OnPauseEvent();
}

//disable warnings from overriding the deprecated forcefeedback.  
//calls to the deprecated function will still generate warnings.
PRAGMA_DISABLE_DEPRECATION_WARNINGS

class FAndroidApplication : public GenericApplication
{
public:

	static FAndroidApplication* CreateAndroidApplication();

#if USE_ANDROID_JNI
	UE_DEPRECATED(5.7, "Use InitializeJavaEnv() instead.")
	static inline void InitializeJavaEnv(JavaVM* VM, jint Version, jobject GlobalThis)
	{
		AndroidJavaEnv::InitializeJavaEnv();
	}
	static FORCEINLINE void InitializeJavaEnv()
	{
		AndroidJavaEnv::InitializeJavaEnv();
    }
	static inline jobject GetGameActivityThis()
	{
		return AndroidJavaEnv::GetGameActivityThis();
	} 
	static inline jobject GetClassLoader()
	{
		return AndroidJavaEnv::GetClassLoader();
	} 
	static inline JNIEnv* GetJavaEnv(bool bRequireGlobalThis = true)
	{
		return AndroidJavaEnv::GetJavaEnv(bRequireGlobalThis);
	}
	static inline jclass FindJavaClass(const char* name)
	{
		return AndroidJavaEnv::FindJavaClass(name);
	}
	static inline jclass FindJavaClassGlobalRef(const char* name)
	{
		return AndroidJavaEnv::FindJavaClassGlobalRef(name);
	}
	UE_DEPRECATED(5.7, "Don't call this.")
	static inline void DetachJavaEnv()
	{
		AndroidJavaEnv::DetachJavaEnv();
	}
	static inline bool CheckJavaException()
	{
		return AndroidJavaEnv::CheckJavaException();
	}
#endif

	static FAndroidApplication* Get() { return _application; }

public:	
	
	virtual ~FAndroidApplication() override;

	void SetMessageHandler( const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler );

	virtual void PollGameDeviceState( const float TimeDelta ) override;

	virtual FPlatformRect GetWorkArea( const FPlatformRect& CurrentWindow ) const override;

	virtual IInputInterface* GetInputInterface() override;

	virtual TSharedRef< FGenericWindow > MakeWindow() override;

	virtual void AddExternalInputDevice(TSharedPtr<class IInputDevice> InputDevice);

	void InitializeWindow( const TSharedRef< FGenericWindow >& InWindow, const TSharedRef< FGenericWindowDefinition >& InDefinition, const TSharedPtr< FGenericWindow >& InParent, const bool bShowImmediately );

	static void OnWindowSizeChanged();

	virtual void Tick(const float TimeDelta) override;

	virtual bool IsGamepadAttached() const override;

	bool GetNativeWindowResolution(int32_t& OutWidth, int32_t& OutHeight) const;

	virtual TSharedPtr<FGenericWindow> GetWindowUnderCursor() override;

	/** Game thread only; the touch loop brackets each OnTouch* call with set / clear. */
	void SetCurrentlyTouchedWindow(TSharedPtr<FGenericWindow> InWindow);

	/** Returns the primary (first-created) Android window, or nullptr if none has been added yet. */
	TSharedPtr<FAndroidWindow> GetPrimaryWindow() const;

	static TAutoConsoleVariable<bool> CVarAndroidSupportsTimestampQueries;
	static TAutoConsoleVariable<bool> CVarAndroidSupportsDynamicResolution;

protected:

	FAndroidApplication();
	FAndroidApplication(TSharedPtr<class FAndroidInputInterface> InInputInterface);


private:

	void OnInputDeviceModuleRegistered(const FName& Type, IModularFeature* ModularFeature);

	TSharedPtr< class FAndroidInputInterface > InputInterface;
	bool bHasLoadedInputPlugins;

	/** See GetWindowUnderCursor. Game thread only. */
	TSharedPtr<FGenericWindow> CurrentlyTouchedWindow;

	TArray< TSharedRef< FAndroidWindow > > Windows;

	static bool bWindowSizeChanged;

	static FAndroidApplication* _application;

    EDeviceScreenOrientation DeviceOrientation;
    void HandleDeviceOrientation();
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS
