// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/App.h"
#include "Misc/FeedbackContext.h"
#include "Misc/OutputDeviceRedirector.h"

class FAndroidFeedbackContext : public FFeedbackContext
{
};
