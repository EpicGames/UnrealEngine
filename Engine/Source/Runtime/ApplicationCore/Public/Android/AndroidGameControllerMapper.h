// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"
#include "Logging/LogMacros.h"
#include <Templates/SharedPointer.h>
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "Android/AndroidPlatform.h"

namespace GameControllerMapping
{
	enum ControllerClassType : int
	{
		Generic,
		XBoxWired,
		XBoxWireless,
		PlaystationWireless,
		
		XBoxWired_AndroidAPIBefore31,
		XBoxWireless_OlderFirmware,
		PS4Wireless_AndroidAPIBefore10_CPUNotSony,
		PS5Wireless_AndroidAPIBefore31,
	};

	enum ButtonRemapType : int
	{
		Normal,
		XBox,
		PS4,
		PS5,
		PS5New
	};

	typedef TMap<int32/*KeyCode*/, TArray<int32/*KeyNameIndex*/>> KeyCodeToKeyNameMap; // ButtonStates[] and ButtonMapping[] use same indices
	typedef TWeakPtr<KeyCodeToKeyNameMap> KeyCodeToKeyNameMap_WPtr;
	typedef TSharedPtr<KeyCodeToKeyNameMap> KeyCodeToKeyNameMap_SPtr;
	typedef TSharedRef<KeyCodeToKeyNameMap> KeyCodeToKeyNameMap_SRef;

	struct DeviceConfig
	{
		// Type of controller
		ControllerClassType ControllerClass = ControllerClassType::Generic;

		// Sets the analog range of the trigger minimum (normally 0).  Final value is mapped as (input - Minimum) / (1 - Minimum) to [0,1] output.
		float LTAnalogRangeMinimum = .0f;
		float RTAnalogRangeMinimum = .0f;

		// Device supports hat as dpad
		bool SupportsHat = true;

		// Device uses threshold to send button pressed events.
		bool TriggersUseThresholdForClick = false;

		// Map L1 and R1 to LTRIGGER and RTRIGGER
		bool MapL1R1ToTriggers = false;

		// Map Z and RZ to LTAnalog and RTAnalog
		bool MapZRZToTriggers = false;

		// Right stick on Z/RZ
		bool RightStickZRZ = true;

		// Right stick on RX/RY
		bool RightStickRXRY = false;

		// Map RX and RY to LTAnalog and RTAnalog
		bool MapRXRYToTriggers = false;

		int DeviceNameIndex = -1;

		TWeakPtr<KeyCodeToKeyNameMap> ButtonMapping;
	};

	typedef TWeakPtr<DeviceConfig> DeviceConfig_WPtr;
	typedef TSharedPtr<DeviceConfig> DeviceConfig_SPtr;
	typedef TSharedRef<DeviceConfig> DeviceConfig_SRef;

	class FAndroidGameControllerMapper
	{
	public:
		~FAndroidGameControllerMapper() {}

		static TWeakPtr<FAndroidGameControllerMapper> GetInstance();

		const DeviceConfig_WPtr GetDeviceConfig(int32 VendorId, int32 ProductId, const FName DeviceName);

		void RegisterDeviceConfig(int32 VendorId, int32 ProductId, DeviceConfig_SRef& DeviceCfg);

	private:
		FAndroidGameControllerMapper() {}

		bool Init();

		bool InitByDataFile();

		const DeviceConfig_SPtr GetDeviceConfig(const FName DeviceName);

		void AppendKeyCodeKeyNameMapping(KeyCodeToKeyNameMap_SRef& TargetContainer, int32 KeyCode, int32 KeyNameIndex);

		uint64 PackDeviceKey(int32 VendorId, int32 ProductId) const
		{
			return ((uint64)VendorId) << 32 | (uint32)ProductId;
		}

		TArray<KeyCodeToKeyNameMap_SRef> ButtonMappings;
		
		TMap<uint64, DeviceConfig_SRef> DeviceConfigurationMap;

		TMap<uint64, DeviceConfig_SRef> DeviceConfigurationMap_Special;

		TArray<const FName> DeviceNames;

		static TSharedPtr<FAndroidGameControllerMapper> Instance;
	};
}
