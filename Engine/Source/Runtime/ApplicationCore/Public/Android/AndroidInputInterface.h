// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "GenericPlatform/GenericApplicationMessageHandler.h"

#include <android/input.h>
#include <android/keycodes.h>
#include <android/api-level.h>
#include "GenericPlatform/ICursor.h"
#include "GenericPlatform/IInputInterface.h"
#include "GenericPlatform/GenericInputDeviceMap.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Math/Color.h"

#include "Android/AndroidGameControllerMapper.h"

class FAndroidWindow;

#if __ANDROID_API__ < 13

// Joystick functions and constants only available API level 13 and above
// Definitions are provided to allow compiling against lower API levels, but
// still using the features when available.

enum
{
	AMOTION_EVENT_AXIS_X = 0,
	AMOTION_EVENT_AXIS_Y = 1,
	AMOTION_EVENT_AXIS_PRESSURE = 2,
	AMOTION_EVENT_AXIS_SIZE = 3,
	AMOTION_EVENT_AXIS_TOUCH_MAJOR = 4,
	AMOTION_EVENT_AXIS_TOUCH_MINOR = 5,
	AMOTION_EVENT_AXIS_TOOL_MAJOR = 6,
	AMOTION_EVENT_AXIS_TOOL_MINOR = 7,
	AMOTION_EVENT_AXIS_ORIENTATION = 8,
	AMOTION_EVENT_AXIS_VSCROLL = 9,
	AMOTION_EVENT_AXIS_HSCROLL = 10,
	AMOTION_EVENT_AXIS_Z = 11,
	AMOTION_EVENT_AXIS_RX = 12,
	AMOTION_EVENT_AXIS_RY = 13,
	AMOTION_EVENT_AXIS_RZ = 14,
	AMOTION_EVENT_AXIS_HAT_X = 15,
	AMOTION_EVENT_AXIS_HAT_Y = 16,
	AMOTION_EVENT_AXIS_LTRIGGER = 17,
	AMOTION_EVENT_AXIS_RTRIGGER = 18,
	AMOTION_EVENT_AXIS_THROTTLE = 19,
	AMOTION_EVENT_AXIS_RUDDER = 20,
	AMOTION_EVENT_AXIS_WHEEL = 21,
	AMOTION_EVENT_AXIS_GAS = 22,
	AMOTION_EVENT_AXIS_BRAKE = 23,
	AMOTION_EVENT_AXIS_DISTANCE = 24,
	AMOTION_EVENT_AXIS_TILT = 25,
	AMOTION_EVENT_AXIS_GENERIC_1 = 32,
	AMOTION_EVENT_AXIS_GENERIC_2 = 33,
	AMOTION_EVENT_AXIS_GENERIC_3 = 34,
	AMOTION_EVENT_AXIS_GENERIC_4 = 35,
	AMOTION_EVENT_AXIS_GENERIC_5 = 36,
	AMOTION_EVENT_AXIS_GENERIC_6 = 37,
	AMOTION_EVENT_AXIS_GENERIC_7 = 38,
	AMOTION_EVENT_AXIS_GENERIC_8 = 39,
	AMOTION_EVENT_AXIS_GENERIC_9 = 40,
	AMOTION_EVENT_AXIS_GENERIC_10 = 41,
	AMOTION_EVENT_AXIS_GENERIC_11 = 42,
	AMOTION_EVENT_AXIS_GENERIC_12 = 43, 
	AMOTION_EVENT_AXIS_GENERIC_13 = 44,
	AMOTION_EVENT_AXIS_GENERIC_14 = 45,
	AMOTION_EVENT_AXIS_GENERIC_15 = 46,
	AMOTION_EVENT_AXIS_GENERIC_16 = 47,
};
enum
{
	AINPUT_SOURCE_CLASS_JOYSTICK = 0x00000010,
};
enum
{
	AINPUT_SOURCE_GAMEPAD = 0x00000400 | AINPUT_SOURCE_CLASS_BUTTON,
	AINPUT_SOURCE_JOYSTICK = 0x01000000 | AINPUT_SOURCE_CLASS_JOYSTICK,
};
#endif // __ANDROID_API__ < 13

enum InputDeviceType
{
	UnknownInputDeviceType,
	TouchScreen,
	GameController,
};

enum TouchType
{
	TouchBegan,
	TouchMoved,
	TouchEnded,
};

enum class InputDeviceStateEvent
{
	Added = 0,
	Removed,
	Changed
};

enum MappingState
{
	Unassigned = 0,
	ToActivate,
	ToValidate,
	Valid
};

struct FAndroidInputDeviceInfo 
{
	FAndroidInputDeviceInfo()
	{
		Descriptor.Empty();
	}

	int32 DeviceId = 0;
	int32 VendorId = 0;
	int32 ProductId = 0;
	int32 ControllerId = 0;
	FName Name;
	FString Descriptor;
	int32 FeedbackMotorCount = 0;
	bool IsExternal = false;

	InputDeviceType DeviceType = InputDeviceType::UnknownInputDeviceType;
	MappingState DeviceState = MappingState::Unassigned;
	int32 DataIndex = -1; // -1 = invalid
};

struct TouchInput
{
	/** This is the Android kernel-provided "Device ID", NOT the Unreal "Device ID". */
	int32 DeviceId;
	int32 Handle;
	FVector2D LastPosition;
	FVector2D Position;
	/**
	 * When valid, the touch loop uses this directly instead of resolving DeviceId through InputDeviceInfoMap.
	 * See FAndroidInputInterface::RegisterSurfaceForTouch.
	 */
	FInputDeviceId InputDeviceId;
	TouchType Type;
};

#define MAX_NUM_CONTROLLERS					8  // reasonable limit for now
#define MAX_NUM_PHYSICAL_CONTROLLER_BUTTONS	18
#define MAX_NUM_VIRTUAL_CONTROLLER_BUTTONS	8
#define MAX_NUM_CONTROLLER_BUTTONS			MAX_NUM_PHYSICAL_CONTROLLER_BUTTONS + MAX_NUM_VIRTUAL_CONTROLLER_BUTTONS
#define MAX_DEFERRED_MESSAGE_QUEUE_SIZE		128

struct FAndroidControllerData
{
	// ID of the controller
	int32 DeviceId;

	// Current button states and the next time a repeat event should be generated for each button
	bool ButtonStates[MAX_NUM_CONTROLLER_BUTTONS];
	double NextRepeatTime[MAX_NUM_CONTROLLER_BUTTONS];

	// Raw analog values for various axes (sticks and triggers)
	float LXAnalog;
	float LYAnalog;
	float RXAnalog;
	float RYAnalog;
	float LTAnalog;
	float RTAnalog;
};

struct FAndroidControllerVibeState
{
	FForceFeedbackValues VibeValues;
	int32 LeftIntensity;
	int32 RightIntensity;
	double LastVibeUpdateTime;
};

struct FAndroidGamepadDeviceMapping
{
	FAndroidGamepadDeviceMapping(const int32 InVendorId, const int32 InProductId, const FName InDeviceName)
	{
		Init(InVendorId, InProductId, InDeviceName);
	}

	void Init(const int32 InVendorId, const int32 InProductId, const FName InDeviceName);

	void ResetRuntimeData()
	{
		FMemory::Memset(OldControllerData, 0);
		FMemory::Memset(NewControllerData, 0);
		FMemory::Memset(ControllerVibeState, 0);
	}

	GameControllerMapping::DeviceConfig_WPtr DeviceConfig;

	FAndroidControllerData OldControllerData;
	FAndroidControllerData NewControllerData;
	FAndroidControllerVibeState ControllerVibeState;
};

enum FAndroidMessageType
{
	MessageType_KeyDown,
	MessageType_KeyUp,
};

struct FDeferredAndroidMessage
{
	FDeferredAndroidMessage() {}

	FAndroidMessageType messageType;
	union
	{
		struct
		{
			int32 keyId{};
			int32 unichar{};
			uint32 modifier{};
			bool isRepeat{};
		}
		KeyEventData{};

	};
};

typedef GameControllerMapping::ControllerClassType ControllerClassType;
/**
 * Interface class for Android input devices                 
 */
class FAndroidInputInterface : public IInputInterface
{
public:

	static TSharedRef< FAndroidInputInterface > Create(  const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler, const TSharedPtr< ICursor >& InCursor);


public:

	~FAndroidInputInterface();

	void SetMessageHandler( const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler );

	/** Tick the interface (i.e check for new controllers) */
	void Tick( float DeltaTime );

	/**
	 * Poll for controller state and send events if needed
	 */
	void SendControllerEvents();

	static void QueueTouchInput(const TArray<TouchInput>& InTouchEvents);

	/**
	 * Register an Android Surface so that touches associated with it can be processed properly by QueueTouchInput.
	 * This is necessary when allocating any Surfaces other than the primary activity one that the engine owns, because Android reports touch device
	 * ID based on the physical hardware, but tracks finger indices uniquely per Surface.  Unreal expects finger indices to be unique per device.
	 *
	 * Caller is responsible for mapping the returned device to an FPlatformUserId, and for forwarding touch events to QueueTouchInput.
	 */
	static APPLICATIONCORE_API FInputDeviceId RegisterSurfaceForTouch(const TSharedRef<FAndroidWindow>& Window);

	/** Release a virtual touchscreen previously returned by RegisterSurfaceForTouch. */
	static APPLICATIONCORE_API void UnregisterSurface(FInputDeviceId DeviceId);

	static void ResetGamepadAssignments();
	static void ResetGamepadAssignmentToController(int32 ControllerId);
	static bool IsControllerAssignedToGamepad(int32 ControllerId);
	static const FInputDeviceId GetMappedInputDeviceId(int32 ControllerId);
	static const FName GetGamepadControllerName(int32 ControllerId);

	static void HandleInputDeviceStateEvent(int32 DeviceId, InputDeviceStateEvent StateEvent, InputDeviceType DeviceType);

	static void JoystickAxisEvent(int32 deviceId, int32 axisId, float axisValue);
	static void JoystickButtonEvent(int32 deviceId, int32 buttonId, bool buttonDown);

	static int32 GetAlternateKeyEventForMouse(int32 deviceId, int32 buttonId);
	static void MouseMoveEvent(int32 deviceId, float absoluteX, float absoluteY, float deltaX, float deltaY);
	static void MouseWheelEvent(int32 deviceId, float wheelDelta);
	static void MouseButtonEvent(int32 deviceId, int32 buttonId, bool buttonDown);

	static void DeferMessage(const FDeferredAndroidMessage& DeferredMessage);

	static void QueueMotionData(const FVector& Tilt, const FVector& RotationRate, const FVector& Gravity, const FVector& Acceleration);

	/**
	* Force Feedback implementation
	*/
	virtual void SetForceFeedbackChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) override;
	virtual void SetForceFeedbackChannelValues(int32 ControllerId, const FForceFeedbackValues &values) override;
	virtual void SetHapticFeedbackValues(int32 ControllerId, int32 Hand, const FHapticFeedbackValues& Values) override;
	virtual void SetLightColor(int32 ControllerId, FColor Color) override;
	virtual void ResetLightColor(int32 ControllerId) override;

	void SetGamepadsAllowed(bool bAllowed) { bAllowControllers = bAllowed; }
	void SetGamepadsBlockDeviceFeedback(bool bBlock) { bControllersBlockDeviceFeedback = bBlock; }

	virtual bool IsGamepadAttached() const;


	virtual void AddExternalInputDevice(TSharedPtr<class IInputDevice> InputDevice);

	const TSharedPtr< ICursor > GetCursor() const { return Cursor; }
	
private:

	FAndroidInputInterface( const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler, const TSharedPtr< ICursor >& InCursor);

	static bool GetInputDeviceByDeviceId(int32 DeviceId, FAndroidInputDeviceInfo** OutDeviceInfo, FAndroidGamepadDeviceMapping** OutDeviceData = nullptr);
	static bool GetInputDeviceByControllerId(int32 ControllerId, FAndroidInputDeviceInfo** OutDeviceInfo, FAndroidGamepadDeviceMapping** OutDeviceData = nullptr);
	
	static void MapControllerToPlayer(const FString& ControllerDesciptor, EInputDeviceConnectionState State);
	static void AddPendingInputDevice(int32 DeviceId, InputDeviceType DeviceType);
	static void RemoveInputDevice(int32 DeviceId);
	static void DumpInputDevices();

public:

	ControllerClassType GetControllerClass(int32 ControllerId) const;

private:

	/** Find controller ID(index) corresponding to validated deviceId (returns -1 if not found) */
	static int32 FindControllerId(int32 DeviceId);

	/** Push Vibration changes to the main device */
	void UpdateVibeMotors();

	/** Push Vibration changes to the controller */
	void UpdateControllerVibeMotors(int32 DeviceId, ControllerClassType ControllerClass, FAndroidControllerVibeState& State);

	struct MotionData
	{
		FVector Tilt;
		FVector RotationRate;
		FVector Gravity;
		FVector Acceleration;
	};

	enum MouseEventType
	{
		MouseMove,
		MouseWheel,
		MouseButtonDown,
		MouseButtonUp
	};

	struct MouseData
	{
		MouseEventType EventType;
		EMouseButtons::Type Button;
		int32 AbsoluteX;
		int32 AbsoluteY;
		int32 DeltaX;
		int32 DeltaY;
		float WheelDelta;
	};

	// protects the input stack
	static FCriticalSection TouchInputCriticalSection;

	static TArray<TouchInput> TouchInputStack;

	/** Vibration settings */
	static int32 CurrentVibeIntensity;
	// Maximum time vibration will be triggered without an update
	static int32 MaxVibeTime;
	static double LastVibeUpdateTime;
	static FForceFeedbackValues VibeValues;

	// should we allow controllers to send input
	static bool bAllowControllers;

	// bluetooth connected controllers will block force feedback.
	static bool bControllersBlockDeviceFeedback;

	// should we allow controllers to send Android_Back and Android_Menu events
	static bool bBlockAndroidKeysOnControllers;

	static TInputDeviceMap<FString> InternalDeviceIdMappings;
	static TArray<int32/*Android DeviceId*/> GameControllerIdMapping;
	static TMap<int32/*Android deviceId*/, FAndroidInputDeviceInfo> InputDeviceInfoMap;
	static TMap<FInputDeviceId, FAndroidGamepadDeviceMapping> GameControllerDataMap;
	/** Windows registered via RegisterSurfaceForTouch, looked up during touch dispatch. Guarded by TouchInputCriticalSection. */
	static TMap<FInputDeviceId, TWeakPtr<FGenericWindow>> InputMappedSurfaces;
	/**
	 * FInputDeviceIds previously returned by RegisterSurfaceForTouch and freed via UnregisterSurface, available for reuse.
	 * Bounds the FInputDeviceId allocation counter across repeated attach/detach cycles.  Guarded by TouchInputCriticalSection.
	 */
	static TArray<FInputDeviceId> FreeSurfaceInputDeviceIds;
	
	static FName InputClassName_DefaultMobileTouch;
	static FName InputClassName_DefaultGamepad;
	// The following two vars should be FName. Waiting for a refactoring of the FInputDeviceScope's ctor
	static FString HardwareDeviceIdentifier_DefaultMobileTouch;
	static FString HardwareDeviceIdentifier_DefaultGamepad;

	static FGamepadKeyNames::Type ButtonMapping[MAX_NUM_CONTROLLER_BUTTONS];

	static float InitialButtonRepeatDelay;
	static float ButtonRepeatDelay;

	static FDeferredAndroidMessage DeferredMessages[MAX_DEFERRED_MESSAGE_QUEUE_SIZE];
	static int32 DeferredMessageQueueLastEntryIndex;
	static int32 DeferredMessageQueueDroppedCount;

	static TArray<MotionData> MotionDataStack;
	static TArray<MouseData> MouseDataStack;

	TSharedRef< FGenericApplicationMessageHandler > MessageHandler;
	const TSharedPtr< ICursor > Cursor;

	/** List of input devices implemented in external modules. */
	TArray<TSharedPtr<class IInputDevice>> ExternalInputDevices;
};
