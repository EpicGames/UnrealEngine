// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#if USE_ANDROID_JNI
#include "Android/AndroidJava.h"
#include "IMediaTextureSample.h"
#include "RHI.h"
#include "RHIUtilities.h"

class FAndroidJavaMediaFrameData
{
public:
	FAndroidJavaMediaFrameData();
	virtual ~FAndroidJavaMediaFrameData();

	FAndroidJavaMediaFrameData(const FAndroidJavaMediaFrameData& Other) = delete;
	FAndroidJavaMediaFrameData& operator=(const FAndroidJavaMediaFrameData& Other) = delete;

	FAndroidJavaMediaFrameData& operator=(FAndroidJavaMediaFrameData& Other);

	operator bool() const
	{
		return (FrameDataGlobalRef != nullptr);
	}

	bool IsReadyToClean()
	{
		if (Fence.IsValid())
		{
			return (Fence->Poll());
		}
		return true;
	}

	bool Set(jobject InFrameData);
	void CleanUp();

	jobject Extract();
	bool ExtractToTextureVulkan(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, IMediaTextureSample* TextureSample);
	bool ExtractToTextureOES(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, IMediaTextureSample* TextureSample);

private:
	TRefCountPtr<FRHITexture> Texture;
	TRefCountPtr<FRHIGPUFence> Fence;

	/** The java frame data from the surface provider*/
	jobject FrameDataGlobalRef;

	/** FrameData members / methods */
	static jfieldID FrameData_HardwareBufferHandle;
	static jfieldID FrameData_UScale;
	static jfieldID FrameData_UOffset;
	static jfieldID FrameData_VScale;
	static jfieldID FrameData_VOffset;
	static jmethodID FrameData_ReleaseFN;
};
#endif //USE_ANDROID_JNI