// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#if USE_ANDROID_JNI
#include "Android/AndroidJava.h"
#include "RHI.h"
#include "RHIResources.h"

class FJavaAndroidMediaDataSource
{
public:
	FJavaAndroidMediaDataSource(const TSharedRef<FArchive, ESPMode::ThreadSafe>& InArchive);

	/** Virtual destructor. */
	virtual ~FJavaAndroidMediaDataSource();

public:
	int64 GetSize();
	int64 GetCurrentPosition();
	int32 ReadAt(int64 Position, uint8* Buffer, int32 Count);

private:
	/** Holds the archive to stream from. */
	TSharedRef<FArchive, ESPMode::ThreadSafe> Archive;

	/** Critical section for locking access to this class. */
	mutable FCriticalSection CriticalSection;
};


namespace UE::Jni::Nio
{
	struct FByteBuffer: Java::Lang::FObject
	{
		static constexpr FAnsiStringView ClassName = "java/nio/ByteBuffer";
	};
}

namespace UE::Jni
{
	struct FMediaPlayer14: Java::Lang::FObject
	{
		static constexpr FAnsiStringView ClassName = "com/epicgames/unreal/MediaPlayer14";

		inline static TConstructor<FMediaPlayer14(jboolean SwizzlePixels, jboolean VulkanRenderer, jboolean NeedTrackInfo)> New;

		static jint JNICALL nativeReadAt(JNIEnv* env, jobject thiz, jlong Identifier, jlong Position, Nio::FByteBuffer* Buffer, jint Offset, jint Count);

		static constexpr FMember Members[]
		{
			UE_JNI_MEMBER(New)
		};
		
		static constexpr FNativeMethod NativeMethods[]
		{
			UE_JNI_NATIVE_METHOD(nativeReadAt)
		};
	};
	
	template struct TInitialize<FMediaPlayer14>;
}

// Wrapper for com/epicgames/unreal/MediaPlayer*.java.
class FJavaAndroidMediaPlayer : public UE::Jni::TClassObject<UE::Jni::FMediaPlayer14>
{
public:
	struct FAudioTrack
	{
		int32 Index;
		FString MimeType;
		FString DisplayName;
		FString Language;
		FString Name;
		uint32 Channels;
		uint32 SampleRate;
	};

	struct FCaptionTrack
	{
		int32 Index;
		FString MimeType;
		FString DisplayName;
		FString Language;
		FString Name;
	};

	struct FVideoTrack
	{
		int32 Index;
		FString MimeType;
		FString DisplayName;
		FString Language;
		FString Name;
		uint32 BitRate;
		FIntPoint Dimensions;
		float FrameRate;
	};

public:
	FJavaAndroidMediaPlayer(bool swizzlePixels, bool vulkanRenderer, bool needTrackInfo, bool forceBitmapRenderLegacy = false);
	virtual ~FJavaAndroidMediaPlayer();
	int32 GetDuration();
	void Reset();
	void Stop();
	int32 GetCurrentPosition();
	bool IsLooping();
	bool IsPlaying();
	bool IsPrepared();
	bool DidComplete();
	bool SetDataSource(const FString & Url);
	bool SetDataSource(const TSharedRef<FArchive, ESPMode::ThreadSafe>& InArchive);
	bool SetDataSource(const FString& MoviePathOnDevice, int64 offset, int64 size);
	bool SetDataSource(jobject AssetMgr, const FString& AssetPath, int64 offset, int64 size);
	bool Prepare();
	bool PrepareAsync();
	void SeekTo(int32 Milliseconds);
	void SetLooping(bool Looping);
	void Release();
	int32 GetVideoHeight();
	int32 GetVideoWidth();
	void SetVideoEnabled(bool enabled = true);
	void SetAudioEnabled(bool enabled = true);
	void SetAudioVolume(float Volume);
	bool IsUsingBitmapRendererImageReader();
	jobject GetVideoLastMediaFrameData();
	bool GetVideoLastFrameData(void* & outPixels, int64 & outCount);
	void Start();
	void Pause();
	bool GetVideoLastFrame(int32 destTexture);
	bool SelectTrack(int32 index);
	bool GetAudioTracks(TArray<FAudioTrack>& AudioTracks);
	bool GetCaptionTracks(TArray<FCaptionTrack>& CaptionTracks);
	bool GetVideoTracks(TArray<FVideoTrack>& VideoTracks);
	bool DidResolutionChange();
	int32 GetExternalTextureId();
	bool UpdateVideoFrame(int32 ExternalTextureId, int32 *CurrentPosition, bool *bRegionChanged);

private:
	static FName GetClassName();

	bool bTrackInfoSupported;

	FJavaClassMethod GetDurationMethod;
	FJavaClassMethod ResetMethod;
	FJavaClassMethod StopMethod;
	FJavaClassMethod GetCurrentPositionMethod;
	FJavaClassMethod IsLoopingMethod;
	FJavaClassMethod IsPlayingMethod;
	FJavaClassMethod IsPreparedMethod;
	FJavaClassMethod DidCompleteMethod;
	FJavaClassMethod SetDataSourceURLMethod;
	FJavaClassMethod SetDataSourceArchiveMethod;
	FJavaClassMethod SetDataSourceFileMethod;
	FJavaClassMethod SetDataSourceAssetMethod;
	FJavaClassMethod PrepareMethod;
	FJavaClassMethod PrepareAsyncMethod;
	FJavaClassMethod SeekToMethod;
	FJavaClassMethod SetLoopingMethod;
	FJavaClassMethod ReleaseMethod;
	FJavaClassMethod GetVideoHeightMethod;
	FJavaClassMethod GetVideoWidthMethod;
	FJavaClassMethod SetVideoEnabledMethod;
	FJavaClassMethod SetAudioEnabledMethod;
	FJavaClassMethod SetAudioVolumeMethod;
	FJavaClassMethod GetVideoLastFrameDataMethod;
	FJavaClassMethod StartMethod;
	FJavaClassMethod PauseMethod;
	FJavaClassMethod GetVideoLastFrameMethod;
	FJavaClassMethod SelectTrackMethod;
	FJavaClassMethod GetAudioTracksMethod;
	FJavaClassMethod GetCaptionTracksMethod;
	FJavaClassMethod GetVideoTracksMethod;
	FJavaClassMethod DidResolutionChangeMethod;
	FJavaClassMethod GetExternalTextureIdMethod;
	FJavaClassMethod UpdateVideoFrameMethod;
	// new way of getting video frame data
	FJavaClassMethod SetUsingBitmapRendererImageReaderMethod;
	FJavaClassMethod GetVideoLastMediaFrameDataMethod;

	// FrameUpdateInfo member field ids
	jclass FrameUpdateInfoClass;
	jfieldID FrameUpdateInfo_CurrentPosition;
	jfieldID FrameUpdateInfo_FrameReady;
	jfieldID FrameUpdateInfo_RegionChanged;
	jfieldID FrameUpdateInfo_UScale;
	jfieldID FrameUpdateInfo_UOffset;
	jfieldID FrameUpdateInfo_VScale;
	jfieldID FrameUpdateInfo_VOffset;

	// AudioDeviceInfo member field ids
	jclass AudioTrackInfoClass;
	jfieldID AudioTrackInfo_Index;
	jfieldID AudioTrackInfo_MimeType;
	jfieldID AudioTrackInfo_DisplayName;
	jfieldID AudioTrackInfo_Language;
	jfieldID AudioTrackInfo_Channels;
	jfieldID AudioTrackInfo_SampleRate;

	// AudioDeviceInfo member field ids
	jclass CaptionTrackInfoClass;
	jfieldID CaptionTrackInfo_Index;
	jfieldID CaptionTrackInfo_MimeType;
	jfieldID CaptionTrackInfo_DisplayName;
	jfieldID CaptionTrackInfo_Language;

	// AudioDeviceInfo member field ids
	jclass VideoTrackInfoClass;
	jfieldID VideoTrackInfo_Index;
	jfieldID VideoTrackInfo_MimeType;
	jfieldID VideoTrackInfo_DisplayName;
	jfieldID VideoTrackInfo_Language;
	jfieldID VideoTrackInfo_BitRate;
	jfieldID VideoTrackInfo_Width;
	jfieldID VideoTrackInfo_Height;
	jfieldID VideoTrackInfo_FrameRate;

	FTextureRHIRef VideoTexture;
	bool bVideoTextureValid;

	float UScale, UOffset;
	float VScale, VOffset;

	/** Current media data source if using one. */
	TSharedPtr<FJavaAndroidMediaDataSource, ESPMode::ThreadSafe> MediaDataSource;

	enum BitmapRendererSelection 
	{
		Unknown,
		Legacy,
		ImageReader 
	};

	BitmapRendererSelection BRSelection; // BR - BitmapRenderer
	bool UsingVulkanRenderer;

	void SelectBitmapRenderer();

public:
	FTextureRHIRef GetVideoTexture()
	{
		return VideoTexture;
	}

	void SetVideoTexture(FTextureRHIRef Texture)
	{
		VideoTexture = Texture;
	}

	void SetVideoTextureValid(bool Condition)
	{
		bVideoTextureValid = Condition;
	}

	bool IsVideoTextureValid()
	{
		return bVideoTextureValid;
	}

	float GetUScale() { return UScale; }
	float GetUOffset() { return UOffset; }

	float GetVScale() { return VScale; }
	float GetVOffset() { return VOffset; }

protected:
	static FCriticalSection MediaDataSourcesCS;
	static TMap<int64, TWeakPtr<FJavaAndroidMediaDataSource, ESPMode::ThreadSafe>> AllMediaDataSources;

public:
	static TSharedPtr<FJavaAndroidMediaDataSource, ESPMode::ThreadSafe> GetMediaDataSourcePtr(int64 Identifier);
	static void AddMediaDataSourcePtr(int64 Identifier, TSharedPtr<FJavaAndroidMediaDataSource, ESPMode::ThreadSafe> MediaDataSource);
};

#endif