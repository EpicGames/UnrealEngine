// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformApplicationMisc.h"

struct FAndroidApplicationMisc : public FGenericPlatformApplicationMisc
{
	static APPLICATIONCORE_API void LoadPreInitModules();
	static APPLICATIONCORE_API class FFeedbackContext* GetFeedbackContext();
	static APPLICATIONCORE_API class FOutputDeviceError* GetErrorOutputDevice();
	static APPLICATIONCORE_API class GenericApplication* CreateApplication();
	static APPLICATIONCORE_API void RequestMinimize();
	static APPLICATIONCORE_API bool IsScreensaverEnabled();
	static APPLICATIONCORE_API bool ControlScreensaver(EScreenSaverAction Action);
	static APPLICATIONCORE_API void SetGamepadsAllowed(bool bAllowed);
	static APPLICATIONCORE_API void SetGamepadsBlockDeviceFeedback(bool bBlock);
	static APPLICATIONCORE_API void ResetGamepadAssignments();
	static APPLICATIONCORE_API void ResetGamepadAssignmentToController(int32 ControllerId);
	static APPLICATIONCORE_API bool IsControllerAssignedToGamepad(int32 ControllerId);
	static APPLICATIONCORE_API FString GetGamepadControllerName(int32 ControllerId);
	static APPLICATIONCORE_API void ClipboardCopy(const TCHAR* Str);
	static APPLICATIONCORE_API void ClipboardPaste(class FString& Dest);
	static APPLICATIONCORE_API EScreenPhysicalAccuracy ComputePhysicalScreenDensity(int32& OutScreenDensity);
	static APPLICATIONCORE_API EScreenPhysicalAccuracy ComputePhysicalScreenDimensions(FVector2f& OutScreenDimensions, float& OutScreenSize);
	static APPLICATIONCORE_API void EnableMotionData(bool bEnable);

	static EScreenPhysicalAccuracy GetNativePhysicalScreenDensity(int32& OutScreenDensity);
	static void GetNativePhysicalScreenResolution(FIntVector2& OutScreenResolution);
};

typedef FAndroidApplicationMisc FPlatformApplicationMisc;
