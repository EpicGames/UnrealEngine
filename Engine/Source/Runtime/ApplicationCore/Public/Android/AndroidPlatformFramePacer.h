// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	AndroidPlatformFramePacer.h: Android platform frame pacer classes.
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformFramePacer.h"
#include "HAL/IConsoleManager.h"

/**
 * Android implementation of FGenericPlatformRHIFramePacer
 **/

struct IAndroidFramePacer
{
	virtual void Init() = 0;
	virtual ~IAndroidFramePacer() = default;
	virtual bool SwapBuffers(struct FPlatformOpenGLSurfaceContext* SurfaceContext, bool bLockToVsync) { return true; }
	virtual bool SupportsFramePace(int32 QueryFramePace) = 0;
};

struct FAndroidOpenGLFramePacer : public IAndroidFramePacer
{
	virtual void Init() override;
	virtual ~FAndroidOpenGLFramePacer();
	virtual bool SwapBuffers(struct FPlatformOpenGLSurfaceContext* SurfaceContext, bool bLockToVsync) override;
	virtual bool SupportsFramePace(int32 QueryFramePace) override;

private:
	void InitSwappy();
	bool SupportsFramePaceInternal(int32 QueryFramePace, int32& OutRefreshRate, int32& OutSyncInterval);

	// swappy state
	bool bSwappyInit = false;
	int32 CachedFramePace = 60;
	int32 CachedRefreshRate = 60;
	int32 CachedSyncInterval = 1;
	struct ANativeWindow* CachedNativeWindow = nullptr;

	// legacy framepacer state
	int32 DesiredSyncIntervalRelativeTo60Hz = -1;
	int32 DesiredSyncIntervalRelativeToDevice = -1;
	int32 DriverSyncIntervalRelativeToDevice = -1;
	float DriverRefreshRate = 60.0f;
	int64 DriverRefreshNanos = 16666666;
	double LastTimeEmulatedSync = -1.0;
	uint32 SwapBufferFailureCount = 0;
};

struct FAndroidVulkanFramePacer : public IAndroidFramePacer
{
	virtual void Init() override;
	virtual ~FAndroidVulkanFramePacer() = default;
	virtual bool SupportsFramePace(int32 QueryFramePace) override;
private:
	bool SupportsFramePaceInternal(int32 QueryFramePace, int32& OutRefreshRate, int32& OutSyncInterval);
	friend class FVulkanAndroidPlatform;
};

struct FAndroidPlatformRHIFramePacer : public FGenericPlatformRHIFramePacer
{
    // FGenericPlatformRHIFramePacer interface
    static bool IsEnabled();
    static void Destroy();
	static int32 GetFramePace();
	static int32 SetFramePace(int32 FramePace);
	static bool SupportsFramePace(int32 QueryFramePace) { return (ensure(FramePacer)) ? FramePacer->SupportsFramePace(QueryFramePace) : false; }

	// FAndroidPlatformRHIFramePacer interface
	static void Init(IAndroidFramePacer* InFramePacer);
	static int32 GetLegacySyncInterval();
	static void SwapBuffers(struct FPlatformOpenGLSurfaceContext* SurfaceContext, bool bLockToVsync) { if (ensure(FramePacer)) { FramePacer->SwapBuffers(SurfaceContext, bLockToVsync); } }

	static TAutoConsoleVariable<int32> CVarUseSwappyForFramePacing;
	static TAutoConsoleVariable<int32> CVarSwappyGPUFrameTimeFence;
	static TAutoConsoleVariable<int32> CVarSupportNonVSyncMultipleFrameRates;

	// Legacy pacer stuff
	static TAutoConsoleVariable<int32> CVarAllowFrameTimestamps;
	static TAutoConsoleVariable<int32> CVarTimeStampErrorRetryCount;
	static TAutoConsoleVariable<int32> CVarUseGetFrameTimestamps;
	static TAutoConsoleVariable<int32> CVarSpewGetFrameTimestamps;
	static TAutoConsoleVariable<float> CVarStallSwap;
	static TAutoConsoleVariable<int32> CVarDisableOpenGLGPUSync;
	   
private:
    /** The actual GL or Vulkan frame pacer */
	static IAndroidFramePacer* FramePacer;
	friend struct FAndroidOpenGLFramePacer;
	friend struct FAndroidVulkanFramePacer;

	/* Actual current frame pace */
	static int32 InternalFramePace;
};

typedef FAndroidPlatformRHIFramePacer FPlatformRHIFramePacer;
