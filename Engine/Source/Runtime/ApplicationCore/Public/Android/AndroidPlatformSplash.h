// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	AndroidSplash.h: Android platform splash screen...
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformSplash.h"

/**
 * Android splash implementation
 */
struct FAndroidSplash : public FGenericPlatformSplash
{
	// default implementation for now
};


typedef FAndroidSplash FPlatformSplash;
