// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	AndroidSurvey.h: Google Android platform hardware-survey classes
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformSurvey.h"

/**
 * Android implementation of FGenericPlatformSurvey
 **/
struct FAndroidPlatformSurvey : public FGenericPlatformSurvey
{
	static bool GetSurveyResults(FHardwareSurveyResults& OutResults, bool bWait=false);
};

typedef FAndroidPlatformSurvey FPlatformSurvey;