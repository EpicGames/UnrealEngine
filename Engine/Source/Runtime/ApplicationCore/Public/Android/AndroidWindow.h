// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericWindow.h"
#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include <android/native_window.h> 
#if USE_ANDROID_JNI
#include <android/native_window_jni.h>
#endif
#include "Async/SharedMutex.h"
#include "Async/Mutex.h"
#include "Async/UniqueLock.h"

/**
 * A platform specific implementation of FNativeWindow.
 * Native windows provide platform-specific backing for and are always owned by an SWindow.
 */
 
class FAndroidWindow : public FGenericWindow, public TSharedFromThis<FAndroidWindow>
{
public:
	APPLICATIONCORE_API ~FAndroidWindow();

	/** Create a new FAndroidWindow.
	 *
	 * @param OwnerWindow		The SlateWindow for which we are crating a backing AndroidWindow
	 * @param InParent			Parent iOS window; usually NULL.
	 */
	static APPLICATIONCORE_API TSharedRef<FAndroidWindow> Make();

	APPLICATIONCORE_API virtual void Destroy() override;
	
	virtual void* GetOSWindowHandle() const override { return (void*) this; }

	APPLICATIONCORE_API void Initialize( class FAndroidApplication* const Application, const TSharedRef< FGenericWindowDefinition >& InDefinition, const TSharedPtr< FAndroidWindow >& InParent, const bool bShowImmediately );

	/** Returns the rectangle of the screen the window is associated with */
	APPLICATIONCORE_API virtual bool GetFullScreenInfo( int32& X, int32& Y, int32& Width, int32& Height ) const override;

	APPLICATIONCORE_API virtual void SetOSWindowHandle(void*);

	static APPLICATIONCORE_API FPlatformRect GetScreenRect(bool bUseEventThreadWindow = false);
	static APPLICATIONCORE_API void InvalidateCachedScreenRect();

	// When bUseEventThreadWindow == false this uses dimensions cached when the game thread processes android events.
	// When bUseEventThreadWindow == true this uses dimensions directly from the android event thread, unless called from event thread this requires acquiring GAndroidWindowLock to use.
	static APPLICATIONCORE_API void CalculateSurfaceSize(int32_t& SurfaceWidth, int32_t& SurfaceHeight, bool bUseEventThreadWindow = false);
	static APPLICATIONCORE_API bool OnWindowOrientationChanged(EDeviceScreenOrientation DeviceScreenOrientation);

	static APPLICATIONCORE_API int32 GetDepthBufferPreference();
	
	static APPLICATIONCORE_API void AcquireWindowRef(ANativeWindow* InWindow);
	static APPLICATIONCORE_API void ReleaseWindowRef(ANativeWindow* InWindow);

	// This returns the current hardware window as set from the event thread.
	static APPLICATIONCORE_API void* GetHardwareWindow_EventThread();
	static APPLICATIONCORE_API void SetHardwareWindow_EventThread(void* InWindow);

	/** Waits on the current thread for a hardware window and returns it. 
	 *  May return nullptr if the application is shutting down.
	 */
	static APPLICATIONCORE_API void* WaitForHardwareWindow();

	static APPLICATIONCORE_API bool IsPortraitOrientation();
	static APPLICATIONCORE_API FVector4 GetSafezone(bool bPortrait);

	// called by the Android event thread to initially set the current window dimensions.
	static APPLICATIONCORE_API void SetWindowDimensions_EventThread(ANativeWindow* DimensionWindow);

	// Called by the event manager to update the cached window dimensions to match the event it is processing.
	static APPLICATIONCORE_API void EventManagerUpdateWindowDimensions(int32 Width, int32 Height);

	APPLICATIONCORE_API bool GetNativeWindowResolution(int32_t& OutWidth, int32_t& OutHeight) const;

	virtual void SetWindowMode(EWindowMode::Type InNewWindowMode) { WindowMode = InNewWindowMode; }

	class FNativeAccessor
	{
		bool bWriteAccess;
		class FAndroidWindow* ProtectedObj = nullptr;
		FNativeAccessor() {}
	public:
		FNativeAccessor(FNativeAccessor&& obj)
		{
			ProtectedObj = obj.ProtectedObj;
			bWriteAccess = obj.bWriteAccess;
			obj.ProtectedObj = nullptr;
		}

		FNativeAccessor(const FNativeAccessor&) = delete;
		FNativeAccessor(bool bWriteAccessIn, FAndroidWindow* ProtectedObjIn) : bWriteAccess(bWriteAccessIn), ProtectedObj(ProtectedObjIn) 
		{ 
			if (bWriteAccess)
			{
				ProtectedObj->ANativeHandleMutex.Lock();
			}
			else
			{
				ProtectedObj->ANativeHandleMutex.LockShared();
			}
		}

		~FNativeAccessor()
		{ 
			if(ProtectedObj)
			{
				if (bWriteAccess)
				{
					ProtectedObj->ANativeHandleMutex.Unlock();
				}
				else
				{
					ProtectedObj->ANativeHandleMutex.UnlockShared();
				}
			}
		}

		FAndroidWindow* Get() { return ProtectedObj; }
		const FAndroidWindow* Get() const { return ProtectedObj; }

		FAndroidWindow* operator ->() { return Get(); }
		const FAndroidWindow* operator ->() const { return Get(); }

		void SetANativeWindow(ANativeWindow* ANativeHandle)
		{ 
			check(bWriteAccess); 
			ProtectedObj->PreviousANativeHandle = ProtectedObj->CurrentANativeHandle;
			ProtectedObj->CurrentANativeHandle = ANativeHandle; 
		}
		ANativeWindow* GetANativeWindow() const { return ProtectedObj->CurrentANativeHandle; }
		ANativeWindow* GetPreviousANativeWindow() const { return ProtectedObj->PreviousANativeHandle; }
	};

	FNativeAccessor GetANativeAccessor(bool bWriteAccess)
	{
		return FNativeAccessor(bWriteAccess, this);
	}

protected:
	/** @return true if the native window is currently in fullscreen mode, false otherwise */
	virtual EWindowMode::Type GetWindowMode() const override { return WindowMode; }

private:
	ANativeWindow* CurrentANativeHandle = nullptr;
	ANativeWindow* PreviousANativeHandle = nullptr;
	UE::FSharedMutex ANativeHandleMutex;
	EWindowMode::Type WindowMode = EWindowMode::Fullscreen;

	/**
	 * Protect the constructor; only TSharedRefs of this class can be made.
	 */
	APPLICATIONCORE_API FAndroidWindow();

	FAndroidApplication* OwningApplication;

	static APPLICATIONCORE_API void* NativeWindow;

	// Waits for the event thread to report an initial window size.
	static APPLICATIONCORE_API bool WaitForWindowDimensions();

	static APPLICATIONCORE_API bool bAreCachedNativeDimensionsValid;
	static APPLICATIONCORE_API int32 CachedNativeWindowWidth;
	static APPLICATIONCORE_API int32 CachedNativeWindowHeight;

	friend class FAndroidWindowManager;
};

class FAndroidWindowManager
{
	friend class FAndroidWindow;

public:

	static FAndroidWindowManager& Get()
	{
		static FAndroidWindowManager Single;
		return Single;
	}

	TOptional<FAndroidWindow::FNativeAccessor> GetMainWindowAsNativeAccessor(bool bWriteAccess)
	{
		UE::TUniqueLock Lock(WindowManagerLock);
		if (Windows.IsEmpty())
		{
			return TOptional<FAndroidWindow::FNativeAccessor>();
		}

		FAndroidWindow::FNativeAccessor Accessor = Windows[0]->GetANativeAccessor(bWriteAccess);
		return TOptional<FAndroidWindow::FNativeAccessor>(MoveTemp(Accessor));
	}


	TOptional<FAndroidWindow::FNativeAccessor > FindFromANativeWindow(bool bWriteAccess, bool bSearchPrevious, const void* SearchANativeHandle)
	{
		UE::TUniqueLock Lock(WindowManagerLock);
		for (TSharedRef<FAndroidWindow>& Window : Windows)
		{
			FAndroidWindow::FNativeAccessor Accessor = Window->GetANativeAccessor(bWriteAccess);
			if (SearchANativeHandle == ( bSearchPrevious ? Accessor.GetPreviousANativeWindow() : Accessor.GetANativeWindow() ) )
			{
				return TOptional<FAndroidWindow::FNativeAccessor>(MoveTemp(Accessor));
			}
		}

		return TOptional<FAndroidWindow::FNativeAccessor>();
	}

	// Android native window handling.
	// this is used to collect the native window before the game has created its own FAndroidWindow.
	class FPendingWindowAccessor
	{
		static inline ANativeWindow* PreviousPendingANativeWindow = nullptr;
		static inline ANativeWindow* PendingANativeWindow = nullptr;
		static inline UE::FMutex Lock;
		FPendingWindowAccessor(const FPendingWindowAccessor&) = delete;
		FPendingWindowAccessor(FPendingWindowAccessor&&) = delete;
	public:
		FPendingWindowAccessor() { Lock.Lock(); }
		~FPendingWindowAccessor() { Lock.Unlock(); }

		void SetANativeWindow(ANativeWindow* ANativeHandle) 
		{ 
			PreviousPendingANativeWindow = PendingANativeWindow;
			PendingANativeWindow = ANativeHandle;
		}

		ANativeWindow* GetPreviousANativeWindow() const { return PreviousPendingANativeWindow; }
		ANativeWindow* GetANativeWindow() const { return PendingANativeWindow; }
	};

	static FPendingWindowAccessor GetPendingWindowAccessor()
	{
		return FPendingWindowAccessor();
	}

private:
	TArray< TSharedRef<FAndroidWindow>> Windows;

	int AddWindow(TSharedRef<FAndroidWindow> AddMe)
	{
		UE::TUniqueLock Lock(WindowManagerLock);
		Windows.Add(AddMe);
		return Windows.Num();
	}
	
	void RemoveWindow(TSharedRef<FAndroidWindow> RemoveMe)
	{
		UE::TUniqueLock Lock(WindowManagerLock);
		Windows.Remove(RemoveMe);
	}

	mutable UE::FMutex WindowManagerLock;
};
