// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/IConsoleManager.h"
#include "Misc/CommandLine.h"
#include "RHI.h"

#if PLATFORM_ANDROID
#include "Android/AndroidPlatformMisc.h"

#endif

DEFINE_LOG_CATEGORY_STATIC(LogAndroidWindowUtils, Log, All)

namespace AndroidWindowUtils
{
	FIntVector2 SanitizeAndroidScreenSize(const FIntVector2&& MaxScreenDims, const FIntVector2&& RequestedScreenDims)
	{
		FIntVector2 SanitizedDims;
		// ensure Width and Height is multiple of 8
		SanitizedDims.X = (RequestedScreenDims.X / 8) * 8;
		SanitizedDims.Y = (RequestedScreenDims.Y / 8) * 8;

		// clamp to native resolution
		SanitizedDims.X = FPlatformMath::Min(SanitizedDims.X, MaxScreenDims.X);
		SanitizedDims.Y = FPlatformMath::Min(SanitizedDims.Y, MaxScreenDims.Y);
		return SanitizedDims;
	}

	void ApplyContentScaleFactor(int32& InOutScreenWidth, int32& InOutScreenHeight)
	{
		// CSF is a multiplier to 1280x720
		static IConsoleVariable* CVarScale = IConsoleManager::Get().FindConsoleVariable(TEXT("r.MobileContentScaleFactor"));
		float RequestedContentScaleFactor = CVarScale->GetFloat();

		static IConsoleVariable* CVarResX = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.DesiredResX"));
		static IConsoleVariable* CVarResY = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.DesiredResY"));
		int32 RequestedResX = CVarResX->GetInt();
		int32 RequestedResY = CVarResY->GetInt();

		FString CmdLineCSF;
		if (FParse::Value(FCommandLine::Get(), TEXT("mcsf="), CmdLineCSF, false))
		{
			RequestedContentScaleFactor = FCString::Atof(*CmdLineCSF);
		}

		FString CmdLineMDRes;
		if (FParse::Value(FCommandLine::Get(), TEXT("mobileresx="), CmdLineMDRes, false))
		{
			RequestedResX = FCString::Atoi(*CmdLineMDRes);
		}
		if (FParse::Value(FCommandLine::Get(), TEXT("mobileresy="), CmdLineMDRes, false))
		{
			RequestedResY = FCString::Atoi(*CmdLineMDRes);
		}

		// 0 means to use native size
		if (RequestedContentScaleFactor == 0.0f && RequestedResX <= 0 && RequestedResY <= 0)
		{
			UE_LOGF(LogAndroidWindowUtils, Log, "Setting Width=%d and Height=%d (requested scale = 0 = auto)", InOutScreenWidth, InOutScreenHeight);
		}
		else
		{
			int32 Width = InOutScreenWidth;
			int32 Height = InOutScreenHeight;

			if (RequestedResX > 0)
			{
				// set long side for orientation to requested X (height > width == GAndroidIsPortrait)
				if (InOutScreenHeight > InOutScreenWidth)
				{
					Height = RequestedResX;
					Width = FMath::TruncToInt32((float)Height * ((float)InOutScreenWidth / (float)InOutScreenHeight) + 0.5f);
				}
				else
				{
					Width = RequestedResX;
					Height = FMath::TruncToInt32((float)Width * ((float)InOutScreenHeight / (float)InOutScreenWidth) + 0.5f);
				}
			}
			else if (RequestedResY > 0)
			{
				// set short side for orientation to requested Y (height > width == GAndroidIsPortrait)
				if (InOutScreenHeight > InOutScreenWidth)
				{
					Width = RequestedResY;
					Height = FMath::TruncToInt32((float)Width * ((float)InOutScreenHeight / (float)InOutScreenWidth) + 0.5f);
				}
				else
				{
					Height = RequestedResY;
					Width = FMath::TruncToInt32((float)Height * ((float)InOutScreenWidth / (float)InOutScreenHeight) + 0.5f);
				}
			}
			else
			{
				const float AspectRatio = (float)InOutScreenWidth / (float)InOutScreenHeight;

				// set resolution relative to 1280x720 ((height > width == GAndroidIsPortrait)
				if (InOutScreenHeight > InOutScreenWidth)
				{
					Height = FMath::TruncToInt32(1280.f * RequestedContentScaleFactor);
				}
				else
				{
					Height = FMath::TruncToInt32(720.f * RequestedContentScaleFactor);
				}

				// apply the aspect ration to get the width
				Width = FMath::TruncToInt32((float)Height * AspectRatio + 0.5f);
			}

			const FIntVector2 Sanitized = SanitizeAndroidScreenSize(FIntVector2(InOutScreenWidth, InOutScreenHeight), FIntVector2(Width, Height));
			InOutScreenWidth = Sanitized.X;
			InOutScreenHeight = Sanitized.Y;

			UE_LOGF(LogAndroidWindowUtils, Log, "Setting Width=%d and Height=%d (requested scale = %f)", InOutScreenWidth, InOutScreenHeight, RequestedContentScaleFactor);
		}
	}
} // end AndroidWindowUtils
