// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Logging/LogMacros.h"
#include "Math/Box2D.h"
#include "Misc/Variant.h"
#include "Stats/Stats.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

/** Whether a widget should be included in accessibility, and if so, how its text should be retrieved. */
enum class EAccessibleBehavior : uint8
{
	/** Not accessible. */
	NotAccessible,
	/** Accessible, for the implementing library to decide what it means. Given all data about a particular widget, it should try to choose the most-relevant text automatically. */
	Auto,
	/** Accessible, and traverse all child widgets and concat their summary text together. */
	Summary,
	/** Accessible, and retrieve manually-assigned text from a TAttribute. */
	Custom,
	/** Accessible, and use the tooltip's accessible text. */
	ToolTip
};

APPLICATIONCORE_API DECLARE_LOG_CATEGORY_EXTERN(LogAccessibility, Log, All);

#if WITH_ACCESSIBILITY

class FGenericWindow;
class IAccessibleWidget;

DECLARE_STATS_GROUP(TEXT("Accessibility"), STATGROUP_Accessibility, STATCAT_Advanced);

/** What kind of widget to tell the operating system this is. This may be translated to a different type depending on the platform. */
enum class EAccessibleWidgetType : uint8
{
	Unknown,
	Button,
	CheckBox,
	ComboBox,
	Hyperlink,
	Image,
	Layout,
	ScrollBar,
	Slider,
	Text,
	TextEdit,
	Window,
	List,
	ListItem
};

/** Events that can be raised from accessible widgets to report back to the platform */
enum class EAccessibleEvent : uint8
{
	/**
	 * A widget has become focused or unfocused.
	 * OldValue - The old focus state of the widget
	 * NewValue - The new focus state of the widget
	 */
	FocusChange,
	/**
	 * A widget has been clicked, checked, or otherwise activated.
	 * OldValue - N/A
	 * NewValue - N/A
	 */
	Activate,
	/**
	 * Warning: Partial implementation
	 * Notify the user that something has happened. The user is not guaranteed to get this message.
	 * OldValue - N/A
	 * NewValue - An FString of the message to read
	 */
	Notification,
	/**
	 * A widget's parent is about to be changed.
	 * OldValue - The AccessibleWidgetId of the old parent, or InvalidAccessibleWidgetId if there was none
	 * NewValue - The AccessibleWidgetId of the new parent, or InvalidAccessibleWidgetId if there was none
	 */
	ParentChanged,
	/**
	 * The widget was removed from the UI tree or deleted.
	 * OldValue - N/A
	 * NewValue - N/A
	 */
	WidgetRemoved
};

/** An index that uniquely identifies every registered accessible user in the application. */
typedef int32 FAccessibleUserIndex;

/**
 * An accessible user is an input source that can interact with the application and needs to provide accessibility services such as a screen reader.
 * This base class provides the basic information all accessible users need to implement to ensure accessibility services function properly.
 * Users should subclass this class to provide additional functionality for their custom accessible users.
 * All accessible users must be registered with an FGenericAccessibleUserRegistry for the users to be retrieved and interacted with.
 * @see FGenericAccessibleUserRegistry
 */
class FGenericAccessibleUser
{
	friend class FGenericAccessibleUserRegistry;
public:
	FGenericAccessibleUser(const FAccessibleUserIndex InUserIndex)
		: UserIndex(InUserIndex)
	{
		
	}
	virtual ~FGenericAccessibleUser() = default;
	/** Returns the unique index that identifies this accessible user.*/
	FAccessibleUserIndex GetIndex() const
	{
		return UserIndex;
	}
	/**
	 * Returns the accessible widget the accessible user is currently focused on. If no accessible widget is focused, nullptr is returned.
	 * @see IAccessibleWidget
	 */
	TSharedPtr<IAccessibleWidget> GetFocusedAccessibleWidget() const
	{
		return FocusedAccessibleWidget.IsValid() ? FocusedAccessibleWidget.Pin() : nullptr;
	}
	/**
	 * Sets which accessible widget the accessible user is currently focused on.
	 *
	 * Note: This function is meant to directly set the widget an accessible user is focused on, bypassing all of the focus change accessible events.
	 * This should only be used by FGenericAccessibleMessageHandler and its subclasses to update the  the accessible focus for an accessible user.
	 * Most of the time, you want to use IAccessibleWidget::SetUserFocus to request an accessible widget be focused on by a user. It raises the appropriate accessible events
	 * so listeners for accessible events can do the appropriate event handling for the focus change event.
	 * @param InWidget The widget this accessible user will be focused on.
	 * @return True if the passed in widget was successfully set. Else returns false.
	 */
	bool SetFocusedAccessibleWidget(const TSharedRef<IAccessibleWidget>& InWidget)
	{
		FocusedAccessibleWidget = InWidget;
		return true;
	}
	/** Clear the accessible focused widget. */
	void ClearFocusedAccessibleWidget()
	{
		FocusedAccessibleWidget.Reset();
	}
protected:
	/**
	 * Called by FGenericAccessibleUserRegistry::RegisterUser.
	 * @see FGenericAccessibleUserRegistry
	 */
	virtual void OnRegistered() {}
	/**
	 * Called by FGenericAccessibleUserRegistry::UnregisterUser.
	 * @see FGenericAccessibleUserRegistry
	 */
	virtual void OnUnregistered() {}
	
private:
	/** The unique index for this accessible user */
	FAccessibleUserIndex UserIndex;
	/** The focused accessible widget widget associated with this user. */
	TWeakPtr<IAccessibleWidget> FocusedAccessibleWidget;
};

/**
 * The base class for an accessible user registry that all accessible users must register with.
 * The registry controls the lifetime of all accessible users.
 * All created accessible users must be registered with an accessible user registry. This should be the only means
 * of retrieving and interacting with accessible users in other parts of the application.
 * Users can subclass this class to provide additional functionality for their custom use cases.
 * @see FGenericAccessibleUser, FGenericAccessibleMessageHandler
 */
class FGenericAccessibleUserRegistry
{
public:
	virtual ~FGenericAccessibleUserRegistry() = default;
	/**
	 * Registers an accessible user with the accessible registry.
	 * It is up to the caller to ensure that the passed in user has the correct user index.
	 * The passed in user will not be registered if another user with the same user index has already been registered.
	 * This function calls FGenericAccessibleUser::OnRegistered if the passed in user is successfully registered.
	 * Once an accessible user has been successfully registered, it can be retrieved and interacted with by the rest of the application.
	 * @param User An accessible user you want to register with the accessible user registry.
	 * @return True if the passed in user was successfully registered. Else, returns false.
	 */
	APPLICATIONCORE_API bool RegisterUser(const TSharedRef<FGenericAccessibleUser>& User);
	/**
	 * Unregisters an accessible user from the accessible registry.
	 * If the passed in user index is not associated with a registered accessible user, nothing will happen.
	 * This function calls FGenericAccessibleUser::OnUnregistered if the passed in user is successfully unregistered.
	 * Once an accessible user has been unregistered, the rest of the application should have no way of retrieving or interacting with the accessible user.
	 * @param UserIndex The index of the accessible user you want to unregister.
	 * @return True if the accessible user with the associated user index is successfully unregistered. Else, returns false.
	 */
	APPLICATIONCORE_API bool UnregisterUser(const FAccessibleUserIndex UserIndex);
	/**
	 * Unregisters all accessible users from the accessible registry.
	 * This function calls FGenericAccessibleUser::OnUnregistered when each registered user is successfully unregistered.
	 */
	APPLICATIONCORE_API void UnregisterAllUsers();
	/** Returns true if the passed in user index is associated with a registered accessible user. Else, returns false. */
	APPLICATIONCORE_API bool IsUserRegistered(const FAccessibleUserIndex UserIndex) const;
	/** Returns the accessible user associated with the passed in user index. If the passed in user index is not associated with any accessible user, nullptr is returned. */
	APPLICATIONCORE_API TSharedPtr<FGenericAccessibleUser> GetUser(const FAccessibleUserIndex UserIndex) const;
	/** Returns the number of accessible users registered with the accessible user registry. */
	APPLICATIONCORE_API int32 GetNumberofUsers() const;
	/** Returns an array of all accessible users that are currently registered with the accessible registry. */
	APPLICATIONCORE_API TArray<TSharedRef<FGenericAccessibleUser>> GetAllUsers() const;
	/**
	 * Returns the index of the primary accessible user.
	 * The primary accessible user should correspond to the default input source that every application should have. This user should also be the primary cursor user.
	 * E.g On desktop platforms, that would correspond to the keyboard user.
	 */
	static FAccessibleUserIndex GetPrimaryUserIndex()
	{
		// @TODOAccessibility: Consider allowing remapping of primary user index
		static const FAccessibleUserIndex PrimaryUserIndex = 0;
		return PrimaryUserIndex;
	}
protected:
	/** A map of accessible user indices to their associated accessible users. */
	TMap<FAccessibleUserIndex, TSharedRef<FGenericAccessibleUser>> UsersMap;
};

/**
 * An accessible window corresponds to a native OS window. Fake windows that are embedded
 * within other widgets that simply look and feel like windows are not IAccessibleWindows.
 */
class IAccessibleWindow
{
public:
	/** The different states a window can be in to describe its screen anchors */
	enum class EWindowDisplayState
	{
		Normal,
		Minimize,
		Maximize
	};

	/**
	 * Retrieve the native OS window backing this accessible window. This can be safely
	 * casted if you know what OS you're in (ie FWindowsWindows on Windows platform).
	 *
	 * @return The native window causing this accessible window to exist
	 */
	virtual TSharedPtr<FGenericWindow> GetNativeWindow() const = 0;

	/**
	 * Finds the deepest accessible widget in the hierarchy at the specified coordinates. The window
	 * may return a pointer to itself in the case where there are no accessible children at the position.
	 * This could return nullptr in the case where the coordinates are outside the window bounds.
	 *
	 * @param X The X coordinate in absolute screen space
	 * @param Y The Y coordinate in absolute screen space
	 * @return The deepest widget in the UI heirarchy at X,Y
	 */
	virtual TSharedPtr<IAccessibleWidget> GetChildAtPosition(int32 X, int32 Y) = 0;
	/**
	 * Retrieves the focused accessible widget for a user.
	 * If the passed in user index is not registered with an accessible user registry, nullptr will be returned.
	 *
	 * @param UserIndex The user index of the accessible user to query for its accessible focus widget.
	 * @return The accessible widget that is focused by the accessible user. Returns nullptr if the accessible user index does not correspond to a registered accessible user.
	 */
	virtual TSharedPtr<IAccessibleWidget> GetUserFocusedWidget(const FAccessibleUserIndex UserIndex) const = 0;
	/**
	 * Request that the window closes itself. This may not happen immediately.
	 */
	virtual void Close() = 0;

	/**
	 * Check if the window can be minimized or maximized.
	 *
	 * @param State Whether to check for minimize or maximize.
	 * @return True if the display state can be switched to, otherwise false.
	 */
	virtual bool SupportsDisplayState(EWindowDisplayState State) const = 0;
	/**
	 * Gets the current state minimize/maximize state of the window.
	 *
	 * @return The display state corresponding to how the window is displayed.
	 */
	virtual EWindowDisplayState GetDisplayState() const = 0;
	/**
	 * Sets a window to be minimized, maximized, or restored to normal.
	 *
	 * @param State What to change the window's display to.
	 */
	virtual void SetDisplayState(EWindowDisplayState State) = 0;
	/**
	 * Whether or not the window is modal.
	 *
	 * @return true if the window is modal, otherwise false.
	 */
	virtual bool IsModal() const = 0;
};

/**
 * A widget that can be triggered to fire an event, such as buttons or checkboxes.
 */
class IAccessibleActivatable
{
public:
	/** Trigger the widget */
	virtual void Activate() = 0;
	/**
	 * Check whether this widget can be toggled between various check states
	 *
	 * @return true if this widget supports being in different states
	 */
	virtual bool IsCheckable() const { return false; }
	/**
	 * If IsCheckable() is true, this gets the current state that the widget is in.
	 * //todo: return ECheckState
	 *
	 * @return true if the current state is considered "on" or "checked"
	 */
	virtual bool GetCheckedState() const { return false; }
};

/**
 * An accessible widget that stores an arbitrary value of any type capable of being serialized into a string.
 * Optional overrides add support for slider-like functionality.
 */
class IAccessibleProperty
{
public:
	/**
	 * Whether the widget is in read-only mode, which could be different than IsEnabled().
	 *
	 * @return true if the widget is in read-only mode.
	 */
	virtual bool IsReadOnly() const { return true; }
	/**
	 * Check if this text is storing password data, indicating that it may need special handling to presenting itself to the user.
	 *
	 * @return true if the text is storing a password or otherwise senstive data that should be hidden.
	 */
	virtual bool IsPassword() const { return false; }
	/**
	 * How much the value should increase/decrease when the user attempts to modify the value using UI controls.
	 * Note: This should always return a positive value. The caller is responsible for negating it when attempting to decrease.
	 *
	 * @return A number suggesting how much to modify GetValue() by when the user wants to increase/decrease the value.
	 */
	virtual float GetStepSize() const { return 0.0f; }
	/**
	 * The maximum allowed value for this property. This should only be used if GetStepSize is not 0.
	 *
	 * @return The maximum value that this property can be assigned when using step sizes.
	 */
	virtual float GetMaximum() const { return 0.0f; }
	/**
	 * The minimum allowed value for this property. This should only be used if GetStepSize is not 0.
	 *
	 * @return The minimum value that this property can be assigned when using step sizes.
	 */
	virtual float GetMinimum() const { return 0.0f; }
	/**
	 * The current value stored by the widget. Even if the underlying value is not a String, it should be serialized to one
	 * in order to match the return type.
	 *
	 * @return A string representing the serialized value stored in the widget.
	 */
	virtual FString GetValue() const = 0;
	/**
	 * The current value stored by the widget as a FVariant. Used on platforms where the type of the value is important for accessibility methods
	 * such as on Mac. .
	 * @return An FVariant that holds the widget's current value and type of value.
	 */
	virtual FVariant GetValueAsVariant() const { return FVariant(); }
	/*
	 * Set the value stored by the widget. While this function accepts a String, there is no way to know
	 * what the underlying data is stored as. The platform layer must retain some additional information
	 * about what kind of widget this is, and ensure it's being called with valid arguments.
	 *
	 * @param Value The new value to assign to the widget, which may need to be converted before assigning to a variable.
	 */
	virtual void SetValue(const FString& Value) {}
};

/**
 * A widget that contains text, with the potential ability to select sections, read specific words/paragraphs, etc.
 * Note: This class is currently incomplete.
 */
class IAccessibleText
{
public:
	/**
	 * Get the full text contained in this widget, even if some if it is clipped.
	 *
	 * @return All the text held by this widget.
	 */
	virtual const FString& GetText() const = 0;
};

/**
* A widget that represents a table such as list views, tile views or tree views 
* Data about the items that are selected in the table or if selection is supported can be queried.
*/
class IAccessibleTable
{
public: 
	/**
	 * Get all of the elements that are selected in the table. 
	 *
	 * @return All the items selected in this table. Returns an empty array if nothing is selected. 
	 */
	virtual TArray<TSharedPtr<IAccessibleWidget>> GetSelectedItems() const { return TArray < TSharedPtr<IAccessibleWidget>>(); }

	/**
	 * Check if the table can select more than one element at a time. 
	 *
	 * @return True if the table can support selection for more than one element at a time. Else false. 
	 */
	virtual bool CanSupportMultiSelection() const { return false; }

	/**
	 * Checks if the table must have an element selected at all times.
	 *
	 * @return True if an element of the table must be selected at all times, else false
	 */
	virtual bool IsSelectionRequired() const { return false; }
};

/**
* A widget that is an element in an accessible table.
* These widgets can be interacted with by selecting them or querying them for the owning table. 
* NOTE: All accessible table rows must have an accessible table as an owner 
*/
class IAccessibleTableRow
{
public:
	/** Selects this table row in the owning table. */
	virtual void Select() {}

	/** Adds this table row to the list of selected items in the owning table */
	virtual void AddToSelection() {}

	/** Removes this table row from the list of selected items in the owning table */
	virtual void RemoveFromSelection() {}

	/**
	 * Checks if this table row is currently selected in the owning table. 
	 *
	 * @return True if this table row is currently selected in the owning table. Else false 
	 */
	virtual bool IsSelected() const { return false; }

	/**
	 * Get the accessible table that owns this accessible table row. 
	 * NOTE: This should always return a valid pointer in concrete classes. An accessible table row must ALWAYS have an accessible table as an owner. 
	 *
	 * @return The owning table 
	 */
	virtual TSharedPtr<IAccessibleWidget> GetOwningTable() const { return nullptr; }
};

typedef int32 AccessibleWidgetId; 

/**
 * Provides the core set of accessible data that is necessary in order for widget traversal and TTS to be implemented.
 * In order to support functionality beyond this, subclasses must implement the other accessible interfaces and
 * then override the As*() functions.
 */
class IAccessibleWidget : public TSharedFromThis<IAccessibleWidget>
{
public:
	IAccessibleWidget() {}
	virtual ~IAccessibleWidget() = default;

	static const AccessibleWidgetId InvalidAccessibleWidgetId = -1;

	/**
	 * Get an application-unique identifier for this widget. If the widget is destroyed,
	 * a different widget is allowed to re-use that ID.
	 *
	 * @return A unique ID that specifically refers to this widget.
	 */
	virtual AccessibleWidgetId GetId() const = 0;
	/**
	 * Whether or not the underlying widget backing this interface still exists
	 *
	 * @return true if functions can be called on this interface and should return valid results
	 */
	virtual bool IsValid() const = 0;

	/**
	 * Returns the window at the top of this widget's hierarchy. This function may return itself for accessible windows,
	 * and could return nullptr in cases where the widget is not currently part of a hierarchy.
	 *
	 * @return The root window in this widget's widget tree, or nullptr if there is no window.
	 */
	virtual TSharedPtr<IAccessibleWidget> GetWindow() const = 0;
	/**
	 * Retrieving the bounding rect in absolute coordinates for this widget. On some platforms this may be used for hit testing.
	 *
	 * @return The bounds of the widget.
	 */
	virtual FBox2D GetBounds() const = 0;
	/**
	 * Get the accessible parent of this widget. This may be nullptr if this widget is a window, or if the widget is
	 * currently disconnected from the UI tree.
	 *
	 * @return The accessible parent widget of this widget.
	 */
	virtual TSharedPtr<IAccessibleWidget> GetParent() = 0;
	/**
	 * Returns the first instance of an ancestor from a source widget that satisfies a search criteria.
	 * If the passed in source widget does not have any ancestors or if there are no ancestors that satisfy the search criteria, nullptr is returned.
	 * The search criteria can be either a functor or lambda that acts as a predicate. It must return a bool and take in a const TSharedRef<IAccessibleWidget>& as an argument.
	 *
	 * @param Source The accessible widget to start searching from.
	 * @param SearchCriteria A predicate that takes a const TSharedRef<IAccessibleWidget>& as an argument and returns true if the passed in widget satisfies the search criteria. When an accessible widget that satisfies the SearchCriteria is found, it is returned by the function.
	 * @return The first instance of an ancestor from the source widget that satisfies the search criteria.
	 */
	template<typename PredicateType>
	static TSharedPtr<IAccessibleWidget> SearchForAncestorFrom(const TSharedRef<IAccessibleWidget>& Source, PredicateType Predicate)
	{
		TSharedPtr<IAccessibleWidget> Ancestor = Source->GetParent();
		while (Ancestor)
		{
			if (Predicate(Ancestor.ToSharedRef()))
			{
				return Ancestor;
			}
			Ancestor = Ancestor->GetNextSibling();
		}
		return nullptr;
	}
	/**
	 * Retrieves the widget after this one in the parent's list of children. This should return nullptr for the last widget.
	 *
	 * @return The next widget on the same level of the UI hierarchy.
	 */
	virtual TSharedPtr<IAccessibleWidget> GetNextSibling() = 0;
	/**
	 * Returns the first instance of a next sibling from a source widget that satisfies a search criteria.
	 * If the passed in source widget does not have any next siblings or if there are no next siblings that satisfy the search criteria, nullptr is returned.
	 * The search criteria can be either a functor or lambda that acts as a predicate. It must return a bool and take in a const TSharedRef<IAccessibleWidget>& as an argument.
	 *
	 * @param Source The accessible widget to start searching from.
	 * @param SearchCriteria A predicate that takes a const TSharedRef<IAccessibleWidget>& as an argument and returns true if the passed in widget satisfies the search criteria. When an accessible widget that satisfies the SearchCriteria is found, it is returned by the function.
	 * @return The first instance of a next sibling from the source widget that satisfies the search criteria.
	 */
	template<typename PredicateType>
	static TSharedPtr<IAccessibleWidget> SearchForNextSiblingFrom(const TSharedRef<IAccessibleWidget>& Source, PredicateType Predicate)
	{
		TSharedPtr<IAccessibleWidget> NextSibling = Source->GetNextSibling();
		while (NextSibling)
		{
			if (Predicate(NextSibling.ToSharedRef()))
			{
				return NextSibling;
			}
			NextSibling = NextSibling->GetNextSibling();
		}
		return nullptr;
	}
	/**
	 * Retrieves the widget before this one in the parent's list of children. This should return nullptr for the first widget.
	 *
	 * @return The previous widget on the same level of the UI hierarchy.
	 */
	virtual TSharedPtr<IAccessibleWidget> GetPreviousSibling() = 0;
	/**
	 * Returns the first instance of a previous sibling from a source widget that satisfies a search criteria.
	 * If the passed in source widget does not have any previous siblings or if there are no previous siblings that satisfy the search criteria, nullptr is returned.
	 * The search criteria can be either a functor or lambda that acts as a predicate. It must return a bool and take in a const TSharedRef<IAccessibleWidget>& as an argument.
	 *
	 * @param Source The accessible widget to start searching from.
	 * @param SearchCriteria A predicate that takes a const TSharedRef<IAccessibleWidget>& as an argument and returns true if the passed in widget satisfies the search criteria. When an accessible widget that satisfies the SearchCriteria is found, it is returned by the function.
	 * @return The first instance of a previous sibling from the source widget that satisfies the search criteria.
	 */
	template<typename PredicateType>
	static TSharedPtr<IAccessibleWidget> SearchForPreviousSiblingFrom(const TSharedRef<IAccessibleWidget>& Source, PredicateType Predicate)
	{
		TSharedPtr<IAccessibleWidget> PreviousSibling = Source->GetPreviousSibling();
		while(PreviousSibling)
		{
			if (Predicate(PreviousSibling.ToSharedRef()))
			{
				return PreviousSibling;
			}
			PreviousSibling = PreviousSibling->GetPreviousSibling();
		}
		return nullptr;
	}
	/**
	 * Retrieves the logical next widget in the accessible widget hierarchy from this widget.
	 * This is conceptually similar to tab navigation with the keyboard.
	 * This is primarily used for mobile devices to simulate the right swipe gesture for IOS Voiceover or Android Talkback.
	 * An example algorithm to find the next widget in the accessible hierarchy is as follows:
	 * 1. If the current widget has children, return the first child.
	 * 2. If the current widget has no children, return the sibling of the current widget.
	 * 3. If the current widget has no next sibling, search for the first ancestor from the current widget with a next sibling. Return that ancestor's next sibling.
	 *
	 * @return The logical next widget in the accessible widget hierarchy.
	 */
	virtual TSharedPtr<IAccessibleWidget> GetNextWidgetInHierarchy() = 0;
	/**
	 * Returns the first instance of a next widget in the accessible hierarchy from a source widget that satisfies a search criteria.
	 * If the passed in source widget does not have any next widgets or if there are no next widgets that satisfy the search criteria, nullptr is returned.
	 * The search criteria can be either a functor or lambda that acts as a predicate. It must return a bool and take in a const TSharedRef<IAccessibleWidget>& as an argument.
	 *
	 * @param Source The accessible widget to start searching from.
	 * @param SearchCriteria A predicate that takes a const TSharedRef<IAccessibleWidget>& as an argument and returns true if the passed in widget satisfies the search criteria. When an accessible widget that satisfies the SearchCriteria is found, it is returned by the function.
	 * @return The first instance of a next widget from the source widget that satisfies the search criteria.
	 */
	template<typename PredicateType>
	static TSharedPtr<IAccessibleWidget> SearchForNextWidgetInHierarchyFrom(const TSharedRef<IAccessibleWidget>& Source, PredicateType Predicate)
	{
		TSharedPtr<IAccessibleWidget> NextWidget = Source->GetNextWidgetInHierarchy();
		while(NextWidget)
		{
			if (Predicate(NextWidget.ToSharedRef()))
			{
				return NextWidget;
			}
			NextWidget = NextWidget->GetNextWidgetInHierarchy();
		}
		return nullptr;
	}
	/**
	 * Retrieves the logical previous widget in the accessible widget hierarchy from this current widget.
	 * This is conceptually similar to shift tab navigation with the keyboard.
	 * This is primarily used for mobile devices to simulate the left swipe gesture for IOS Voiceover or Android Talkback.
	 * An example algorithm to find the previous widget in the hierarchy is as follows:
	 * 1. Find the previous sibling of the current widget and check if it has children.
	 * 2. If the previous sibling has children, we will call the last child of the previous sibling C. Recursively take the last child from C until a leaf is found. Return the leaf.
	 * 3. If the previous sibling has no children, return the previous sibling.
	 * 4. If the current widget has no previous sibling, we return the parent of the current widget.
	 *
	 * @return The logical previous widget in the accessible widget hierarchy.
	 */
	virtual TSharedPtr<IAccessibleWidget> GetPreviousWidgetInHierarchy() = 0;
	/**
	 * Returns the first instance of a previous widget in the accessible hierarchy from a source widget that satisfies a search criteria.
	 * If the passed in source widget does not have any previous widgets or if there are no previous widgets that satisfy the search criteria, nullptr is returned.
	 * The search criteria can be either a functor or lambda that acts as a predicate. It must return a bool and take in a const TSharedRef<IAccessibleWidget>& as an argument.
	 *
	 * @param Source The accessible widget to start searching from.
	 * @param SearchCriteria A predicate that takes a const TSharedRef<IAccessibleWidget>& as an argument and returns true if the passed in widget satisfies the search criteria. When an accessible widget that satisfies the SearchCriteria is found, it is returned by the function.
	 * @return The first instance of a previous widget from the source widget that satisfies the search criteria.
	 */
	template<typename PredicateType>
	static TSharedPtr<IAccessibleWidget> SearchForPreviousWidgetInHierarchyFrom(const TSharedRef<IAccessibleWidget>& Source, PredicateType Predicate)
	{
		TSharedPtr<IAccessibleWidget> PreviousWidget = Source->GetPreviousWidgetInHierarchy();
		while(PreviousWidget)
		{
			if (Predicate(PreviousWidget.ToSharedRef()))
			{
				return PreviousWidget;
			}
			PreviousWidget = PreviousWidget->GetPreviousWidgetInHierarchy();
		}
		return nullptr;
	}
	/**
	 * Retrieves the accessible child widget at a certain index. This should return nullptr if Index < 0 or Index >= GetNumberOfChildren().
	 *
	 * @param The index of the child widget to get
	 * @return The accessible child widget at the specified index.
	 */
	virtual TSharedPtr<IAccessibleWidget> GetChildAt(int32 Index) = 0;
	/**
	 * How many accessible children this widget has.
	 *
	 * @return The number of accessible children that exist for this widget.
	 */
	virtual int32 GetNumberOfChildren() = 0;
	/**
	 * Returns the first instance of a child from a source widget that satisfies a search criteria.
	 * If the passed in source widget does not have any children or if there are no children that satisfy the search criteria, nullptr is returned.
	 * The search criteria can be either a functor or lambda that acts as a predicate. It must return a bool and take in a const TSharedRef<IAccessibleWidget>& as an argument.
	 *
	 * @param Source The accessible widget to start searching from.
	 * @param SearchCriteria A predicate that takes a const TSharedRef<IAccessibleWidget>& as an argument and returns true if the passed in widget satisfies the search criteria. When an accessible widget that satisfies the SearchCriteria is found, it is returned by the function.
	 * @return The first child from the source widget that satisfies the search criteria.
	 */
	template<typename PredicateType>
	static TSharedPtr<IAccessibleWidget> SearchForFirstChildFrom(const TSharedRef<IAccessibleWidget>& Source, PredicateType Predicate)
	{
		if (Source->GetNumberOfChildren() == 0)
		{
			return nullptr;
		}
		TSharedPtr<IAccessibleWidget> Child = nullptr;
		int32 NumChildren = Source->GetNumberOfChildren();
		for (int32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			Child = Source->GetChildAt(ChildIndex);
			if (Child && Predicate(Child.ToSharedRef()))
			{
				return Child;
			}
		}
		return nullptr;
	}
	/**
	 * What type of accessible widget the underlying widget should be treated as. A widget may be capable of presenting itself
	 * as multiple different types of widgets, but only one can be reported back to the platform.
	 *
	 * @return Which type of widget the platform layer should treat this as.
	 */
	virtual EAccessibleWidgetType GetWidgetType() const = 0;

	/**
	 * The name of the underlying class that this accessible widget represents.
	 *
	 * @return The class name of the underlying widget.
	 */
	virtual FString GetClassName() const = 0;
	/**
	 * The name of the widget to report to the platform layer. For screen readers, this is often the text that will be spoken.
	 *
	 * @return Ideally, a human-readable name that represents what the widget does.
	 */
	virtual FString GetWidgetName() const = 0;
	/**
	 * Additional information a user may need in order to effectively interact or use the widget, such as a tooltip.
	 *
	 * @return A more-detailed description of what the widget is or how its used.
	 */
	virtual FString GetHelpText() const = 0;
	/** Returns a string representation of this widget. Primarily used for debugging. */
	virtual FString ToString() const
	{
		TStringBuilder<256> Builder;
		Builder.Appendf(TEXT("Label: %s. Role: %s."), *GetWidgetName(), *GetClassName());
		return Builder.ToString();
	}

	/**
	 * Whether the widget is enabled and can be interacted with.
	 *
	 * @return true if the widget is enabled.
	 */
	virtual bool IsEnabled() const = 0;
	/**
	 * Whether the widget is being rendered on screen or not.
	 *
	 * @return true if the widget is hidden off screen, collapsed, or something similar.
	 */
	virtual bool IsHidden() const = 0;
	/**
	 * Whether the widget can ever support keyboard/gamepad focus.
	 *
	 * @return true if the widget can ever receive keyboard/gamepad focus. Else, returns false.
	 */
	virtual bool SupportsFocus() const = 0;
	/**
	 * Whether the widget can ever support accessible focus.
	 *
	 * @return true if the widget can ever receive accessible focus. Else, returns false.
	 */
	virtual bool SupportsAccessibleFocus() const = 0;
	/**
	 * Whether the widget can currently support accessible focus.
	 *
	 * @return true if the widget can currently receive accessible focus. Else, returns false.
	 */
	virtual bool CanCurrentlyAcceptAccessibleFocus() const = 0;
	/**
	 * Whether the widget has accessible focus or not by a particular user .
	 *
	 * @param UserIndex The user index associated with the accessible user to query if this accessible widget is currently being focused.
	 * @return true if the widget currently has accessible focus by the user. Else, returns false.
	 */
	virtual bool HasUserFocus(const FAccessibleUserIndex UserIndex) const = 0;
	/**
	 * Assign accessible focus to the widget. Also sets keyboard/gamepad focus if the widget supports it.
	*
	 * @param UserIndex The user index associated with the accessible user that requested focus for this accessible widget.
	 * @return True if focus was successfully set to this widget for the requested user. Else false.
	*/
	virtual bool SetUserFocus(const FAccessibleUserIndex UserIndex) = 0;

	/**
	 * Attempt to cast this to an IAccessibleWindow
	 *
	 * @return 'this' as an IAccessibleWindow if possible, otherwise nullptr
	 */
	virtual IAccessibleWindow* AsWindow() { return nullptr; }
	/**
	 * Attempt to cast this to an IAccessibleActivatable
	 *
	 * @return 'this' as an IAccessibleActivatable if possible, otherwise nullptr
	 */
	virtual IAccessibleActivatable* AsActivatable() { return nullptr; }
	/**
	 * Attempt to cast this to an IAccessibleProperty
	 *
	 * @return 'this' as an IAccessibleProperty if possible, otherwise nullptr
	 */
	virtual IAccessibleProperty* AsProperty() { return nullptr; }
	/**
	 * Attempt to cast this to an IAccessibleText
	 *
	 * @return 'this' as an IAccessibleText if possible, otherwise nullptr
	 */
	virtual IAccessibleText* AsText() { return nullptr; }

	/**
	 * Attempt to cast this to an IAccessibleTable
	 *
	 * @return 'this' as an IAccessibleTable if possible, otherwise nullptr
	 */
	virtual IAccessibleTable* AsTable() { return nullptr; }

	/**
	 * Attempt to cast this to an IAccessibleTableRow
	 *
	 * @return 'this' as an IAccessibleTableRow if possible, otherwise nullptr
	 */
	virtual IAccessibleTableRow* AsTableRow() { return nullptr; }
};

/**
 * The arguments for an accessible event that is raised by accessible widgets to be
 * passed to an accessibility event handler such as a native OS.
 * It is up to the client to create an instance of this struct and fill in the appropriate data members.
 * @see FGenericAccessibleMessageHandler::RaiseEvent
 */
struct FAccessibleEventArgs
{
	FAccessibleEventArgs(TSharedRef<IAccessibleWidget> InWidget, EAccessibleEvent InEvent, FVariant InOldValue = FVariant(), FVariant InNewValue = FVariant(), FAccessibleUserIndex InUserIndex = 0)
		: Widget(InWidget)
		, Event(InEvent)
		, OldValue(InOldValue)
		, NewValue(InNewValue)
		, UserIndex(InUserIndex)
	{}
	
	/** The accessible widget that generated the event */
	TSharedRef<IAccessibleWidget> Widget;
	/** The type of event generated */
	EAccessibleEvent Event;
	/** If this was a property changed event, the 'before' value */
	FVariant OldValue;
	/** If this was a property changed event, the 'after' value. This may also be set for other events such as Notification. */
	FVariant NewValue;
	/** The Id of the user this event is intended for. Think of a hardware device such as a controller or keyboard/mouse. */
	FAccessibleUserIndex UserIndex;
};

/**
 * Platform and application-agnostic messaging system for accessible events. The message handler
 * lives in GenericApplication and any subclass that wishes to support accessibility should subclass
 * this and use GenericAppliation::SetAccessibleMessageHandler to enable functionality.
 *
 * GetAccessibleWindow() is the entry point to all accessible widgets. Once the window is retrieved, it
 * can be queried for children in various ways. RaiseEvent() allows messages to bubble back up to the
 * native OS through anything bound to the AccessibleEventDelegate.
 *
 * Callers can use ApplicationIsAccessible() to see if accessibility is supported or not. Alternatively,
 * calling GetAccessibleWindow and seeing if the result is valid should provide the same information.
 *
 * Callers should also use GetAccessibleUserRegistry() to register and interact with accessible users. Accessible users must be registered
 * with the set accessible user registry for the rest of the application to be able to retrieve and interact with the accessible users.
 */
class FGenericAccessibleMessageHandler
{
public:
	/**
	 * A delegate accessible event handlers such as platform accessibility APIs can
	 * listen for accessibility events raised by widgets.  .
	 */
	DECLARE_DELEGATE_OneParam(FAccessibleEvent, const FAccessibleEventArgs&);

	APPLICATIONCORE_API FGenericAccessibleMessageHandler();
	virtual ~FGenericAccessibleMessageHandler()
	{
		UnbindAccessibleEventDelegate();
	}

	/**
	 * Subclasses should return true to indicate that they support accessibility.
	 *
	 * @return true if the application intends to return valid accessible widgets when queried.
	 */
	APPLICATIONCORE_API bool ApplicationIsAccessible() const;

	/**
	 * Checks if accessibility is enabled in the application. Usually this happens when screen-reading software is turned on.
	 * Note: On some platforms, there is no way to deactivate this after enabling it. 
	 *
	 * @return The last value SetActive() was called with.
	 */
	bool IsActive() const { return bIsActive; }

	/**
	 * Notify the application to start or stop processing accessible messages from the platform layer.
	 *
	 * @param bActive Whether to enable to disable the message handler.
	 */
	APPLICATIONCORE_API void SetActive(bool bActive);

	/**
	 * Creates or retrieves an accessible object for a native OS window.
	 * todo: Behavior for non-native windows (virtual or others) is currently undefined.
	 *
	 * @param InWindow The native window to find the accessible window for
	 * @return The accessible object corresponding to the supplied native window
	 */
	virtual TSharedPtr<IAccessibleWidget> GetAccessibleWindow(const TSharedRef<FGenericWindow>& InWindow) const { return nullptr; }

	/**
	 * Creates or retrieves the identifier for an accessible object for a native OS window.
	 * todo: Behavior for non-native windows (virtual or others) is currently undefined.
	 *
	 * @param InWindow The native window to find the accessible window for
	 * @return The identifier for the accessible window created
	 */
	virtual AccessibleWidgetId GetAccessibleWindowId(const TSharedRef<FGenericWindow>& InWindow) const { return IAccessibleWidget::InvalidAccessibleWidgetId; }

	/**
	 * Retrieves an accessible widget that matches the given identifier.
	 *
	 * @param Id The identifier for the widget to get.
	 * @return The widget that matches this identifier, or nullptr if the widget does not exist.
	 */
	virtual TSharedPtr<IAccessibleWidget> GetAccessibleWidgetFromId(AccessibleWidgetId Id) const { return nullptr; }

	/**
	 * Push an event from an accessible widget back to the platform layer.
	 *
	 * @param Args The arguments to be passed to the accessible event delegate.
	 * @see FAccessibleEventArgs
	 */
	void RaiseEvent(const FAccessibleEventArgs& Args)
	{
		AccessibleEventDelegate.ExecuteIfBound(Args);
	}

	/**
	 * Assign a function to be called whenever an accessible event is raised.
	 *
	 * @param Delegate The delegate to execute when an event is raised.
	 */
	void SetAccessibleEventDelegate(const FAccessibleEvent& Delegate)
	{
		AccessibleEventDelegate = Delegate;
	}

	/**
 * Unbinds the delegate called during accessible events if it is bound. 
 */
	void UnbindAccessibleEventDelegate()
	{
		AccessibleEventDelegate.Unbind();
	}

	/**
	 * Request a function to be run in a particular thread. 
	 * Primarily used to service accessibility requests from the OS thread about accessible Slate widget data in the game thread. 
	 *
	 * @param Function The function to execute in the game thread.
	 * @bWaitForCompletion If true, the function will block until the requested Function completes executing on the requested Thread
	 * @param Thread The thread to run the Function in.
	 */
	virtual void RunInThread(const TFunction<void()>& Function, bool bWaitForCompletion = true, ENamedThreads::Type Thread = ENamedThreads::GameThread) { }

	/**
	 * Request a string to be announced to the user.
	 * Note: Currently only supported on Win10, Mac, iOS. If requests are made too close together,
	 * earlier announcement requests will get stomped by later announcements on certain platforms.
	 */
	virtual void MakeAccessibleAnnouncement(const FString& AnnouncementString) { }

	/**
	 * Retrieves the accessible user registry. This is the means of retrieving, registering and interacting with accessible useres in the rest of the application.
	 */
	FGenericAccessibleUserRegistry& GetAccessibleUserRegistry()
	{
		return *AccessibleUserRegistry;
	}
	/**
	 * Retrieves the accessible user registry. This is the means of retrieving, registering and interacting with accessible useres in the rest of the application.
	 */
	const FGenericAccessibleUserRegistry& GetAccessibleUserRegistry() const
	{
		return *AccessibleUserRegistry;
	}
	/**
	 * Sets a new accessible user registry for the application.
	 * Note: As of now, none of the registered users will carry over to the new accessible user manager. It is up to the caller to unregister all currently registered users and
	 * register them with the new accessible user registry.   .
	 */
	void SetAccessibleUserRegistry(const TSharedRef<FGenericAccessibleUserRegistry>& InAccessibleUserRegistry)
	{
		// @TODOAccessibility: Have some means of storing a default manager
		AccessibleUserRegistry = InAccessibleUserRegistry;
	}
protected:
	/** Triggered when bIsActive changes from false to true. */
	virtual void OnActivate() {}
	/** Triggered when bIsActive changes from true to false. */
	virtual void OnDeactivate() {}
	
	/** Subclasses should override this to indicate that they support accessibility. */
	bool bApplicationIsAccessible;

private:
	/** Whether or not accessibility is currently enabled in the application */
	bool bIsActive;
	/** Delegate for the platform layer to listen to widget events */
	FAccessibleEvent AccessibleEventDelegate;
	/**
	 * Registry for all accessible users in the application.
	 * Accessible users must be registered with the accessible user registry to allow other parts of the application to retrieve and interact with the accessible users.
	 */
	TSharedRef<FGenericAccessibleUserRegistry> AccessibleUserRegistry;
};

#endif
