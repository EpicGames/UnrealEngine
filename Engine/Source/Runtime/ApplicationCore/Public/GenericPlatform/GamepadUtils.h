// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"
#include "CoreTypes.h"

/**
 * Properties and functions for a trigger's Dyanamic Release DeadZone.
 */
struct FDynamicReleaseDeadZone
{
public:
	FDynamicReleaseDeadZone()
		: DeadZone(1.0f)
		, TriggerThreshold(0)
		, bHasOverride(false)
		, bWasSimplePressed(false)
		, bWasDynamicPressed(false)
	{
		RefreshSettings();
	}

	APPLICATIONCORE_API void RefreshSettings();

	APPLICATIONCORE_API void OverrideDeadZone(const float InDeadZone);

	bool IsDynamicReleaseEnabled() const;

	APPLICATIONCORE_API bool IsPressed(const uint8 TriggerAnalog, const uint8 PreviousTriggerAnalog);

	APPLICATIONCORE_API bool IsPressed(const uint8 TriggerAnalog, const uint8 PreviousTriggerAnalog, const bool bIsSimplePressed, const bool bPreviousSimplePressed);

	APPLICATIONCORE_API bool IsPressed(const uint8 TriggerAnalog, const bool bIsSimplePressed);

protected:
	float DeadZone = 1.0f;
	uint8 TriggerThreshold = 0;
	uint8 bHasOverride : 1;
	uint8 bWasSimplePressed : 1;
	uint8 bWasDynamicPressed : 1;
};
