// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "Math/IntPoint.h"
#include "Math/MathFwd.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#include "Templates/SharedPointer.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "GenericPlatform/GenericWindow.h"
#include "GenericPlatform/GenericWindowDefinition.h"

class FGenericAccessibleMessageHandler;
class FSlateApplication;
class IAnalyticsProvider;
class ICursor;
class IInputInterface;
class ITextInputMethodSystem;

/**
* Enumerates available modifier keys for input gestures.
*/
namespace EModifierKey
{
	typedef uint8 Type;

	/** No key. */
	inline const Type None	= 0;

	/** Ctrl key (Command key on Mac, Control key on Windows). */
	inline const Type Control = 1 << 0;

	/** Alt key. */
	inline const Type Alt = 1 << 1;

	/** Shift key. */
	inline const Type Shift = 1 << 2;

	/** Cmd key (Control key on Mac, Win key on Windows) */
	inline const Type Command = 1 << 3;

	inline EModifierKey::Type FromBools(const bool bControl, const bool bAlt, const bool bShift, const bool bCommand)
	{
		EModifierKey::Type ModifierMask = EModifierKey::None;
		if (bControl)	ModifierMask |= EModifierKey::Control;
		if (bAlt)		ModifierMask |= EModifierKey::Alt;
		if (bShift)		ModifierMask |= EModifierKey::Shift;
		if (bCommand)	ModifierMask |= EModifierKey::Command;

		return ModifierMask;
	}
};


namespace EPopUpOrientation
{
	enum Type
	{
		Horizontal,
		Vertical,
	};
}


/**
 * FModifierKeysState stores the pressed state of keys that are commonly used as modifiers
 */
class FModifierKeysState
{

public:

	/**
	 * Constructor.  Events are immutable once constructed.
	 *
	 * @param  bInIsLeftShiftDown  True if left shift is pressed
	 * @param  bInIsRightShiftDown  True if right shift is pressed
	 * @param  bInIsLeftControlDown  True if left control is pressed
	 * @param  bInIsRightControlDown  True if right control is pressed
	 * @param  bInIsLeftAltDown  True if left alt is pressed
	 * @param  bInIsRightAltDown  True if right alt is pressed
	 */
	FModifierKeysState( const bool bInIsLeftShiftDown,
						const bool bInIsRightShiftDown,
						const bool bInIsLeftControlDown,
						const bool bInIsRightControlDown,
						const bool bInIsLeftAltDown,
						const bool bInIsRightAltDown,
						const bool bInIsLeftCommandDown,
						const bool bInIsRightCommandDown,
						const bool bInAreCapsLocked)
		: bIsLeftShiftDown( bInIsLeftShiftDown ),
		  bIsRightShiftDown( bInIsRightShiftDown ),
		  bIsLeftControlDown( bInIsLeftControlDown ),
		  bIsRightControlDown( bInIsRightControlDown ),
		  bIsLeftAltDown( bInIsLeftAltDown ),
		  bIsRightAltDown( bInIsRightAltDown ),
		  bIsLeftCommandDown( bInIsLeftCommandDown ),
		  bIsRightCommandDown( bInIsRightCommandDown ),
		  bAreCapsLocked( bInAreCapsLocked )
	{
	}

	FModifierKeysState()
		: bIsLeftShiftDown(false),
		  bIsRightShiftDown(false),
		  bIsLeftControlDown(false),
		  bIsRightControlDown(false),
		  bIsLeftAltDown(false),
		  bIsRightAltDown(false),
		  bIsLeftCommandDown(false),
		  bIsRightCommandDown(false),
		  bAreCapsLocked(false)
	{
	}
	
	/**
	 * Returns true if either shift key was down when this event occurred
	 *
	 * @return  True if shift is pressed
	 */
	bool IsShiftDown() const
	{
		return bIsLeftShiftDown || bIsRightShiftDown;
	}

	/**
	 * Returns true if left shift key was down when this event occurred
	 *
	 * @return  True if left shift is pressed
	 */
	bool IsLeftShiftDown() const
	{
		return bIsLeftShiftDown;
	}

	/**
	 * Returns true if right shift key was down when this event occurred
	 *
	 * @return  True if right shift is pressed
	 */
	bool IsRightShiftDown() const
	{
		return bIsRightShiftDown;
	}

	/**
	 * Returns true if either control key was down when this event occurred
	 *
	 * @return  True if control is pressed
	 */
	bool IsControlDown() const
	{
		return bIsLeftControlDown || bIsRightControlDown;
	}

	/**
	 * Returns true if left control key was down when this event occurred
	 *
	 * @return  True if left control is pressed
	 */
	bool IsLeftControlDown() const
	{
		return bIsLeftControlDown;
	}

	/**
	 * Returns true if right control key was down when this event occurred
	 *
	 * @return  True if right control is pressed
	 */
	bool IsRightControlDown() const
	{
		return bIsRightControlDown;
	}

	/**
	 * Returns true if either alt key was down when this event occurred
	 *
	 * @return  True if alt is pressed
	 */
	bool IsAltDown() const
	{
		return bIsLeftAltDown || bIsRightAltDown;
	}

	/**
	 * Returns true if left alt key was down when this event occurred
	 *
	 * @return  True if left alt is pressed
	 */
	bool IsLeftAltDown() const
	{
		return bIsLeftAltDown;
	}

	/**
	 * Returns true if right alt key was down when this event occurred
	 *
	 * @return  True if right alt is pressed
	 */
	bool IsRightAltDown() const
	{
		return bIsRightAltDown;
	}
	
	/**
	 * Returns true if either command key was down when this event occurred
	 *
	 * @return  True if command is pressed
	 */
	bool IsCommandDown() const
	{
		return bIsLeftCommandDown || bIsRightCommandDown;
	}
	
	/**
	 * Returns true if left command key was down when this event occurred
	 *
	 * @return  True if left command is pressed
	 */
	bool IsLeftCommandDown() const
	{
		return bIsLeftCommandDown;
	}
	
	/**
	 * Returns true if right command key was down when this event occurred
	 *
	 * @return  True if right command is pressed
	 */
	bool IsRightCommandDown() const
	{
		return bIsRightCommandDown;
	}


	/**
	 * @return  true if the Caps Lock key has been toggled to the enabled state.
	 */
	bool AreCapsLocked() const
	{
		return bAreCapsLocked;
	}

	/**
	 * @param ModifierKeys the modifier keys to test to see if they are pressed.  Returns true if no modifiers are specified.
	 * @return true if all modifier keys are pressed specified in the modifier keys.
	 */
	bool AreModifersDown(EModifierKey::Type ModiferKeys) const
	{
		bool AllModifersDown = true;

		if ( (ModiferKeys & EModifierKey::Shift) == EModifierKey::Shift )
		{
			AllModifersDown &= IsShiftDown();
		}
		if ( (ModiferKeys & EModifierKey::Command) == EModifierKey::Command )
		{
			AllModifersDown &= IsCommandDown();
		}
		if ( (ModiferKeys & EModifierKey::Control) == EModifierKey::Control )
		{
			AllModifersDown &= IsControlDown();
		}
		if ( (ModiferKeys & EModifierKey::Alt) == EModifierKey::Alt )
		{
			AllModifersDown &= IsAltDown();
		}

		return AllModifersDown;
	}

	/**
	 * @return true if any modifier key is down
	 */
	bool AnyModifiersDown() const
	{
		return IsControlDown() || IsCommandDown() || IsShiftDown() || IsAltDown();
	}

private:

	/** True if the left shift key was down when this event occurred. */
	uint16 bIsLeftShiftDown:1;

	/** True if the right shift key was down when this event occurred. */
	uint16 bIsRightShiftDown:1;

	/** True if the left control key was down when this event occurred. */
	uint16 bIsLeftControlDown:1;

	/** True if the right control key was down when this event occurred. */
	uint16 bIsRightControlDown:1;

	/** True if the left alt key was down when this event occurred. */
	uint16 bIsLeftAltDown:1;

	/** True if the right alt key was down when this event occurred. */
	uint16 bIsRightAltDown:1;
	
	/** True if the left command key was down when this event occurred. */
	uint16 bIsLeftCommandDown:1;
	
	/** True if the right command key was down when this event occurred. */
	uint16 bIsRightCommandDown:1;

	/** True if the Caps Lock key has been toggled to the enabled state. */
	uint16 bAreCapsLocked:1;
};


struct FPlatformRect
{
	int32 Left;
	int32 Top;
	int32 Right;
	int32 Bottom;

	FPlatformRect() : Left(0), Top(0), Right(0), Bottom(0) {}
	FPlatformRect(int32 InLeft, int32 InTop, int32 InRight, int32 InBottom) : Left(InLeft), Top(InTop), Right(InRight), Bottom(InBottom) {}

	bool operator==(const FPlatformRect& Other) const
	{
		return Left == Other.Left && Top == Other.Top && Right == Other.Right && Bottom == Other.Bottom;
	}
};


/**
 * Contains info on a physical monitor connected to the display device
 */
struct FMonitorInfo
{
	/** Legacy name that may not be as friendly as FriendlyName for surfacing to UI. */
	FString Name;

	/** Friendly name for surfacing to UI. */
	FString FriendlyName;

	/** String used for monitor identification, which is less likely to alias displays than Name or FriendlyName. */
	FString ID;

	int32 NativeWidth = 0;
	int32 NativeHeight = 0;
	FIntPoint MaxResolution = FIntPoint(ForceInitToZero);
	FPlatformRect DisplayRect;
	FPlatformRect WorkArea;
	bool bIsPrimary = false;
	int32 DPI = 0;
	void* NativeHandle = nullptr;
};


/**
 * Contains metrics information for the desktop area
 */
struct FDisplayMetrics
{
	FDisplayMetrics()
		: TitleSafePaddingSize(0, 0, 0, 0)
		, ActionSafePaddingSize(0, 0, 0, 0)
	{
	}

	/** Width of the primary display in pixels */
	int32 PrimaryDisplayWidth;

	/** Height of the primary display in pixels */
	int32 PrimaryDisplayHeight;

	/** Info on connected physical monitors. Only available on platforms where this information is accessible (PC currently) */
	TArray<FMonitorInfo> MonitorInfo;

	/** Area of the primary display not covered by task bars or other docked widgets */
	FPlatformRect PrimaryDisplayWorkAreaRect;

	/** Virtual display coordinate range (includes all active displays) */
	FPlatformRect VirtualDisplayRect;

	/**
	 * The safe area for all content on TVs (see http://en.wikipedia.org/wiki/Safe_area_%28television%29) - content will be inset 
	 * Left - X
	 * Top - Y
	 * Right - Z
	 * Bottom - W
	 */
	FVector4 TitleSafePaddingSize;

	/** The safe area for less important spill over on TVs (see TitleSafePaddingSize) */
	FVector4 ActionSafePaddingSize;

	APPLICATIONCORE_API static void RebuildDisplayMetrics(struct FDisplayMetrics& OutDisplayMetrics);

	/** Gets the monitor work area from a position in the global display rect */
	APPLICATIONCORE_API FPlatformRect GetMonitorWorkAreaFromPoint(const FVector2D& Point) const;

	/** Gets the closest monitor from monitor id and index favoring id */
	APPLICATIONCORE_API int32 GetClosestMonitorFromIDAndIndex(const FString& MonitorID, int32 MonitorIndex) const;

	/** Logs out display metrics */
	APPLICATIONCORE_API void PrintToLog() const;

	// The title safe zone ratio that will be returned by FDisplayMetrics::GetDisplayMetrics on platforms that don't have a defined safe zone
	APPLICATIONCORE_API static float GetDebugTitleSafeZoneRatio();
protected:
	// Fills the InOutTitleSafePaddingSize with CVar overwrites (Sets padding to 0 for values that are not set).
	bool TryGetTitleSafeZoneOverwrite(FVector4& InOutTitleSafePaddingSize);

	// The action safe zone ratio that will be returned by FDisplayMetrics::GetDisplayMetrics on platforms that don't have a defined safe zone
	static float GetDebugActionSafeZoneRatio();

	// Apply the debug/default safe zones
	void ApplyDefaultSafeZones();
};


/**
 * Enumerates available horizontal alignments for window title bars.
 */
namespace EWindowTitleAlignment
{
	enum Type
	{
		/** The window title should be aligned to the left. */
		Left,
		/** The window title should be centered within the title bar. */
		Center,
		/** The window title should be aligned to the right. */
		Right
	};
}


/**
 * Generic platform application interface
 */
class GenericApplication 
{
public:

	DECLARE_MULTICAST_DELEGATE_OneParam( FOnConsoleCommandAdded, const FString& /*Command*/ );
	typedef FOnConsoleCommandAdded::FDelegate FOnConsoleCommandListener;

	APPLICATIONCORE_API GenericApplication(const TSharedPtr< ICursor >& InCursor);
	APPLICATIONCORE_API virtual ~GenericApplication();

	virtual void SetMessageHandler( const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler ) { MessageHandler = InMessageHandler; }

	TSharedRef< FGenericApplicationMessageHandler > GetMessageHandler() { return MessageHandler; }

#if WITH_ACCESSIBILITY
	APPLICATIONCORE_API virtual void SetAccessibleMessageHandler(const TSharedRef<FGenericAccessibleMessageHandler>& InAccessibleMessageHandler);
	APPLICATIONCORE_API TSharedRef<FGenericAccessibleMessageHandler> GetAccessibleMessageHandler() const;
#endif

	virtual void PollGameDeviceState( const float TimeDelta ) { }

	virtual void PumpMessages( const float TimeDelta ) { }

	virtual void ProcessDeferredEvents( const float TimeDelta ) { }

	virtual void Tick ( const float TimeDelta ) { }

	virtual TSharedRef< FGenericWindow > MakeWindow() { return MakeShareable( new FGenericWindow() ); }

	virtual void InitializeWindow( const TSharedRef< FGenericWindow >& Window, const TSharedRef< FGenericWindowDefinition >& InDefinition, const TSharedPtr< FGenericWindow >& InParent, const bool bShowImmediately ) { }

	virtual void SetCapture( const TSharedPtr< FGenericWindow >& InWindow ) { }

	virtual void* GetCapture( void ) const { return NULL; }

	virtual FModifierKeysState GetModifierKeys() const  { return FModifierKeysState(); }

	/** @return true if the system cursor is currently directly over a slate window. */
	virtual bool IsCursorDirectlyOverSlateWindow() const { return true; }

	/** @return Native window under the mouse cursor. */
	virtual TSharedPtr< FGenericWindow > GetWindowUnderCursor() { return TSharedPtr< FGenericWindow >( nullptr ); }

	virtual bool IsMinimized() const { return false; }

	virtual void SetHighPrecisionMouseMode( const bool Enable, const TSharedPtr< FGenericWindow >& InWindow ) { };

	virtual bool IsUsingHighPrecisionMouseMode() const { return false; }
	
	virtual bool IsUsingTrackpad() const { return false; }

	virtual bool IsMouseAttached() const { return true; }

	virtual bool IsGamepadAttached() const { return false; }

	virtual void RegisterConsoleCommandListener(const FOnConsoleCommandListener& InListener) {}

	virtual void AddPendingConsoleCommand(const FString& InCommand) {}

	virtual FPlatformRect GetWorkArea( const FPlatformRect& CurrentWindow ) const
	{
		FPlatformRect OutRect;
		OutRect.Left = 0;
		OutRect.Top = 0;
		OutRect.Right = 0;
		OutRect.Bottom = 0;

		return OutRect;
	}

	virtual bool TryCalculatePopupWindowPosition( const FPlatformRect& InAnchor, const FVector2D& InSize, const FVector2D& ProposedPlacement, const EPopUpOrientation::Type Orientation, /*OUT*/ FVector2D* const CalculatedPopUpPosition ) const { return false; }

	DECLARE_EVENT_OneParam(GenericApplication, FOnDisplayMetricsChanged, const FDisplayMetrics&);
	
	/** Notifies subscribers when any of the display metrics change: e.g. resolution changes or monitor sare re-arranged. */
	FOnDisplayMetricsChanged& OnDisplayMetricsChanged(){ return OnDisplayMetricsChangedEvent; }

	virtual void GetInitialDisplayMetrics( FDisplayMetrics& OutDisplayMetrics ) const { FDisplayMetrics::RebuildDisplayMetrics(OutDisplayMetrics); }

	
	/** Delegate for virtual keyboard being shown/hidden in case UI wants to slide out of the way */
	DECLARE_EVENT_OneParam(FSlateApplication, FVirtualKeyboardShownEvent, FPlatformRect);
	FVirtualKeyboardShownEvent& OnVirtualKeyboardShown()  { return VirtualKeyboardShownEvent; }
	
	DECLARE_EVENT(FSlateApplication, FVirtualKeyboardHiddenEvent);
	FVirtualKeyboardHiddenEvent& OnVirtualKeyboardHidden()  { return VirtualKeyboardHiddenEvent; }
	
	DECLARE_EVENT(FSlateApplication, FOnClipboardContentChanged);
	FOnClipboardContentChanged& OnClipboardContentChanged()  { return OnClipboardContentChangedEvent; }
	
	/** Gets the horizontal alignment of the window title bar's title text. */
	virtual EWindowTitleAlignment::Type GetWindowTitleAlignment() const
	{
		return EWindowTitleAlignment::Left;
	}

	virtual EWindowTransparency GetWindowTransparencySupport() const
	{
		return EWindowTransparency::None;
	}

	virtual void DestroyApplication() { }

	virtual IInputInterface* GetInputInterface() { return nullptr; }	

	/** Function to return the current implementation of the Text Input Method System */
	virtual ITextInputMethodSystem *GetTextInputMethodSystem() { return NULL; }
	
	/** Send any analytics captured by the application */
	virtual void SendAnalytics(IAnalyticsProvider* Provider) { }

	virtual bool SupportsSystemHelp() const { return false; }

	/** Whether the platform requires a separate tooltip window per toplevel window.
	 *  On platforms where tooltip popups cannot be re-parented at runtime (e.g. Linux/Wayland),
	 *  Slate must destroy and recreate the tooltip when the mouse moves to a different toplevel. */
	virtual bool RequiresPerWindowTooltips() const { return false; }

	virtual void ShowSystemHelp() {}

	virtual bool ApplicationLicenseValid(FPlatformUserId PlatformUser = PLATFORMUSERID_NONE) { return true; }

	virtual bool IsAllowedToRender() const { return true; }

	virtual void FinishedInputThisFrame() {}
public:

	const TSharedPtr< ICursor > Cursor;

protected:

	TSharedRef< class FGenericApplicationMessageHandler > MessageHandler;

#if WITH_ACCESSIBILITY
	TSharedRef<FGenericAccessibleMessageHandler> AccessibleMessageHandler;
#endif

	
	/** Trigger the OnDisplayMetricsChanged event with the argument 'InMetrics' */
	void BroadcastDisplayMetricsChanged( const FDisplayMetrics& InMetrics ){ OnDisplayMetricsChangedEvent.Broadcast( InMetrics ); }
	
	// Notifies subscribers when any of the display metrics change: e.g. resolution changes or monitor sare re-arranged.
	FOnDisplayMetricsChanged OnDisplayMetricsChangedEvent;

	/** Delegate for virtual keyboard being shown */
	FVirtualKeyboardShownEvent VirtualKeyboardShownEvent;
	
	/** Delegate for virtual keyboard being shown */
	FVirtualKeyboardHiddenEvent VirtualKeyboardHiddenEvent;
	
	/** Delegate for clipboard contents change */
	FOnClipboardContentChanged OnClipboardContentChangedEvent;
};
