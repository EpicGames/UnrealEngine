// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "Misc/Optional.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "GenericPlatform/InputDeviceRegistry.h"

class FGenericWindow;

namespace EMouseButtons
{
	enum Type
	{
		Left = 0,
		Middle,
		Right,
		Thumb01,
		Thumb02,

		Invalid,
	};
}

struct FGamepadKeyNames
{
	typedef FName Type;

	static APPLICATIONCORE_API const FName Invalid;

	static APPLICATIONCORE_API const FName LeftAnalogX;
	static APPLICATIONCORE_API const FName LeftAnalogY;
	static APPLICATIONCORE_API const FName RightAnalogX;
	static APPLICATIONCORE_API const FName RightAnalogY;
	static APPLICATIONCORE_API const FName LeftTriggerAnalog;
	static APPLICATIONCORE_API const FName RightTriggerAnalog;

	static APPLICATIONCORE_API const FName LeftThumb;
	static APPLICATIONCORE_API const FName RightThumb;
	static APPLICATIONCORE_API const FName SpecialLeft;
	static APPLICATIONCORE_API const FName SpecialLeft_X;
	static APPLICATIONCORE_API const FName SpecialLeft_Y;
	static APPLICATIONCORE_API const FName SpecialLeft_Touched;
	static APPLICATIONCORE_API const FName SpecialRight;
	static APPLICATIONCORE_API const FName FaceButtonBottom;
	static APPLICATIONCORE_API const FName FaceButtonRight;
	static APPLICATIONCORE_API const FName FaceButtonLeft;
	static APPLICATIONCORE_API const FName FaceButtonTop;
	static APPLICATIONCORE_API const FName LeftShoulder;
	static APPLICATIONCORE_API const FName RightShoulder;
	static APPLICATIONCORE_API const FName LeftTriggerThreshold;
	static APPLICATIONCORE_API const FName RightTriggerThreshold;
	static APPLICATIONCORE_API const FName DPadUp;
	static APPLICATIONCORE_API const FName DPadDown;
	static APPLICATIONCORE_API const FName DPadRight;
	static APPLICATIONCORE_API const FName DPadLeft;

	static APPLICATIONCORE_API const FName LeftStickUp;
	static APPLICATIONCORE_API const FName LeftStickDown;
	static APPLICATIONCORE_API const FName LeftStickRight;
	static APPLICATIONCORE_API const FName LeftStickLeft;

	static APPLICATIONCORE_API const FName RightStickUp;
	static APPLICATIONCORE_API const FName RightStickDown;
	static APPLICATIONCORE_API const FName RightStickRight;
	static APPLICATIONCORE_API const FName RightStickLeft;
};

enum class EWindowActivation : uint8
{
	Activate,
	ActivateByMouse,
	Deactivate
};

namespace EWindowZone
{
	/**
	 * The Window Zone is the window area we are currently over to send back to the operating system for operating system compliance.
	 * 
	 * Must remain synchronized with platform enum maps in FWindowsApplication::ProcessMessage and FLinuxWindow::HitTest.
	 */
	enum Type
	{
		NotInWindow			= 0,
		ClientArea			= 1,

		// IsBorder below assumes these are contiguous in this order
		TopLeftBorder		= 2,
		TopBorder			= 3,
		TopRightBorder		= 4,
		LeftBorder			= 5,
		RightBorder			= 6,
		BottomLeftBorder	= 7,
		BottomBorder		= 8,
		BottomRightBorder	= 9,

		TitleBar			= 10,
		MinimizeButton		= 11,
		MaximizeButton		= 12,
		CloseButton			= 13,
		SysMenu				= 14,

		COUNT = 15,

		/** No zone specified */
		Unspecified	= 0,
	};

	inline bool IsBorder(Type Zone)
	{
		return Zone >= TopLeftBorder && Zone <= BottomRightBorder;
	}
}


namespace EWindowAction
{
	enum Type
	{
		ClickedNonClientArea	= 1,
		Maximize				= 2,
		Restore					= 3,
		WindowMenu				= 4,
	};
}


/**
 * 
 */
namespace EDropEffect
{
	enum Type
	{
		None = 0,
		Copy = 1,
		Move = 2,
		Link = 3,
	};
}


enum class EGesturePhase : uint8
{
	None,
	Update,
	Started,
	Ended
};

enum class EGestureEvent : uint8
{
	None,
	Scroll,
	Magnify,
	Swipe,
	Rotate,
	LongPress,
	Pan,
	Tap,
	Flick,
	Count
};


/** Defines the minimum and maximum dimensions that a window can take on. */
struct FWindowSizeLimits
{
public:
	FWindowSizeLimits& SetMinWidth(TOptional<float> InValue){ MinWidth = InValue; return *this; }
	const TOptional<float>& GetMinWidth() const { return MinWidth; }

	FWindowSizeLimits& SetMinHeight(TOptional<float> InValue){ MinHeight = InValue; return *this; }
	const TOptional<float>& GetMinHeight() const { return MinHeight; }

	FWindowSizeLimits& SetMaxWidth(TOptional<float> InValue){ MaxWidth = InValue; return *this; }
	const TOptional<float>& GetMaxWidth() const { return MaxWidth; }

	FWindowSizeLimits& SetMaxHeight(TOptional<float> InValue){ MaxHeight = InValue; return *this; }
	const TOptional<float>& GetMaxHeight() const { return MaxHeight; }

private:
	TOptional<float> MinWidth;
	TOptional<float> MinHeight;
	TOptional<float> MaxWidth;
	TOptional<float> MaxHeight;
};

class UE_DEPRECATED(5.8, "Use FInputDeviceRegistry::Get() instead") FInputDeviceScope;

/**
 * Context scope that indicates which IInputDevice is currently being handled.
 * This can be used to determine hardware-specific information when handling input from FGenericApplicationMessageHandler subclasses.
 * This is generally set during SendControllerEvents or Tick and is only valid on the game thread.
 *
 * @deprecated FInputDeviceScope is being superseded by FInputDeviceRegistry, which provides the same
 * hardware metadata in a thread-safe manner keyed on FInputDeviceId. All call sites should migrate to
 * FInputDeviceRegistry::FindDescriptor(DeviceId) and FInputDeviceScope will be removed once the
 * migration is complete.
 */
class FInputDeviceScope
{
public:
	/** The specific InputDevice that is currently being polled. This is only valid within the current function scope and may be null */
	class IInputDevice* InputDevice;

	/** Logical name of the input device interface. This is not translated but is platform-specific */
	FName InputDeviceName;

	/** A system-specific device id, this is not the same as controllerId and represents a physical device instead of logical user. -1 represents an unknown device */
	int32 HardwareDeviceHandle;

	/** Logical string identifying the hardware device. This is not translated and is system-specific, it may be empty */
	FString HardwareDeviceIdentifier;

	/** Constructor, this should only be allocated directly on the stack */
	APPLICATIONCORE_API FInputDeviceScope(IInputDevice* InInputDevice, FName InInputDeviceName, int32 InHardwareDeviceHandle = -1, FString InHardwareDeviceIdentifier = FString());
	APPLICATIONCORE_API ~FInputDeviceScope();

	/** Cannot be copied/moved */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FInputDeviceScope() = delete;
	FInputDeviceScope(const FInputDeviceScope&) = delete;
	FInputDeviceScope& operator=(const FInputDeviceScope&) = delete;
	FInputDeviceScope(FInputDeviceScope&&) = delete;
	FInputDeviceScope& operator=(FInputDeviceScope&&) = delete;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Returns the currently active InputDeviceScope. This is only valid to call on the game thread and may return null */
	static APPLICATIONCORE_API const FInputDeviceScope* GetCurrent();
};

/** Interface that defines how to handle interaction with a user via hardware input and output */
class FGenericApplicationMessageHandler
{
public:

	virtual ~FGenericApplicationMessageHandler() = default;

	virtual bool ShouldProcessUserInputMessages( const TSharedPtr< FGenericWindow >& PlatformWindow ) const
	{
		return false;
	}

	virtual bool OnKeyChar( const TCHAR Character, const bool IsRepeat )
	{
		return false;
	}

	virtual bool OnKeyDown( const int32 KeyCode, const uint32 CharacterCode, const bool IsRepeat ) 
	{
		return false;
	}

	virtual bool OnKeyUp( const int32 KeyCode, const uint32 CharacterCode, const bool IsRepeat )
	{
		return false;
	}

	virtual void OnInputLanguageChanged()
	{
	}

	virtual bool OnMouseDown( const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button )
	{
		return false;
	}

	virtual bool OnMouseDown( const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button, const FVector2D CursorPos )
	{
		return false;
	}

	virtual bool OnMouseUp( const EMouseButtons::Type Button )
	{
		return false;
	}

	virtual bool OnMouseUp( const EMouseButtons::Type Button, const FVector2D CursorPos )
	{
		return false;
	}

	virtual bool OnMouseDoubleClick( const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button )
	{
		return false;
	}

	virtual bool OnMouseDoubleClick( const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button, const FVector2D CursorPos )
	{
		return false;
	}

	virtual bool OnMouseWheel( const float Delta )
	{
		return false;
	}

	virtual bool OnMouseWheel( const float Delta, const FVector2D CursorPos )
	{
		return false;
	}

	virtual bool OnMouseMove()
	{
		return false;
	}

	virtual bool OnRawMouseMove( const int32 X, const int32 Y )
	{
		return false;
	}

	virtual bool OnCursorSet()
	{
		return false;
	}

	/** 
	 * Return true if this message handler expects FPlatformUserIds. This base class will convert both directions.
	 * As part of a larger fixup to allow mapping of multiple input devices to the same player, 
	 * physical device id will be passed as part of the InputScope above and used to compute a logical input user.
	 */
	virtual bool ShouldUsePlatformUserId() const
	{
		return true;
	}

	virtual bool OnControllerAnalog(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, float AnalogValue)
	{
		return false;
	}

	virtual bool OnControllerButtonPressed(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat)
	{
		return false;
	}

	virtual bool OnControllerButtonReleased(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat)
	{
		return false;
	}

    virtual void OnBeginGesture()
    {
    }

	virtual bool OnTouchGesture( EGestureEvent GestureType, const FVector2D& Delta, float WheelDelta, bool bIsDirectionInvertedFromDevice )
	{
		return false;
	}
    
    virtual void OnEndGesture()
    {
    }

	virtual bool OnNativeGestureBegin(EGestureEvent GestureType, const FVector2D& Delta, float Value, bool bIsDirectionInvertedFromDevice, FPlatformUserId PlatformUserId, FInputDeviceId DeviceId)
	{
		return false;
	}

	virtual bool OnNativeGestureUpdate(EGestureEvent GestureType, const FVector2D& Delta, float Value, bool bIsDirectionInvertedFromDevice, FPlatformUserId PlatformUserId, FInputDeviceId DeviceId)
	{
		return false;
	}

	virtual bool OnNativeGestureEnd(EGestureEvent GestureType, const FVector2D& Delta, float Value, bool bIsDirectionInvertedFromDevice, FPlatformUserId PlatformUserId, FInputDeviceId DeviceId)
	{
		return false;
	}

	virtual bool OnTouchStarted( const TSharedPtr< FGenericWindow >& Window, const FVector2D& Location, float Force, int32 TouchIndex, FPlatformUserId PlatformUserId, FInputDeviceId DeviceId )
	{
		if (!ShouldUsePlatformUserId())
		{
			return OnTouchStarted(Window, Location, Force, TouchIndex, PlatformUserId.GetInternalId());
		}
		return false;
	}

	virtual bool OnTouchMoved( const FVector2D& Location, float Force, int32 TouchIndex, FPlatformUserId PlatformUserId, FInputDeviceId DeviceID )
	{
		if (!ShouldUsePlatformUserId())
		{
			return OnTouchMoved(Location, Force, TouchIndex, PlatformUserId.GetInternalId());
		}
		return false;
	}

	virtual bool OnTouchEnded( const FVector2D& Location, int32 TouchIndex, FPlatformUserId PlatformUserId, FInputDeviceId DeviceID )
	{
		if (!ShouldUsePlatformUserId())
		{
			return OnTouchEnded(Location, TouchIndex, PlatformUserId.GetInternalId());
		}
		return false;
	}

	virtual bool OnTouchForceChanged(const FVector2D& Location, float Force, int32 TouchIndex, FPlatformUserId PlatformUserId, FInputDeviceId DeviceID)
	{
		if (!ShouldUsePlatformUserId())
		{
			return OnTouchForceChanged(Location, Force, TouchIndex, PlatformUserId.GetInternalId());
		}
		return false;
	}

	virtual bool OnTouchFirstMove(const FVector2D& Location, float Force, int32 TouchIndex, FPlatformUserId PlatformUserId, FInputDeviceId DeviceID)
	{
		if (!ShouldUsePlatformUserId())
		{
			return OnTouchFirstMove(Location, Force, TouchIndex, PlatformUserId.GetInternalId());
		}
		return false;
	}

	virtual void ShouldSimulateGesture(EGestureEvent Gesture, bool bEnable)
	{

	}

	virtual bool OnMotionDetected( const FVector& Tilt, const FVector& RotationRate, const FVector& Gravity, const FVector& Acceleration, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId )
	{
		if (!ShouldUsePlatformUserId())
		{
			return OnMotionDetected(Tilt, RotationRate, Gravity, Acceleration, PlatformUserId.GetInternalId());
		}
		return false;
	}

	virtual bool OnSizeChanged( const TSharedRef< FGenericWindow >& Window, const int32 Width, const int32 Height, bool bWasMinimized = false )
	{
		return false;
	}

	virtual void OnOSPaint( const TSharedRef<FGenericWindow>& Window )
	{
	
	}

	virtual FWindowSizeLimits GetSizeLimitsForWindow( const TSharedRef<FGenericWindow>& Window ) const
	{
		return FWindowSizeLimits();
	}

	virtual void OnResizingWindow( const TSharedRef< FGenericWindow >& Window )
	{

	}

	virtual bool BeginReshapingWindow( const TSharedRef< FGenericWindow >& Window )
	{
		return true;
	}

	virtual void FinishedReshapingWindow( const TSharedRef< FGenericWindow >& Window )
	{

	}

	virtual void DismissAllMenus()
	{

	}

	virtual void HandleDPIScaleChanged( const TSharedRef< FGenericWindow >& Window )
	{

	}

	virtual void SignalSystemDPIChanged(const TSharedRef< FGenericWindow >& Window)
	{

	}

	virtual void OnRegularWindowMovedOrResized(const TSharedRef< FGenericWindow >& Window)
	{

	}

	virtual void OnMovedWindow( const TSharedRef< FGenericWindow >& Window, const int32 X, const int32 Y )
	{

	}

	virtual bool OnWindowActivationChanged( const TSharedRef< FGenericWindow >& Window, const EWindowActivation ActivationType )
	{
		return false;
	}

	virtual bool OnApplicationActivationChanged( const bool IsActive )
	{
		return false;
	}

	virtual bool OnConvertibleLaptopModeChanged()
	{
		return false;
	}

	virtual EWindowZone::Type GetWindowZoneForPoint( const TSharedRef< FGenericWindow >& Window, const int32 X, const int32 Y )
	{
		return EWindowZone::NotInWindow;
	}

	virtual void OnWindowClose( const TSharedRef< FGenericWindow >& Window )
	{

	}

	virtual EDropEffect::Type OnDragEnterText( const TSharedRef< FGenericWindow >& Window, const FString& Text )
	{
		return EDropEffect::None;
	}

	virtual EDropEffect::Type OnDragEnterFiles( const TSharedRef< FGenericWindow >& Window, const TArray< FString >& Files )
	{
		return EDropEffect::None;
	}

	virtual EDropEffect::Type OnDragEnterExternal( const TSharedRef< FGenericWindow >& Window, const FString& Text, const TArray< FString >& Files )
	{
		return EDropEffect::None;
	}

	virtual EDropEffect::Type OnDragOver( const TSharedPtr< FGenericWindow >& Window )
	{
		return EDropEffect::None;
	}

	virtual void OnDragLeave( const TSharedPtr< FGenericWindow >& Window )
	{

	}

	virtual EDropEffect::Type OnDragDrop( const TSharedPtr< FGenericWindow >& Window )
	{
		return EDropEffect::None;
	}

	virtual bool OnWindowAction( const TSharedRef< FGenericWindow >& Window, const EWindowAction::Type InActionType)
	{
		return true;
	}

	virtual void SetCursorPos(const FVector2D& MouseCoordinate)
	{

	}
	
	virtual bool OnTouchStarted(const TSharedPtr< FGenericWindow >& Window, const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId)
	{ 
		if (ShouldUsePlatformUserId())
		{
			// Remap the old int32 ControlerId to the new Platform user for backwards compat
			FPlatformUserId UserId = FGenericPlatformMisc::GetPlatformUserForUserIndex(ControllerId);
			FInputDeviceId DeviceId = INPUTDEVICEID_NONE;
			IPlatformInputDeviceMapper::Get().RemapControllerIdToPlatformUserAndDevice(ControllerId, UserId, DeviceId);
			return OnTouchStarted(Window, Location, Force, TouchIndex, UserId, DeviceId);
		}
		return false;
	}
	virtual bool OnTouchMoved(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId)
	{
		if (ShouldUsePlatformUserId())
		{
			// Remap the old int32 ControlerId to the new Platform user for backwards compat
			FPlatformUserId UserId = FGenericPlatformMisc::GetPlatformUserForUserIndex(ControllerId);
			FInputDeviceId DeviceId = INPUTDEVICEID_NONE;
			IPlatformInputDeviceMapper::Get().RemapControllerIdToPlatformUserAndDevice(ControllerId, UserId, DeviceId);
			return OnTouchMoved(Location, Force, TouchIndex, UserId, DeviceId);
		}
		return false;
	}
	virtual bool OnTouchEnded(const FVector2D& Location, int32 TouchIndex, int32 ControllerId)
	{
		if (ShouldUsePlatformUserId())
		{
			// Remap the old int32 ControlerId to the new Platform user for backwards compat
			FPlatformUserId UserId = FGenericPlatformMisc::GetPlatformUserForUserIndex(ControllerId);
			FInputDeviceId DeviceId = INPUTDEVICEID_NONE;
			IPlatformInputDeviceMapper::Get().RemapControllerIdToPlatformUserAndDevice(ControllerId, UserId, DeviceId);
			return OnTouchEnded(Location, TouchIndex, UserId, DeviceId);
		}
		return false;
	}
	virtual bool OnTouchForceChanged(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId)
	{
		if (ShouldUsePlatformUserId())
		{
			// Remap the old int32 ControlerId to the new Platform user for backwards compat
			FPlatformUserId UserId = FGenericPlatformMisc::GetPlatformUserForUserIndex(ControllerId);
			FInputDeviceId DeviceId = INPUTDEVICEID_NONE;
			IPlatformInputDeviceMapper::Get().RemapControllerIdToPlatformUserAndDevice(ControllerId, UserId, DeviceId);
			return OnTouchForceChanged(Location, Force, TouchIndex, UserId, DeviceId);
		}
		return false;
	}
	virtual bool OnTouchFirstMove(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId)
	{
		if (ShouldUsePlatformUserId())
		{
			// Remap the old int32 ControlerId to the new Platform user for backwards compat
			FPlatformUserId UserId = FGenericPlatformMisc::GetPlatformUserForUserIndex(ControllerId);
			FInputDeviceId DeviceId = INPUTDEVICEID_NONE;
			IPlatformInputDeviceMapper::Get().RemapControllerIdToPlatformUserAndDevice(ControllerId, UserId, DeviceId);
			return OnTouchFirstMove(Location, Force, TouchIndex, UserId, DeviceId);
		}
		return false;
	}
	virtual bool OnMotionDetected(const FVector& Tilt, const FVector& RotationRate, const FVector& Gravity, const FVector& Acceleration, int32 ControllerId)
	{
		if (ShouldUsePlatformUserId())
		{
			// Remap the old int32 ControlerId to the new Platform user for backwards compat
			FPlatformUserId UserId = FGenericPlatformMisc::GetPlatformUserForUserIndex(ControllerId);
			FInputDeviceId DeviceId = INPUTDEVICEID_NONE;
			IPlatformInputDeviceMapper::Get().RemapControllerIdToPlatformUserAndDevice(ControllerId, UserId, DeviceId);
			return OnMotionDetected(Tilt, RotationRate, Gravity, Acceleration, UserId, DeviceId);
		}
		return false;
	}
};
