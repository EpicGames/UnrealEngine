// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

DECLARE_LOG_CATEGORY_EXTERN(LogConfigRules, Log, All);

class FGenericConfigRules
{
public:
    /** Parses a rule file based on PredefinedVariables and returns the Variable Map. */
    static APPLICATIONCORE_API TMap<FString, FString> ParseConfigRules(TConstArrayView<uint8> ConfigRulesData, TMap<FString, FString>&& PredefinedVariables);
};
