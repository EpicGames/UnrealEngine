// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "Misc/CoreMiscDefines.h"	// For FInputDeviceId/FPlatformUserId

/**
* An interface which can be added to any IInputDevice (or any other type 
* which is creating Human Interface Devices) to store it's unique identifier
* for those physical devices and associate them with a FInputDeviceId so that
* the rest of the Unreal Engine can interact with them
* 
* Some input interfaces use simple int32's to identify input devices, while
* others may have more specific GUID types types which are needed.
*/
template <class TDeviceKeyType>
class TInputDeviceMap
{
public:	

	TInputDeviceMap() = default;
	~TInputDeviceMap() = default;

	/**
	* Given the DeviceKey, find it's associated FInputDeviceId.
	* 
	* If one does not exist yet, such as for a newly connected device,
	* then create one from the IPlatformInputDeviceMapper.
	*/
	[[nodiscard]] FInputDeviceId GetOrCreateDeviceId(const TDeviceKeyType& DeviceKey)
	{
		FInputDeviceId OutDeviceId = INPUTDEVICEID_NONE;
		
		// If we already know about this device, then we can just use that info
		if (FInputDeviceId* ExistingDeviceId = MappedDeviceIds.Find(DeviceKey))
		{
			OutDeviceId = *ExistingDeviceId;
		}
		// Otherwise, we have not seen this input device before (its a new connection)
		// So we need a new FInputDeviceId and a FPlatformUserId to map it to. 
		else
		{
			OutDeviceId = IPlatformInputDeviceMapper::Get().AllocateNewInputDeviceId();

			// Keep track of both the FInputDeviceId -> DeviceKey 
			// and DeviceKey -> FInputDeviceID for quick lookup
			MappedDeviceIds.Add(DeviceKey, OutDeviceId);
			MappedIdToKey.Add(OutDeviceId, DeviceKey);
		}

		return OutDeviceId;
	}

	/**
	* Maps a DeviceKey to the default FInputDeviceId which has the internal value 0.
	* Returns the default FInputDeviceId for straightforward use.
	*/
	[[nodiscard]] FInputDeviceId MapDefaultInputDevice(const TDeviceKeyType& DeviceKey)
	{
		FInputDeviceId OutDeviceId = IPlatformInputDeviceMapper::Get().GetDefaultInputDevice();

		MappedDeviceIds.Add(DeviceKey, OutDeviceId);
		MappedIdToKey.Add(OutDeviceId, DeviceKey);

		return OutDeviceId;
	}

	/**
	* Returns the FInputDeviceId for the given device key.
	* 
	* Returns FInputDeviceId::Invalid if it is not yet mapped.
	*/
	[[nodiscard]] FInputDeviceId FindDeviceId(const TDeviceKeyType& DeviceKey) const
	{
		FInputDeviceId OutDeviceId = INPUTDEVICEID_NONE;

		// If we already know about this device, then we can just use that info
		if (const FInputDeviceId* ExistingDeviceId = MappedDeviceIds.Find(DeviceKey))
		{
			OutDeviceId = *ExistingDeviceId;
		}

		return OutDeviceId;
	}

	/**
	* Returns the FInputDeviceId associated with the given device key. Asserts if not found.
	*/
	[[nodiscard]] FInputDeviceId FindDeviceIdChecked(const TDeviceKeyType& DeviceKey) const
	{
		return MappedDeviceIds.FindChecked(DeviceKey);
	}

	/**
	* Returns the device key for the given FInputDeviceId. Nullptr if not found.
	*/
	[[nodiscard]] const TDeviceKeyType& GetDeviceKeyChecked(const FInputDeviceId DeviceId) const
	{
		return MappedIdToKey.FindChecked(DeviceId);
	}

	/**
	* Returns the device key for the given FInputDeviceId. Nullptr if not found.
	*/
	[[nodiscard]] const TDeviceKeyType* FindDeviceKey(const FInputDeviceId DeviceId) const
	{
		return MappedIdToKey.Find(DeviceId);
	}

protected:

	/**
	* Map of the assigned DeviceKeyType to their assigned FInputDeviceId's from the engine.
	*/
	TMap<TDeviceKeyType, FInputDeviceId> MappedDeviceIds;

	/**
	* A map of the assigned FInputDeviceId to their associated Device Key type for 
	* fast lookup.
	*/
	TMap<FInputDeviceId, TDeviceKeyType> MappedIdToKey;
};