// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "HAL/IConsoleManager.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Templates/IsFloatingPoint.h"
#include "Templates/UnrealTypeTraits.h"
#include "Templates/Function.h"
#include "Misc/EnumClassFlags.h"

class FAutoConsoleVariableRef;
class FFeedbackContext;
class FOutputDeviceConsole;
class FOutputDeviceError;
class IPlatformInputDeviceMapper;

/**
 * The accuracy when dealing with physical characteristics of the monitor/screen of the device we're running on.
 */
enum class EScreenPhysicalAccuracy
{
	Unknown,
	Approximation,
	Truth
};

/**
 * Callback for when FindInputDeviceForUserWithUI has completed
 */
struct FShowInputDeviceSelectorParams
{
	FInputDeviceId InputDeviceId;
	FPlatformUserId PlatformUserId;
};
typedef TFunction<void(const FShowInputDeviceSelectorParams&)> FShowInputDeviceSelectorComplete;

/**
 * Callback for when ShowPlatformUserSelector has completed.
 */
struct FPlatformUserSelectionCompleteParams
{
	FPlatformUserId SelectedUserId;
	bool bSuccess;
};
typedef TFunction<void(const FPlatformUserSelectionCompleteParams& Params)> FPlatformUserSelectionComplete;

/*
 * Options for ShowPlatformUserSelector. Not all platforms will support all flags
 */
enum class EPlatformUserSelectorFlags : uint8
{
	None                         = 0,
	RequiresOnlineEnabledProfile = (1 << 1), // whether to only show profiles that have an online account
	ShowSkipButton               = (1 << 2), // display a 'skip' button UI to be closed & return the initiating user
	AllowGuests                  = (1 << 3), // temporary guest accounts, throwaways etc.
	ShowNewUsersOnly             = (1 << 4), // do not show profiles that are already in-use by the title

	Default                      = ShowSkipButton, // most commonly used options
};
ENUM_CLASS_FLAGS(EPlatformUserSelectorFlags);


struct FGenericPlatformApplicationMisc
{
	static APPLICATIONCORE_API void PreInit();

	static APPLICATIONCORE_API void Init();

	static APPLICATIONCORE_API void PostInit();

	static APPLICATIONCORE_API void TearDown();

	/**
	 * Load the preinit modules required by this platform, typically they are the renderer modules
	 */
	static void LoadPreInitModules()
	{
	}

	/**
	 * Load the platform-specific startup modules
	 */
	static void LoadStartupModules()
	{
	}

	/**
	 * Creates a console output device for this platform. Should only be called once.
	 */
	static APPLICATIONCORE_API FOutputDeviceConsole* CreateConsoleOutputDevice();

	/**
	 * Gets a pointer to the platform error output device singleton.
	 */
	static APPLICATIONCORE_API FOutputDeviceError* GetErrorOutputDevice();

	/**
	 * Gets a pointer to the default platform feedback context implementation.
	 */
	static APPLICATIONCORE_API FFeedbackContext* GetFeedbackContext();

	/**
	 * Gets a pointer to the default platform input device manager.
	 */
	static APPLICATIONCORE_API IPlatformInputDeviceMapper* CreatePlatformInputDeviceManager();

	/**
	 * Creates an application instance.
	 */
	static APPLICATIONCORE_API class GenericApplication* CreateApplication();

	/** Request application to minimize (goto background). **/
	static APPLICATIONCORE_API void RequestMinimize();

	/** Returns true if the specified application has a visible window, and that window is active/has focus/is selected */
	static APPLICATIONCORE_API bool IsThisApplicationForeground();	

	/**
	* Returns whether the platform wants to use a touch screen for a virtual keyboard.
	*/
	static APPLICATIONCORE_API bool RequiresVirtualKeyboard();

	/**
	 *	Pumps Windows messages.
	 *	@param bFromMainLoop if true, this is from the main loop, otherwise we are spinning waiting for the render thread
	 */
	inline static void PumpMessages(bool bFromMainLoop)
	{
	}

	/**
	 * Prevents screen-saver from kicking in by moving the mouse by 0 pixels. This works even
	 * in the presence of a group policy for password protected screen saver.
	 */
	static void PreventScreenSaver()
	{
	}

	enum EScreenSaverAction
	{
		Disable,
		Enable
	};

	/**
	 * Returns state of screensaver (if platform supports it)
	 *
	 * @return	true if screensaver enabled (returns false if platform does not support it)
	 *
	 */
	static bool IsScreensaverEnabled()
	{
		return false;
	}

	/**
	 * Disables screensaver (if platform supports such an API)
	 *
	 * @param Action enable or disable
	 * @return true if succeeded, false if platform does not have this API and PreventScreenSaver() hack is needed
	 */
	static bool ControlScreensaver(EScreenSaverAction Action)
	{
		return false;
	}

	/**
	 * Sample the displayed pixel color from anywhere on the screen using the OS
	 *
	 * @param	InScreenPos		The screen coords to sample for current pixel color
	 * @param	InGamma			Optional gamma correction to apply to the screen color
	 * @return					The color of the pixel displayed at the chosen location
	 */
	static APPLICATIONCORE_API struct FLinearColor GetScreenPixelColor(const FVector2D& InScreenPos, float InGamma = 1.0f);

	/**
	 * Searches for a window that matches the window name or the title starts with a particular text. When
	 * found, it returns the title text for that window
	 *
	 * @param TitleStartsWith an alternative search method that knows part of the title text
	 * @param OutTitle the string the data is copied into
	 *
	 * @return whether the window was found and the text copied or not
	 */
	static bool GetWindowTitleMatchingText(const TCHAR* TitleStartsWith, FString& OutTitle)
	{
		return false;
	}

	/**
	 * Allows the OS to enable high DPI mode
	 */
	static void SetHighDPIMode()
	{

	}

	/**
	* Returns monitor's DPI scale factor at given screen coordinates (expressed in pixels)
	* @return Monitor's DPI scale factor at given point
	*/
	static float GetDPIScaleFactorAtPoint(float X, float Y)
	{
		return 1.0f;
	}


	/** @return true if the application is high dpi aware */

	static APPLICATIONCORE_API bool IsHighDPIAwarenessEnabled();

	/*
	 * UE expects mouse coordinates in screen space. Some platforms provides in client space. 
	 * Return true to anchor the window at the top/left corner to make sure client space coordinates and screen space coordinates match up. 
	 */
	static bool AnchorWindowWindowPositionTopLeft()
	{
		return false;
	}

	/*
	 * Return true if notification windows must be parented to an existing window and anchored to it,
	 * for platforms that cannot position top-level windows at an absolute screen coordinate (pure Wayland).
	 */
	static bool RequiresParentedNotifications()
	{
		return false;
	}

	/*
	* Set whether gamepads are allowed at the platform level.
	*/
	static void SetGamepadsAllowed(bool bAllowed)
	{}

	/*
	* Set whether gamepads are allowed at the platform level.
	*/
	static void SetGamepadsBlockDeviceFeedback(bool bAllowed)
	{}

	/*
	 * Resets the gamepad to player controller id assignments
	 */
	static void ResetGamepadAssignments()
	{}

	/*
	* Resets the gamepad assignment to player controller id
	*/
	static void ResetGamepadAssignmentToController(int32 ControllerId)
	{}

	/*
	 * Returns true if controller id assigned to a gamepad
	 */
	static bool IsControllerAssignedToGamepad(int32 ControllerId)
	{
		return (ControllerId == 0);
	}

	/*
	* Returns name of gamepad if controller id assigned to a gamepad
	*/
	static FString GetGamepadControllerName(int32 ControllerId)
	{
		if (IsControllerAssignedToGamepad(ControllerId))
		{
			return FString("Generic");
		}
		return FString("None");
	}

	/*
	* Returns a texture of the glyph representing the specified button on the specified controller, or nullptr if not supported.
	*/
    static class UTexture2D* GetGamepadButtonGlyph(const FGamepadKeyNames::Type& ButtonKey, uint32 ControllerIndex)
    {
        return nullptr;
    }
    
	/*
	* Whether to enable controller motion data polling (by default motion data is enabled)
	* Some platforms may want to disable it to reduce battery drain
	*/
	static void EnableMotionData(bool bEnable)
	{}

	/*
	* Whether controller motion data polling is enabled (by default motion data is enabled)
	*/
	static bool IsMotionDataEnabled()
	{
		return true;
	}
				
	/** Copies text to the operating system clipboard. */
	static APPLICATIONCORE_API void ClipboardCopy(const TCHAR* Str);

	/** Pastes in text from the operating system clipboard. */
	static APPLICATIONCORE_API void ClipboardPaste(class FString& Dest);

	/**
	 * Gets the physical size of the screen if possible.  Some platforms lie, some platforms don't know.
	 */
	static APPLICATIONCORE_API EScreenPhysicalAccuracy GetPhysicalScreenDensity(int32& OutScreenDensity);

	static APPLICATIONCORE_API EScreenPhysicalAccuracy GetPhysicalScreenDimensions(FVector2f& OutScreenDimensions);

	static APPLICATIONCORE_API EScreenPhysicalAccuracy GetPhysicalScreenSize(float& OutScreenSize);

	/**
	 * Gets the physical size of the screen if possible.  Some platforms lie, some platforms don't know.
	 */
	static APPLICATIONCORE_API EScreenPhysicalAccuracy ComputePhysicalScreenDensity(int32& OutScreenDensity);

	static APPLICATIONCORE_API EScreenPhysicalAccuracy ComputePhysicalScreenDimensions(FVector2f& OutScreenDimensions, float& OutScreenSize);

	static void ComputePhysicalScreenData();

	/**
	 * If we know or can approximate the pixel density of the screen we will convert the incoming inches
	 * to pixels on the device.  If the accuracy is unknown OutPixels will be set to 0.
	 */
	template<typename T, typename T2 UE_REQUIRES(std::is_floating_point_v<T> && std::is_floating_point_v<T2>)>
	static EScreenPhysicalAccuracy ConvertInchesToPixels(T Inches, T2& OutPixels)
	{
		int32 ScreenDensity = 0;
		const EScreenPhysicalAccuracy Accuracy = GetPhysicalScreenDensity(ScreenDensity);

		if (ScreenDensity != 0)
		{
			OutPixels = static_cast<T2>(Inches * ScreenDensity);
		}
		else
		{
			OutPixels = 0;
		}

		return Accuracy;
	}

	/**
	 * If we know or can approximate the pixel density of the screen we will convert the incoming pixels
	 * to inches on the device.  If the accuracy is unknown OutInches will be set to 0.
	 */
	template<typename T, typename T2 UE_REQUIRES(std::is_floating_point_v<T> && std::is_floating_point_v<T2>)>
	static EScreenPhysicalAccuracy ConvertPixelsToInches(T Pixels, T2& OutInches)
	{
		int32 ScreenDensity = 0;
		const EScreenPhysicalAccuracy Accuracy = GetPhysicalScreenDensity(ScreenDensity);

		if (ScreenDensity != 0)
		{
			OutInches = static_cast<T2>(Pixels / ScreenDensity);
		}
		else
		{
			OutInches = 0;
		}

		return Accuracy;		
	}
	
	/**
	 * Asyncronously display the platform-specific input device selection UI, if supported.
	 * 
	 * @param InitiatingUserId The platform user to find an input device for.
	 * @param OnShowInputDeviceSelectorComplete Callback for when the input device selection operation has completed
	 * @return true if the UI will be shown and the callback will be called
	 * @return false if the platform does not support an input device selection API, or it is not currently available (the callback will not be called)
	 */
	static bool ShowInputDeviceSelector( FPlatformUserId InitiatingUserId, FShowInputDeviceSelectorComplete OnShowInputDeviceSelectorComplete )
	{
		// no default implementation
		return false;
	}

	/**
	 * Asyncronously display the platform-specific user selection UI, if supported.
	 * 
	 * @param InitiatingInputDeviceId The input device that prompted showing the UI, if applicable. The input device may be re-paired with the newly-selected user, depending on the platform
	 * @param Flags customization options for the user selection UI.
	 * @param OnUserSelectionComplete callback for when the user selection operation has completed
	 * @return true if the UI will be shown and the callback will be called
	 * @return false if the platform does not support a user-selection API, or it is not currently available (the callback will not be called)
	 */
	static bool ShowPlatformUserSelector(FInputDeviceId InitiatingInputDeviceId, EPlatformUserSelectorFlags Flags, FPlatformUserSelectionComplete OnUserSelectionComplete)
	{
		// no default implementation
		return false;
	}

	/** Returns true when native platform gesture recognizers should be used instead of software gesture detection. */
	static APPLICATIONCORE_API bool AreNativeGesturesEnabled();

protected:
	static APPLICATIONCORE_API bool CachedPhysicalScreenData;
	static APPLICATIONCORE_API EScreenPhysicalAccuracy CachedPhysicalScreenAccuracy;
	static APPLICATIONCORE_API int32 CachedPhysicalScreenDensity;
	static APPLICATIONCORE_API FAutoConsoleVariableRef CVarEnableHighDPIAwareness;
	static APPLICATIONCORE_API FAutoConsoleVariableRef CVarAllowVirtualKeyboard;

	static APPLICATIONCORE_API EScreenPhysicalAccuracy CachedPhysicalScreenDimensionsAccuracy;
	static APPLICATIONCORE_API FVector2f CachedPhysicalScreenDimensions;
	static APPLICATIONCORE_API float CachedPhysicalScreenSize;

private:
	static APPLICATIONCORE_API TAutoConsoleVariable<bool> CVarEnableNativeGestureRecognition;
};
