// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	GenericPlatformFramePacer.h: Generic platform frame pacer classes
==============================================================================================*/


#pragma once

#include "CoreFwd.h"
#include "CoreTypes.h"

/**
 * Generic implementation for most platforms
 **/
struct FGenericPlatformRHIFramePacer
{
	/**
	 * The pace we are running at (30 = 30fps, 0 = unpaced)
	 * The generic implementation returns a result based on rhi.SyncInterval and FPlatformMisc::GetMaxRefreshRate().
	 */
	static inline int32 GetFramePace() 
	{
		return GetFramePaceFromSyncInterval();
	}

	/**
	 * Sets the pace we would like to running at (30 = 30fps, 0 = unpaced).
	 * The generic implementation sets the value for rhi.SyncInterval according to FPlatformMisc::GetMaxRefreshRate().
	 *
	 * @return the pace we will run at.
	 */
	static inline int32 SetFramePace(int32 FramePace)
	{
		return SetFramePaceToSyncInterval(FramePace);
	}

	/**
	 * Returns whether the hardware is able to frame pace at the specified frame rate
	 */
	static APPLICATIONCORE_API bool SupportsFramePace(int32 QueryFramePace);

protected:
	/**
	 * The generic implementation returns a result based on rhi.SyncInterval and FPlatformMisc::GetMaxRefreshRate().
	 */
	static APPLICATIONCORE_API int32 GetFramePaceFromSyncInterval();

	/**
	 * The generic implementation sets rhi.SyncInterval based on FPlatformMisc::GetMaxRefreshRate().
	 *
	 * @return the pace we will run at.
	 */
	static APPLICATIONCORE_API int32 SetFramePaceToSyncInterval(int32 FramePace);
};
