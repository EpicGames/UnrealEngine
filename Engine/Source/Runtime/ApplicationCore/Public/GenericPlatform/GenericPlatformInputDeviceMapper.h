// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogInputDeviceMapper, Log, All);

enum class EInputDeviceMappingPolicy : int32;

/**
 * Base class to private a mapping of Platform Users (FPlatformUserID)
 * to their associated available input devices (FInputDeviceID).
 *
 * This will handle the allocation of the globally unique identifier
 * of the FInputDeviceID, and allow overrides of how each platform
 * maps input devices to their users. Some platforms may desire to
 * have each new input device assigned to a different user, while others
 * may want multiple input devices associated with a single user.
 * 
 * @note Keep any function comments up to date with those in PlatformInputDeviceMapperLibrary.h!
 */
class IPlatformInputDeviceMapper
{
public:
	
	/** Get the platform input device mapper */
	static APPLICATIONCORE_API IPlatformInputDeviceMapper& Get();
	
	APPLICATIONCORE_API IPlatformInputDeviceMapper();
	
	/** Virtual destructor */
	APPLICATIONCORE_API virtual ~IPlatformInputDeviceMapper();
	
	/**
	 * Populates the OutInputDevices array with any InputDeviceID's that are mapped to the given platform user
	 *
	 * @param UserId				The Platform User to gather the input devices of.
	 * @param OutInputDevices		Array of input device ID's that will be populated with the mapped devices.
	 * @return						The number of mapped devices, INDEX_NONE if the user was not found.
	 */
	APPLICATIONCORE_API virtual int32 GetAllInputDevicesForUser(const FPlatformUserId UserId, TArray<FInputDeviceId>& OutInputDevices) const;

	/**
	 * Get all mapped input devices on this platform regardless of their connection state.
	 * 
	 * @param OutInputDevices	Array of input devices to populate
	 * @return					The number of connected input devices
	 */
	APPLICATIONCORE_API virtual int32 GetAllInputDevices(TArray<FInputDeviceId>& OutInputDevices) const;
	
	/**
	 * Gather all currently connected input devices
	 * 
	 * @param OutInputDevices	Array of input devices to populate
	 * @return					The number of connected input devices
	 */
	APPLICATIONCORE_API virtual int32 GetAllConnectedInputDevices(TArray<FInputDeviceId>& OutInputDevices) const;

	/**
	 * Gather all currently connected input devices for specific user
	 *
	 * @param UserId				The Platform User to gather the input devices of.
	 * @param OutInputDevices		Array of input device ID's that will be populated with the mapped devices.
	 * @return						The number of mapped devices, INDEX_NONE if the user was not found.
	 */
	APPLICATIONCORE_API virtual int32 GetAllConnectedInputDevicesForUser(const FPlatformUserId UserId, TArray<FInputDeviceId>& OutInputDevices) const;

	/**
	 * Get all currently active platform ids, anyone who has a mapped input device
	 *
	 * @param OutUsers		Array that will be populated with the platform users.
	 * @return				The number of active platform users
	 */
	APPLICATIONCORE_API virtual int32 GetAllActiveUsers(TArray<FPlatformUserId>& OutUsers) const;

	/**
	 * Finds the first FPlatformUserId which was already allocated with no input devices.
	 * 
	 * Returns PLATFORMUSERID_NONE if every current platform user has a valid input device mapped to it.
	 * 
	 * Will not return the Unpaired platform user.
	 */
	APPLICATIONCORE_API virtual FPlatformUserId GetFirstPlatformUserWithNoInputDevice() const;

	/**
	 * Returns the platform user id that is being used for unmapped input devices.
	 * Will be PLATFORMUSERID_NONE if platform does not support this (this is the default behavior)
	 */
	virtual FPlatformUserId GetUserForUnpairedInputDevices() const = 0;

	/** Returns true if the given Platform User Id is the user for unpaired input devices on this platform. */
	APPLICATIONCORE_API virtual bool IsUnpairedUserId(const FPlatformUserId PlatformId) const;

	/** Returns true if the given input device is mapped to the unpaired platform user id. */
	APPLICATIONCORE_API virtual bool IsInputDeviceMappedToUnpairedUser(const FInputDeviceId InputDevice) const;

	/**
	 * Returns the 'Primary' Platform user for this platform.
	 * This typically has an internal ID of '0' and is used as the default platform user to
	 * map devices such as the keyboard and mouse that don't get assigned unique ID's from their
	 * owning platform code.
	 */
	virtual FPlatformUserId GetPrimaryPlatformUser() const = 0;
	
	/** Returns the default device id used for things like keyboard/mouse input */
	virtual FInputDeviceId GetDefaultInputDevice() const = 0;

	/**
	 * Returns the platform user id that a newly connected device should be mapped to.
	 *
	 * If "ShouldCreateUniqueUserForEachDevice" is true, then a new platform user id will be allocated for this device.
	 * Otherwise, this will return the primary platform user.
	 *
	 * @param InUserId		An optional user index which can be passed in. If this is not -1, and IsUsingControllerIdAsUserId is true,
	 *						then the platform user id for that user will be returned. 	
	 */
	APPLICATIONCORE_API virtual FPlatformUserId GetPlatformUserForNewlyConnectedDevice(const int32 InUserId = -1); 

	/** Returns the platform user attached to this input device, or PLATFORMUSERID_NONE if invalid */
	APPLICATIONCORE_API virtual FPlatformUserId GetUserForInputDevice(FInputDeviceId DeviceId) const;

	/** Returns the primary input device used by a specific player, or INPUTDEVICEID_NONE if invalid */
	APPLICATIONCORE_API virtual FInputDeviceId GetPrimaryInputDeviceForUser(FPlatformUserId UserId) const;

	/**
	 * Set the connection state of a given Input Device to something new. This will
	 * broadcast the OnInputDeviceConnectionChange delegate.
	 * This should be called by the platform's implementation.
	 *
	 * @param DeviceId		The device id that has had a connection change
	 * @param NewState		The new connection state of the given device
	 * @return				True if the connection state was set successfully
	 */
	APPLICATIONCORE_API virtual bool Internal_SetInputDeviceConnectionState(FInputDeviceId DeviceId, EInputDeviceConnectionState NewState);
	
	/**
	 * Gets the connection state of the given input device.
	 * 
	 * @param DeviceId		The device to get the connection state of
	 * @return				The connection state of the given device. EInputDeviceConnectionState::Unknown if the device is not mapped
	 */
	APPLICATIONCORE_API virtual EInputDeviceConnectionState GetInputDeviceConnectionState(const FInputDeviceId DeviceId) const;

	/**
	 * Maps the given Input Device to the given userID. This will broadcast the OnInputDeviceConnectionChange delegate.
	 * This should be called by the platform's implementation.
	 *
	 * @param DeviceId			The device id to map
	 * @param UserId			The Platform User that owns the given device
	 * @param ConnectionState	The connection state of the device
	 * @return					True if the input device was successfully mapped
	 */
	APPLICATIONCORE_API virtual bool Internal_MapInputDeviceToUser(FInputDeviceId DeviceId, FPlatformUserId UserId, EInputDeviceConnectionState ConnectionState);

	/**
	 * Change the user mapping of the given input device from an old user to a new one.
	 * This will broadcast the OnInputDevicePairingChange delegate.
	 * Use this when you know that an input device is already mapped, but it has changed platform users
	 * This should be called by the platform's implementation.
	 *
	 * @param DeviceId		The input device to change the owner on
	 * @param NewUserId		The new platform user that this input device should be mapped to
	 * @param OldUserId		The old platform user that this input device is currently mapped to
	 * @return				True if the device was successfully remapped
	 */
	APPLICATIONCORE_API virtual bool Internal_ChangeInputDeviceUserMapping(FInputDeviceId DeviceId, FPlatformUserId NewUserId, FPlatformUserId OldUserId);

	/**
	* Returns the max allowed number of platform users this platform can have.
	* 
	* By default, this will return the value specified in the input settings ini file
	*/
	APPLICATIONCORE_API virtual int32 GetMaxPlatformUserCount() const;

	//////////////////////////////////////////////////////////////////////////////
	// Delegates for listening to input device changes

	/**
	 * Callback which is fired before the "user" input device connection change. When this is fired, the input device metadata may not yet be populated.
	 * 
	 * This is intended for any "lower level" systems which want to directly access device connection changes before other engine plugins and game code does.
	 * 
	 * This should only be fired if there is a valid FInputDeviceScope which can be used to derive the hardware info from.
	 * 
	 * @param NewConnectionState	The new connection state of this device
	 * @param FPlatformUserId		The User ID whose input device has changed
	 * @param FInputDeviceId		The Input Device ID that has changed connection
	 */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnInputDeviceHardwareInfoAvailable, EInputDeviceConnectionState /* NewConnectionState */, FPlatformUserId /* PlatformUserId */, FInputDeviceId /* InputDeviceId */);
	
	/**
	 * Callback for handling an Input Device's connection state change.
	 * 
	 * @param NewConnectionState	The new connection state of this device
	 * @param FPlatformUserId		The User ID whose input device has changed
	 * @param FInputDeviceId		The Input Device ID that has changed connection
	 */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnUserInputDeviceConnectionChange, EInputDeviceConnectionState /* NewConnectionState */, FPlatformUserId /* PlatformUserId */, FInputDeviceId /* InputDeviceId */);

	/**
	 * Callback for handling an Input Device pairing change.
	 * 
	 * @param FInputDeviceId	Input device ID
	 * @param FPlatformUserId	The NewUserPlatformId
	 * @param FPlatformUserId	The OldUserPlatformId
	 */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnUserInputDevicePairingChange, FInputDeviceId /* InputDeviceId */, FPlatformUserId /* NewUserPlatformId */, FPlatformUserId /* OldUserPlatformId */);

	FOnInputDeviceHardwareInfoAvailable& GetOnInputDeviceHardwareInfoAvailable() const { return OnInputDeviceHardwareInfoAvailable; }
	FOnUserInputDeviceConnectionChange& GetOnInputDeviceConnectionChange() const { return OnInputDeviceConnectionChange; }
	FOnUserInputDevicePairingChange& GetOnInputDevicePairingChange() const { return OnInputDevicePairingChange; }

#if WITH_EDITOR
	/**
	 * Remaps the currently connected input devices based on the new given policy. This property can only be changed
	 * at editor time and should not be used at runtime.
	 */
	APPLICATIONCORE_API void HandleInputDevicePolicyChanged(const EInputDeviceMappingPolicy NewPolicy);
#endif	// #if WITH_EDITOR
	
	//////////////////////////////////////////////////////////////////////////////
	// Functions to provide compatibility between the old "int32 ControllerId"
	// and the new FPlatformUserId and FInputDeviceId structs.
	
	/**
	 * Remap a FPlatformUserId to a ControllerId that is needed by legacy code. Use this function to
	 * add compatibility to platforms that may not have implemented this device mapper yet.
	 *
	 * @param UserId				The platform user that should be converted
	 * @param OutControllerId		Set to the old plain "int32" that represented gamepad id or user index depending on the context.
	 * @param OptionalDeviceId		Set this to a specific device id if known
	 * @return						True if this maps to a real user
	 */
	virtual bool RemapUserAndDeviceToControllerId(FPlatformUserId UserId, int32& OutControllerId, FInputDeviceId OptionalDeviceId = INPUTDEVICEID_NONE) = 0;
	
	/*
	 * Remap a FPlatformUserId to the legacy platform LocalUserIndex (often the same as ControllerId).
	 * This is useful in cases where the input device id is not needed.
	 *
	 * @param UserId			The platform user that should be converted
	 * @returns					A valid platform user index, or INDEX_NONE
	 */
	virtual int32 GetUserIndexForPlatformUser(FPlatformUserId UserId) = 0;

	/**
	 * Remap the legacy "int32 ControllerId" to the updated FPlatformUserId and FInputDeviceId. Use this function to
	 * add compatibility to platforms that may not have implemented this device mapper yet.
	 *
	 * This is useful for functions such as FGenericApplicationMessageHandler::OnControllerAnalog that used to
	 * use the "int32 ControllerId" as a parameter so that you can call the new FGenericApplicationMessageHandler
	 * that take in a PlatformUserId and an InputDeviceId.
	 * 
	 * @param ControllerId			The old plain "int32" that represented gamepad id or user index depending on the context.
	 * @param InOutUserId			If the old function provides a PlatformId then pass it here, otherwise pass PLATFORMUSERID_NONE.
	 * @param OutInputDeviceId		The best guess for an InputDeviceId based on the legacy int32 ControllerId. This may be INPUTDEVICEID_NONE.
	 * @return						True if this maps to a real user
	 */
	virtual bool RemapControllerIdToPlatformUserAndDevice(int32 ControllerId, FPlatformUserId& InOutUserId, FInputDeviceId& OutInputDeviceId) = 0;

	/*
	 * Remap the legacy platform LocalUserIndex (often the same as ControllerId) to a FPlatformUserId.
	 * This is useful in cases where the input device id is not needed.
	 *
	 * @param LocalUserIndex	A platform user index where 0 should always be a valid user
	 * @returns					A platform-allocated user id, or PLATFORMUSERID_NONE
	 */
	virtual FPlatformUserId GetPlatformUserForUserIndex(int32 LocalUserIndex) = 0;

	/** Allocates a new user id when a user becomes active, will return none if no more can be created */
	virtual FPlatformUserId AllocateNewUserId() = 0;

	/** Returns the next available input device id. This ID should be globally unique! */
	virtual FInputDeviceId AllocateNewInputDeviceId() = 0;
	
	APPLICATIONCORE_API virtual const EInputDeviceMappingPolicy GetCurrentDeviceMappingPolicy() const;
protected:

	/**
	* Binds to any core delegates that the platform may broadcast. By default it is only the OnUserLoginChangedEvent delegate. 
	* Called during the constructor of this device mapper.
	*/
	APPLICATIONCORE_API virtual void BindCoreDelegates();
	
	/** 
	* Unbind from any delegates that have been hooked into.
	* Called during the destructor of this device mapper.
	*/
	APPLICATIONCORE_API virtual void UnbindCoreDelegates();

	/** Callback for when FCoreDelegates::OnUserLoginChangedEvent is broadcasted */
	virtual void OnUserLoginChangedEvent(bool bLoggedIn, int32 UserId, int32 UserIndex) = 0;
	
	/**
	 * If true, this device mapper is operating in a backward compatible mode where there is a
	 * 1:1 mapping between controller id and user id
	 */
	virtual bool IsUsingControllerIdAsUserId() const = 0;

	/**
	 * If true, then when we remap the platform's raw int32 ControllerId to its FInputDeviceId
	 * and owning FPlatformUserId we should create a new platform user id if this controller id is
	 * not already mapped. 
	 */
	APPLICATIONCORE_API virtual bool ShouldCreateUniqueUserForEachDevice() const;

	/**
	 * If true, than this device mapper will broadcast the older 
	 * CoreDelegates as well as the new delegates. Set this to
	 * true if your platform needs calls from OnControllerConnectionChange or
	 * OnControllerPairingChange
	 */
	virtual bool ShouldBroadcastLegacyDelegates() const = 0;

	/**
	 * Callback which is fired right before the actual OnInputDeviceConnectionChange function for any systems which want to be notified
	 * before any game code is, if there is a valid FInputDeviceScope.
	 */
	static APPLICATIONCORE_API FOnInputDeviceHardwareInfoAvailable OnInputDeviceHardwareInfoAvailable;
	
	/** Callback when input devices are disconnected/reconnected */
	static APPLICATIONCORE_API FOnUserInputDeviceConnectionChange OnInputDeviceConnectionChange;

	/** Callback when an input device's owning platform user pairing changes */
	static APPLICATIONCORE_API FOnUserInputDevicePairingChange OnInputDevicePairingChange;
	
	/** A map of all input devices to their current state */
	TMap<FInputDeviceId, FPlatformInputDeviceState> MappedInputDevices;

	/** Highest used platform user id. Incremented in AllocateNewUserId and Internal_MapInputDeviceToUser by default. */
	FPlatformUserId LastPlatformUserId = PLATFORMUSERID_NONE;

	/** Highest used input device id. Incremented in AllocateNewInputDeviceId and Internal_MapInputDeviceToUser by default. */
	FInputDeviceId LastInputDeviceId = INPUTDEVICEID_NONE;

	/** Keeps track of any allocated platform user ID's by this device mapper. */
	TArray<FPlatformUserId> AllocatedPlatformUserIds;
};

/**
 * Generic implementation of the IPlatformInputDeviceMapper.
 * This provides the base functionality that can be used on most platforms.
 */
class FGenericPlatformInputDeviceMapper : public IPlatformInputDeviceMapper
{
public:

	APPLICATIONCORE_API FGenericPlatformInputDeviceMapper(const bool InbUsingControllerIdAsUserId, const bool InbShouldBroadcastLegacyDelegates);
	
	/** This is unsupported by default and will return PLATFORMUSERID_NONE on the generic platform */  
	APPLICATIONCORE_API virtual FPlatformUserId GetUserForUnpairedInputDevices() const override;
	APPLICATIONCORE_API virtual FPlatformUserId GetPrimaryPlatformUser() const override;
	APPLICATIONCORE_API virtual FInputDeviceId GetDefaultInputDevice() const override;

	APPLICATIONCORE_API virtual bool RemapControllerIdToPlatformUserAndDevice(int32 ControllerId, FPlatformUserId& InOutUserId, FInputDeviceId& OutInputDeviceId) override;
	APPLICATIONCORE_API virtual FPlatformUserId GetPlatformUserForUserIndex(int32 LocalUserIndex) override;
	APPLICATIONCORE_API virtual bool RemapUserAndDeviceToControllerId(FPlatformUserId UserId, int32& OutControllerId, FInputDeviceId OptionalDeviceId = INPUTDEVICEID_NONE) override;
	APPLICATIONCORE_API virtual int32 GetUserIndexForPlatformUser(FPlatformUserId UserId) override;
	APPLICATIONCORE_API virtual bool IsUsingControllerIdAsUserId() const override;
	APPLICATIONCORE_API virtual bool ShouldBroadcastLegacyDelegates() const override;

	/** Allocates a new user id when a user becomes active, will return none if no more can be created */
	APPLICATIONCORE_API virtual FPlatformUserId AllocateNewUserId() override;

	/** Returns the next available input device id */
	APPLICATIONCORE_API virtual FInputDeviceId AllocateNewInputDeviceId() override;
	
protected:

	/** 
	* Callback for when FCoreDelegates::OnUserLoginChangedEvent is broadcasted.
	* If the user has logged out, then remap any input devices that the user had to the "Unpaired" user on this platform (if bUnpairInputDevicesWhenLoggingOut is true)
	*/
	APPLICATIONCORE_API virtual void OnUserLoginChangedEvent(bool bLoggedIn, int32 RawPlatformUserId, int32 UserIndex) override;

	/** Flags for backwards compatibility with the older "int32 ControllerId" implementation */
	const bool bUsingControllerIdAsUserId = true;
	const bool bShouldBroadcastLegacyDelegates = true;

	/** Additional functionality that specializations can customize */
	bool bUnpairInputDevicesWhenLoggingOut = false;
};
