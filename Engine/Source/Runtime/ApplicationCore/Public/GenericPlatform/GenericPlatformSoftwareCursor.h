// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ICursor.h"
#include "Math/IntRect.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"

// Windows has special needs with RECT, and we don't need this class on Windows anyway, so just skip it
#if !PLATFORM_WINDOWS

class FGenericPlatformSoftwareCursor : public ICursor
{
public:

	FGenericPlatformSoftwareCursor();

	virtual ~FGenericPlatformSoftwareCursor() = default;

	virtual FVector2D GetPosition() const override
	{
		return CurrentPosition;
	}

	virtual void SetPosition(const int32 X, const int32 Y) override;

	virtual void SetType(const EMouseCursor::Type InNewCursor) override;

	virtual EMouseCursor::Type GetType() const override
	{
		return CurrentType;
	}

	virtual void GetSize(int32& Width, int32& Height) const override;

	virtual void Show(bool bInShow) override;

	virtual void Lock(const RECT* const Bounds) override;

	virtual void SetTypeShape(EMouseCursor::Type InCursorType, void* CursorHandle) override { }

private:
	bool UpdateCursorClipping(FVector2D& CursorPosition);

	EMouseCursor::Type CurrentType;
	FVector2D CurrentPosition;
	FIntRect CursorClipRect;
	bool bShow;
};

#endif
