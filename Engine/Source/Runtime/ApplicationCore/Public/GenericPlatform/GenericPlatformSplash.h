// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "CoreTypes.h"
#include "Math/IntPoint.h"

struct FPlatformRect;

/**
 * SplashTextType defines the types of text on the splash screen
 */
namespace SplashTextType
{
	enum Type
	{
		/** Startup progress text */
		StartupProgress	= 0,

		/** Version information text line 1 */
		VersionInfo1,

		/** Copyright information text */
		CopyrightInfo,

		/** Game Name */
		GameName,

		// ...

		/** Licensee-usable slots */
		User1,
		User2,
		User3,
		User4,
		User5,
		User6,

		// Only add further User types here, the rest should go above this block

		/** Number of text types (must be final enum value) */
		NumTextTypes
	};
}

/**
 * SplashImageType defines the types of images on the splash screen
 */
namespace SplashImageType
{
	enum Type
	{
		/** The main background image */
		Background,

		// ...

		/** Licensee-usable slots */
		User1,
		User2,
		User3,
		User4,
		User5,
		User6,

		// Only add further User types here, the rest should go above this block

		/** Number of image types (must be final enum value) */
		NumImageTypes
	};
}

enum class ESplashTextSizeType : uint8
{
	Small,
	Normal,
	Title,
};

enum class ESplashTextBrightnessType : uint8
{
	Low,
	Medium,
	High,
};

enum class ESplashTextFlags : uint8
{
	None,
	WordWrap = 1 << 0,
};
ENUM_CLASS_FLAGS(ESplashTextFlags)

class FString;
class FText;

/**
 * Generic implementation for most platforms
 */
struct FGenericPlatformSplash
{
	/** Show the splash screen. */
	inline static void Show() { }

	/** Hide the splash screen. */
	inline static void Hide() { }

	/**
	 * Sets a custom splash image to display
	 * 
	 * @param SplashFilename Full path to the splash image to display
	 */
	static APPLICATIONCORE_API void SetCustomSplashImage(const TCHAR* SplashFilename);


	/**
	 * Sets the progress displayed on the application icon (for startup/loading progress).
	 *
	 * @param ProgressPercent Progress value in percent.
	 */
	inline static void SetProgress(int ProgressPercent)
	{
	
	}

	/**
	 * Sets the text displayed on the splash screen (for startup/loading progress)
	 *
	 * @param	InType		Type of text to change
	 * @param	InText		Text to display
	 */
	UE_DEPRECATED(5.6, "Use the overload of FPlatformSlash::SetSplashText that takes a FText")
	static APPLICATIONCORE_API void SetSplashText( const SplashTextType::Type InType, const TCHAR* InText );
	inline static void SetSplashText(const SplashTextType::Type InType, const FText& InText)
	{
	}

	/** Sets the image to be assigned to the specific slot */
	inline static void SetSplashImage(const SplashImageType::Type InType, const TCHAR* InFilename) { }

	/** Registers a new licensee text. Only User types are allowed, and only before the splash screen is initialized! */
	inline static void RegisterUserSplashText(const SplashTextType::Type InType, const FPlatformRect& InRect, ESplashTextSizeType InSize, ESplashTextBrightnessType InBrightness, ESplashTextFlags Flags = ESplashTextFlags::None) { }

	/** Registers a new licensee image. Only User types are allowed, and only before the splash screen is initialized! */
	inline static void RegisterUserSplashImage(const SplashImageType::Type InType, const FIntPoint& InPos, const TCHAR* InFilename) { }

	/**
	 * Return whether the splash screen is being shown or not
	 */
	inline static bool IsShown()
	{
		return true;
	}

protected:
	/**
	* Finds a usable splash pathname for the given filename
	*
	* @param SplashFilename Name of the desired splash name("Splash")
	* @param IconFilename Name of the desired icon name("Splash")
	* @param OutPath String containing the path to the file, if this function returns true
	* @param OutIconPath String containing the path to the icon, if this function returns true
	*
	* @return true if a splash screen was found
	*/
	static APPLICATIONCORE_API bool GetSplashPath(const TCHAR* SplashFilename, FString& OutPath, bool& OutIsCustom, bool bAllowCustom = true);
	static APPLICATIONCORE_API bool GetSplashPath(const TCHAR* SplashFilename, const TCHAR* IconFilename, FString& OutPath, FString& OutIconPath, bool& OutIsCustom);
};
