// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Logging/LogMacros.h"

APPLICATIONCORE_API DECLARE_LOG_CATEGORY_EXTERN(LogSynthBenchmark, Log, All);

// time and amount of work that was measured
struct FTimeSample
{
	FTimeSample(float InTotalTime, float InNormalizedTime)
		: TotalTime(InTotalTime)
		, NormalizedTime(InNormalizedTime)
	{
	}

	// in seconds, >=0
	float TotalTime;
	// Time / WorkScale, WorkScale might have been adjusted (e.g. quantized), >0
	float NormalizedTime;
};


struct FSynthBenchmarkStat
{
	FSynthBenchmarkStat()
		: Desc(0)
		, MeasuredTotalTime(-1)
		, MeasuredNormalizedTime(-1)
		, IndexNormalizedTime(-1)
		, ValueType(0)
		, Confidence(0)
		, Weight(1.f)
	{ }

	// @param InDesc descriptions
	FSynthBenchmarkStat(const TCHAR* InDesc, float InIndexNormalizedTime, const TCHAR* InValueType, float InWeight)
		: Desc(InDesc)
		, MeasuredTotalTime(-1)
		, MeasuredNormalizedTime(-1)
		, IndexNormalizedTime(InIndexNormalizedTime)
		, ValueType(InValueType)
		, Confidence(0)
		, Weight(InWeight)
	{
//		check(InDesc);
//		check(InValueType);
	}

	// Computes the linear performance index (>0), around 100 with good hardware but higher numbers are possible
	float ComputePerfIndex() const
	{
		return 100.0f * IndexNormalizedTime / MeasuredNormalizedTime;
	}

	// @param TimeSample seconds and normalized time (e.g. seconds / GPixels)
	// @param InConfidence 0..100
	void SetMeasuredTime(const FTimeSample& TimeSample, float InConfidence = 90)
	{
//		check(InConfidence >= 0.0f && InConfidence <= 100.0f);
		MeasuredTotalTime = TimeSample.TotalTime;
		MeasuredNormalizedTime = TimeSample.NormalizedTime;
		Confidence = InConfidence;
	}

	// @return can be 0 
	const TCHAR* GetDesc() const
	{
		return Desc;
	}

	// @return can be 0 
	const TCHAR* GetValueType() const
	{
		return ValueType;
	}

	float GetNormalizedTime() const
	{
		return MeasuredNormalizedTime;
	}

	float GetMeasuredTotalTime() const
	{
		return MeasuredTotalTime;
	}

	// @return 0=no..100=full
	float GetConfidence() const
	{
		return Confidence;
	}

	float GetWeight() const
	{
		return Weight;
	}

private:

	// 0 if not valid
	const TCHAR *Desc;
	// -1 if not defined, in seconds, useful to see if a test did run too long (some slower GPUs might timeout)
	float MeasuredTotalTime;
	// -1 if not defined, depends on the test (e.g. s/GPixels), WorkScale is divided out
	float MeasuredNormalizedTime;
	// -1 if not defined, timing value expected on a norm GPU (index value 100, here NVidia 670)
	float IndexNormalizedTime;
	// 0 if not valid
	const TCHAR *ValueType;
	// 0..100, 100: fully confident
	float Confidence;
	// 1 is normal weight, 0 is no weight, > 1 is unbounded additional weight
	float Weight;
};


struct FSynthBenchmarkResults 
{
	FSynthBenchmarkStat CPUStats[2];
	FSynthBenchmarkStat GPUStats[7];

	// 100: avg good CPU, <100:slower, >100:faster
	APPLICATIONCORE_API float ComputeCPUPerfIndex(TArray<float>* OutIndividualResults = nullptr) const;

	// 100: avg good GPU, <100:slower, >100:faster
	APPLICATIONCORE_API float ComputeGPUPerfIndex(TArray<float>* OutIndividualResults = nullptr) const;

	// @return in seconds, useful to check if a benchmark takes too long (very slow hardware, don't make tests with large WorkScale)
	float ComputeTotalGPUTime() const
	{
		float Ret = 0.0f;
		for(uint32 i = 0; i < sizeof(GPUStats) / sizeof(GPUStats[0]); ++i)
		{
			Ret += GPUStats[i].GetMeasuredTotalTime();
		}

		return Ret;
	}
};


struct FHardwareDisplay	
{
	static const uint32 MaxStringLength = 260;

	uint32 CurrentModeWidth;
	uint32 CurrentModeHeight;

	TCHAR GPUCardName_DEPRECATED[MaxStringLength];
	uint32 GPUDedicatedMemoryMB_DEPRECATED;
	TCHAR GPUDriverVersion_DEPRECATED[MaxStringLength];
};


struct FGPUAdpater	
{
	static const uint32 MaxStringLength = 260;

	TCHAR AdapterName[MaxStringLength];
	//
	TCHAR AdapterInternalDriverVersion[MaxStringLength];
	// extra data when RHI starts up, AMD has the Catalyst Driver version which is more useful for the user, otherwise it's the same as GRHIAdapterInternalDriverVersion
	TCHAR AdapterUserDriverVersion[MaxStringLength];
	// extra data when RHI starts up
	TCHAR AdapterDriverDate[MaxStringLength];
	//
	TCHAR AdapterDedicatedMemoryMB[MaxStringLength];
};


struct FHardwareSurveyResults
{
	static const int32 MaxDisplayCount = 8; 
	static const int32 MaxStringLength = 260;

	TCHAR Platform[MaxStringLength];

	TCHAR OSVersion[MaxStringLength];
	TCHAR OSSubVersion[MaxStringLength];
	uint32 OSBits;
	TCHAR OSLanguage[MaxStringLength];

	TCHAR RenderingAPI[MaxStringLength];
	TCHAR MultimediaAPI_DEPRECATED[MaxStringLength];

	uint32 HardDriveGB;
	uint32 HardDriveFreeMB;
	uint32 MemoryMB;

	float CPUPerformanceIndex;
	float GPUPerformanceIndex;
	float RAMPerformanceIndex;

	uint32 bIsLaptopComputer:1;
	uint32 bIsRemoteSession:1;

	uint32 CPUCount;
	float CPUClockGHz;
	TCHAR CPUBrand[MaxStringLength];
	TCHAR CPUNameString[MaxStringLength];
	uint32 CPUInfo;

	uint32 DisplayCount;
	FHardwareDisplay Displays[MaxDisplayCount];
	// RHI chosen adapter, only if RHI started on Windows DX11
	FGPUAdpater RHIAdapter;

	uint32 ErrorCount;
	TCHAR LastSurveyError[MaxStringLength];
	TCHAR LastSurveyErrorDetail[MaxStringLength];
	TCHAR LastPerformanceIndexError[MaxStringLength];
	TCHAR LastPerformanceIndexErrorDetail[MaxStringLength];

	FSynthBenchmarkResults SynthBenchmark;
};


/**
* Generic implementation for most platforms, these tend to be unused and unimplemented
**/
struct FGenericPlatformSurvey
{
	/**
	 * Attempt to get hardware survey results now.
	 * If they aren't available the survey will be started/ticked until complete.
	 *
	 * @param OutResults	The struct that receives the results if available.
	 * @param bWait			If true, the function won't return until the results are available or the survey fails. Defaults to false.
	 * @return				True if the results were available, false otherwise.
	 */
	static bool GetSurveyResults( FHardwareSurveyResults& OutResults, bool bWait = false )
	{
		return false;
	}
};
