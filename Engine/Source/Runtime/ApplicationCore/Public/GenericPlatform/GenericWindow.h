// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/MathFwd.h"
#include "Templates/SharedPointer.h"

struct FGenericWindowDefinition;

/**
 * Modes that an FGenericWindow can be in
 */
namespace EWindowMode
{
	enum Type : int
	{
		/** The window is in true fullscreen mode */
		Fullscreen,
		/** The window has no border and takes up the entire area of the screen */
		WindowedFullscreen,
		/** The window has a border and may not take up the entire screen area */
		Windowed,

		/** The total number of supported window modes */
		NumWindowModes
	};

	inline Type ConvertIntToWindowMode(int32 InWindowMode)
	{
		Type WindowMode = Windowed;
		switch (InWindowMode)
		{
		case 0:
			WindowMode = Fullscreen;
			break;
		case 1:
			WindowMode = WindowedFullscreen;
			break;
		case 2:
		default:
			WindowMode = Windowed;
			break;
		}
		return WindowMode;
	}
}

inline const TCHAR* LexToString(EWindowMode::Type WindowMode)
{
	switch (WindowMode)
	{
	case EWindowMode::Fullscreen: return TEXT("Fullscreen");
	case EWindowMode::WindowedFullscreen: return TEXT("WindowedFullscreen");
	case EWindowMode::Windowed: return TEXT("Windowed");
	case EWindowMode::NumWindowModes:
	default: return TEXT("Unknown");
	}
}

enum class EWindowDrawAttentionRequestType
{
	/**
	 * Indicates that the attention-drawing behavior continues until the
	 * application or window is activated.
	 */
	UntilActivated,

	/**
	 * Indicates that the attention-drawing behavior, if any, should stop.
	 */
	Stop,
};

/**
 * Parameters for drawing attention to a window or application. Some
 * parameters may only be used on certain platforms.
 */
struct FWindowDrawAttentionParameters
{
	FWindowDrawAttentionParameters() = default;

	explicit FWindowDrawAttentionParameters(
		EWindowDrawAttentionRequestType InRequestType
	)
		: RequestType(InRequestType)
	{
	}

	EWindowDrawAttentionRequestType RequestType = EWindowDrawAttentionRequestType::UntilActivated;
};

class FGenericWindow
{
public:

	APPLICATIONCORE_API FGenericWindow();

	APPLICATIONCORE_API virtual ~FGenericWindow();

	/** Native windows should implement ReshapeWindow by changing the platform-specific window to be located at (X,Y) and be the dimensions Width x Height. */
	APPLICATIONCORE_API virtual void ReshapeWindow( int32 X, int32 Y, int32 Width, int32 Height );

	/** Returns the rectangle of the screen the window is associated with */
	APPLICATIONCORE_API virtual bool GetFullScreenInfo( int32& X, int32& Y, int32& Width, int32& Height ) const;

	/** Native windows should implement MoveWindowTo by relocating the platform-specific window to (X,Y). */
	APPLICATIONCORE_API virtual void MoveWindowTo ( int32 X, int32 Y );

	/** Native windows should implement BringToFront by making this window the top-most window (i.e. focused). */
	APPLICATIONCORE_API virtual void BringToFront( bool bForce = false );

	/** @hack Force a window to front even if a different application is in front. */
	APPLICATIONCORE_API virtual void HACK_ForceToFront();

	/** Native windows should implement this function by asking the OS to destroy OS-specific resource associated with the window (e.g. Win32 window handle) */
	APPLICATIONCORE_API virtual void Destroy();

	/** Native window should implement this function by performing the equivalent of the Win32 minimize-to-taskbar operation */
	APPLICATIONCORE_API virtual void Minimize();

	/** Native window should implement this function by performing the equivalent of the Win32 maximize operation */
	APPLICATIONCORE_API virtual void Maximize();

	/** Native window should implement this function by performing the equivalent of the Win32 restore operation */
	APPLICATIONCORE_API virtual void Restore();

	/** Native window should make itself visible */
	APPLICATIONCORE_API virtual void Show();

	/** Native window should hide itself */
	APPLICATIONCORE_API virtual void Hide();

	/** Toggle native window between fullscreen and normal mode */
	APPLICATIONCORE_API virtual void SetWindowMode( EWindowMode::Type InNewWindowMode );

	/** @return true if the native window is currently in fullscreen mode, false otherwise */
	APPLICATIONCORE_API virtual EWindowMode::Type GetWindowMode() const;

	/** @return true if the native window is maximized, false otherwise */
	APPLICATIONCORE_API virtual bool IsMaximized() const;

	/** @return true if the native window is minimized, false otherwise */
	APPLICATIONCORE_API virtual bool IsMinimized() const;

	/** @return true if the native window is visible, false otherwise */
	APPLICATIONCORE_API virtual bool IsVisible() const;

	/**
	 * Populates the size and location of the window when it is restored.
	 * If the function fails, false is returned and X,Y,Width,Height will be undefined.
	 *
	 * @return true when the size and location and successfully retrieved; false otherwise.
	 */
	APPLICATIONCORE_API virtual bool GetRestoredDimensions(int32& X, int32& Y, int32& Width, int32& Height);

	/** 
	 * Native windows should implement SetWindowFocus to let the OS know that a window has taken focus.  
	 * Slate handles focus on a per widget basis internally but the OS still needs to know what window has focus for proper message routing
	 */
	APPLICATIONCORE_API virtual void SetWindowFocus();

	/**
	 * Sets the opacity of this window
	 *
	 * @param	InOpacity	The new window opacity represented as a floating point scalar
	 */
	APPLICATIONCORE_API virtual void SetOpacity( const float InOpacity );

	/** 
	 * Enables or disables the window.  If disabled the window receives no input
	 * 
	 * @param bEnable	true to enable the window, false to disable it.
	 */
	APPLICATIONCORE_API virtual void Enable( bool bEnable );

	/** @return true if native window exists underneath the coordinates */
	APPLICATIONCORE_API virtual bool IsPointInWindow( int32 X, int32 Y ) const;
	
	/** Gets OS specific window border size. This is necessary because Win32 does not give control over this size. */
	APPLICATIONCORE_API virtual int32 GetWindowBorderSize() const;

	/** Gets OS specific window title bar size */
	APPLICATIONCORE_API virtual int32 GetWindowTitleBarSize() const;

	/** Gets the OS Window handle in the form of a void pointer for other API's */
	APPLICATIONCORE_API virtual void* GetOSWindowHandle() const;

	/** @return true if the window is in the foreground */
	APPLICATIONCORE_API virtual bool IsForegroundWindow() const;

	/** @return true if the window supports dynamic cursor locking */
	APPLICATIONCORE_API virtual bool SupportsDynamicCursorLock() const;

	/** @return true if the window supports fullscreen */
	APPLICATIONCORE_API virtual bool IsFullscreenSupported() const;

	/**
	 * Sets the window text - usually the title but can also be text content for things like controls
	 *
	 * @param Text	The window's title or content text
	 */
	APPLICATIONCORE_API virtual void SetText(const TCHAR* const Text);

	/** @return	The definition describing properties of the window */
	APPLICATIONCORE_API virtual const FGenericWindowDefinition& GetDefinition() const;

	/** @return	Returns true if the window definition is valid */
	APPLICATIONCORE_API virtual bool IsDefinitionValid() const;

	/** @return	Gives the native window a chance to adjust our stored window size before we cache it off */
	APPLICATIONCORE_API virtual void AdjustCachedSize( FVector2D& Size ) const;

	/**
	 * @return ratio of pixels to SlateUnits in this window.
	 * E.g. DPIScale of 2.0 means there is a 2x2 pixel square for every 1x1 SlateUnit.
	 */
	APPLICATIONCORE_API virtual float GetDPIScaleFactor() const;

	/** sets a new DPI scale factor */
	APPLICATIONCORE_API virtual void SetDPIScaleFactor(const float Factor);

	/** determines whether or not this window does its own DPI management */
	APPLICATIONCORE_API virtual bool IsManualManageDPIChanges() const;

	/** call with a true argument if this window need to do its custom size management in response to DPI variations */
	APPLICATIONCORE_API virtual void SetManualManageDPIChanges(const bool bAutoHandle);

	/**
	 * Attempts to draw the user's attention to this window in whatever way is
	 * appropriate for the platform if this window is not the current active
	 * window.
	 *
	 * @param Parameters The parameters for drawing attention. Depending on
	 *        the platform, not all parameters may be supported.
	 */
	APPLICATIONCORE_API virtual void DrawAttention(const FWindowDrawAttentionParameters& Parameters);

	/** Shows or hides native window buttons on platforms that use them */
	APPLICATIONCORE_API virtual void SetNativeWindowButtonsVisibility(bool bVisible);

	/** Returns true if this window will not be used for presenting rendered frames to a display. */
	APPLICATIONCORE_API virtual bool IsRenderingOffScreen() const;

protected:

	TSharedPtr< FGenericWindowDefinition > Definition;
};
