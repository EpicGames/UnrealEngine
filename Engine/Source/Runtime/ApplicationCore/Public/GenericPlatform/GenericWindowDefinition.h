// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"


#ifndef ALPHA_BLENDED_WINDOWS
	#define ALPHA_BLENDED_WINDOWS IS_PROGRAM || WITH_EDITOR
#endif


/** Enumeration to specify different window types for SWindows */
enum class EWindowType
{
	/** Value indicating that this is a standard, general-purpose window */
	Normal,
	/** Value indicating that this is a window used for a popup menu */
	Menu,
	/** Value indicating that this is a window used for a tooltip */
	ToolTip,
	/** Value indicating that this is a window used for a notification toast */
	Notification,
	/** Value indicating that this is a window used for a cursor decorator */
	CursorDecorator,

	GameWindow UE_DEPRECATED(5.8, "There is no longer any meaningful distinction between game and normal window. Use Normal instead. Windows now opt in to the Slate Console Debugger using a boolean property.")
};


/** Enumeration to specify different transparency options for SWindows */
enum class EWindowTransparency
{
	/** Value indicating that a window does not support transparency */
	None,

	/** Value indicating that a window supports transparency at the window level (one opacity applies to the entire window) */
	PerWindow,

#if ALPHA_BLENDED_WINDOWS
	/** Value indicating that a window supports per-pixel alpha blended transparency */
	PerPixel,
#endif
};


/** Enumeration to specify whether the window gets activated upon showing it */
enum class EWindowActivationPolicy
{
	/** Value indicating that a window never activates when it is shown */
	Never,

	/** Value indicating that a window always activates when it is shown */
	Always,

	/** Value indicating that a window only activates when it is first shown */
	FirstShown
};


struct FGenericWindowDefinition
{
	/** Window type */
	EWindowType Type;
	
	/** The initially desired horizontal screen position */
	float XDesiredPositionOnScreen;
	/** The initially desired vertical screen position */
	float YDesiredPositionOnScreen;

	/** The initially desired width */
	float WidthDesiredOnScreen;
	/** The initially desired height */
	float HeightDesiredOnScreen;

	/** the level of transparency supported by this window */
	EWindowTransparency TransparencySupport;

	/** true if the window is using the os window border instead of a slate created one */
	bool HasOSWindowBorder;
	/** should this window show up in the taskbar */
	bool AppearsInTaskbar;
	/** true if the window should be on top of all other windows; false otherwise */
	bool IsTopmostWindow;
	/** true if the window accepts input; false if the window is non-interactive */
	bool AcceptsInput;
	/** the policy for activating the window upon each show */
	EWindowActivationPolicy ActivationPolicy;
	/** true if this window will be focused when it is first shown */
	bool FocusWhenFirstShown;
	/** true if this window displays an enabled close button on the toolbar area */
	bool HasCloseButton;
	/** true if this window displays an enabled minimize button on the toolbar area */
	bool SupportsMinimize;
	/** true if this window displays an enabled maximize button on the toolbar area */
	bool SupportsMaximize;

	/** true if the window is modal (prevents interacting with its parent) */
	bool IsModalWindow;
	/** true if this is a vanilla window, or one being used for some special purpose: e.g. tooltip or menu */
	bool IsRegularWindow;
	/** true if this is a user-sized window with a thick edge */
	bool HasSizingFrame;
	/** true if we expect the size of this window to change often, such as if its animated, or if it recycled for tool-tips. */
	bool SizeWillChangeOften;
	/** true if the window should preserve its aspect ratio when resized by user */
	bool ShouldPreserveAspectRatio;
	/** The expected maximum width of the window.  May be used for performance optimization when SizeWillChangeOften is set. */
	int32 ExpectedMaxWidth;
	/** The expected maximum height of the window.  May be used for performance optimization when SizeWillChangeOften is set. */
	int32 ExpectedMaxHeight;

	/** the title of the window */
	FString Title;
	/** opacity of the window (0-1) */
	float Opacity;
	/** the radius of the corner rounding of the window */
	int32 CornerRadius;

	FWindowSizeLimits SizeLimits;

	/** false if the window should respond to system DPI changes, otherwise this will be handled, internally, by the application */
	bool bManualDPI = false;
};
