// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Vector2D.h"
#include "UObject/CoreReflectedTypeAccessors.h"

struct tagRECT;
typedef struct tagRECT RECT;


/**
 * Mouse cursor types
 */
namespace EMouseCursor
{
	enum Type : int
	{
		/** Causes no mouse cursor to be visible */
		None,

		/** Default cursor (arrow) */
		Default,

		/** Text edit beam */
		TextEditBeam,

		/** Resize horizontal */
		ResizeLeftRight,

		/** Resize vertical */
		ResizeUpDown,

		/** Resize diagonal */
		ResizeSouthEast,

		/** Resize other diagonal */
		ResizeSouthWest,

		/** MoveItem */
		CardinalCross,

		/** Target Cross */
		Crosshairs,

		/** Hand cursor */
		Hand,

		/** Grab Hand cursor */
		GrabHand,

		/** Grab Hand cursor closed */
		GrabHandClosed,

		/** a circle with a diagonal line through it */
		SlashedCircle,

		/** Eye-dropper cursor for picking colors */
		EyeDropper,

		/** Custom cursor shape for platforms that support setting a native cursor shape. Same as specifying None if not set. */
		Custom,

		/** Number of cursors we support */
		TotalCursorCount
	};
}

#ifdef COREUOBJECT_API
template<> COREUOBJECT_API UEnum* StaticEnum<EMouseCursor::Type>();
#endif

class ICursor
{
public:
	virtual ~ICursor() = default;

	/** Creates a hardware cursor from file. Can return nullptr when not available. */
	virtual void* CreateCursorFromFile(const FString& InPathToCursorWithoutExtension, FVector2D HotSpot) { return nullptr; }

	/** Is CreateCursorFromRGBABuffer() supported by this cursor? */
	virtual bool IsCreateCursorFromRGBABufferSupported() const { return false; }

	/** Creates a hardware cursor from bitmap data. Can return nullptr when not available. */
	virtual void* CreateCursorFromRGBABuffer(const FColor* Pixels, int32 Width, int32 Height, FVector2D InHotSpot) { return nullptr; }

	/** The position of the cursor */
	virtual FVector2D GetPosition() const = 0;

	/** Sets the position of the cursor */
	virtual void SetPosition( const int32 X, const int32 Y ) = 0;

	/** Sets the cursor */
	virtual void SetType( const EMouseCursor::Type InNewCursor ) = 0;

	/** Gets the current type of the cursor */
	virtual EMouseCursor::Type GetType() const = 0;

	/** Gets the size of the cursor */
	virtual void GetSize( int32& Width, int32& Height ) const = 0;

	/**
	 * Shows or hides the cursor
	 *
	 * @param bShow	true to show the mouse cursor, false to hide it
	 */
	virtual void Show( bool bShow ) = 0;

	/**
	 * Perform operations that can't be done on an event-basis
	 */
	virtual void Tick() {}

	/**
	 * Locks the cursor to the passed in bounds
	 * 
	 * @param Bounds	The bounds to lock the cursor to.  Pass NULL to unlock.
	 */
	virtual void Lock( const RECT* const Bounds ) = 0;

	/**
	 * Allows overriding the shape of a particular cursor.
	 */
	virtual void SetTypeShape(EMouseCursor::Type InCursorType, void* CursorHandle) = 0;
};
