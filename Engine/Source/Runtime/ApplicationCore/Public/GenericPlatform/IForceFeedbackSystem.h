// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * DEPRECATED GenericPlatform/IForceFeedbackSystem.h. Use GenericPlatform/IInputInterface.h instead
 */
#pragma once

// HEADER_UNIT_SKIP - Deprecated

UE_DEPRECATED_HEADER(5.1, "Use GenericPlatform / IInputInterface.h instead of GenericPlatform / IForceFeedbackSystem.h.")

#include "GenericPlatform/IInputInterface.h"


/**
 * Interface for the force feedback system.
 *
 * Note: This class is deprecated and will be removed in favor of IInputInterface
 */
UE_DEPRECATED(5.1, "IForceFeedbackSystem has been deprecated, use IInputInterface instead.")
class IForceFeedbackSystem
	: public IInputInterface
{
public:

	virtual ~IForceFeedbackSystem() = default;
};
