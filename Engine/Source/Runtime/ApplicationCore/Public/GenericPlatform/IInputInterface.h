// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Math/Color.h"
#include "UObject/NameTypes.h"
#include "UObject/CoreReflectedTypeAccessors.h"


// General identifiers for potential force feedback channels. These will be mapped according to the
// platform specific implementation.
// For example, Platform A only listens to the XXX_LARGE channels and ignores the rest, while Platform B could
// map the XXX_LARGE to the handle motors and XXX_SMALL to the trigger motors. And others can map LEFT_SMALL to
// its single motor.
enum class FForceFeedbackChannelType
{
	LEFT_LARGE,
	LEFT_SMALL,
	RIGHT_LARGE,
	RIGHT_SMALL
};


struct FForceFeedbackValues
{
	float LeftLarge;
	float LeftSmall;
	float RightLarge;
	float RightSmall;

	FForceFeedbackValues()
		: LeftLarge(0.f)
		, LeftSmall(0.f)
		, RightLarge(0.f)
		, RightSmall(0.f)
	{ }
};

struct FHapticFeedbackBuffer
{
	const uint8* RawData;
	uint32 CurrentPtr;
	int BufferLength;
	int SamplesSent;
	bool bFinishedPlaying;
	int SamplingRate;
	float ScaleFactor;
	bool bUseStereo;
	uint32 CurrentSampleIndex[2];

	FHapticFeedbackBuffer()
		: CurrentPtr(0)
		, BufferLength(0)
		, SamplesSent(0)
		, bFinishedPlaying(false)
		, SamplingRate(0)
		, bUseStereo(false)
		, CurrentSampleIndex{}
	{
	}

	bool NeedsUpdate() const
	{
		return !bFinishedPlaying;
	}
};

struct FHapticFeedbackValues
{
	float Frequency;
	float Amplitude;

	FHapticFeedbackBuffer* HapticBuffer;

	FHapticFeedbackValues()
		: Frequency(0.f)
		, Amplitude(0.f)
		, HapticBuffer(NULL)
	{
	}

	FHapticFeedbackValues(const float InFrequency, const float InAmplitude)
	{
		// can't use FMath::Clamp here due to header files dependencies
		Frequency = (InFrequency < 0.f) ? 0.f : ((InFrequency > 1.f) ? 1.f : InFrequency);
		Amplitude = (InAmplitude < 0.f) ? 0.f : ((InAmplitude > 1.f) ? 1.f : InAmplitude);
		HapticBuffer = NULL;
	}
};

/**
 * Represents input device triggers that are available
 *
 * NOTE: Make sure to keep this type in sync with the reflected version in NoExportTypes.h!
 */
enum class EInputDeviceTriggerMask : uint8
{
	None		= 0x00,
	Left		= 0x01,
	Right		= 0x02,
	All			= Left | Right
};
ENUM_CLASS_FLAGS(EInputDeviceTriggerMask)

#ifdef COREUOBJECT_API
template<> COREUOBJECT_API UEnum* StaticEnum<EInputDeviceTriggerMask>();
#endif

/**
 * Represents input device analog sticks that are available
 *
 * NOTE: Make sure to keep this type in sync with the reflected version in NoExportTypes.h!
 */
enum class EInputDeviceAnalogStickMask : uint8
{
	None = 0x00,
	Left = 0x01,
	Right = 0x02
};
ENUM_CLASS_FLAGS(EInputDeviceAnalogStickMask)

#ifdef COREUOBJECT_API
template<> COREUOBJECT_API UEnum* StaticEnum<EInputDeviceAnalogStickMask>();
#endif

struct FInputDeviceProperty
{
	FInputDeviceProperty(FName InName)
		: Name(InName)
	{}

	FName Name;
};

/** A generic gyro calibration property for input devices whose gyroscopes (angular velocity sensors) can have their calibration enabled or disabled */
struct FInputDeviceGyroAutoCalibrationProperty : public FInputDeviceProperty
{
	FInputDeviceGyroAutoCalibrationProperty()
		: FInputDeviceProperty(PropertyName())
	{}

	FInputDeviceGyroAutoCalibrationProperty(bool bInEnable)
		: FInputDeviceProperty(PropertyName())
		, bEnable(bInEnable)
	{}

	static FName PropertyName() { return FName("InputDeviceGyroAutoCalibration"); }

	/** If gyro auto calibration should be enabled. */
	bool bEnable = true;
};

/** A generic light color property for input devices that have lights on them */
struct FInputDeviceLightColorProperty : public FInputDeviceProperty
{
	FInputDeviceLightColorProperty()
		: FInputDeviceProperty(PropertyName())
	{}

	FInputDeviceLightColorProperty(bool bInEnable, FColor InColor)
		: FInputDeviceProperty(PropertyName())
		, bEnable(bInEnable)
		, Color(InColor)
	{}

	static FName PropertyName() { return FName("InputDeviceLightColor"); }

	/** If the light should be enabled or not. */
	bool bEnable = true;
	
	/** The color to set the light */
	FColor Color = FColor::White;
}; 

/** Base class for device properties that affect Triggers */
struct FInputDeviceTriggerProperty : public FInputDeviceProperty
{
	FInputDeviceTriggerProperty(FName InName)
		: FInputDeviceProperty(InName)
	{}

	/** Which trigger this property should effect */
	EInputDeviceTriggerMask AffectedTriggers = EInputDeviceTriggerMask::None;
};

/** This property can be used to reset the state of a given trigger. */
struct FInputDeviceTriggerResetProperty : public FInputDeviceTriggerProperty
{
	FInputDeviceTriggerResetProperty()
		: FInputDeviceTriggerProperty(PropertyName())
	{
		AffectedTriggers = EInputDeviceTriggerMask::All;
	}

	static FName PropertyName() { return FName("InputDeviceTriggerResetProperty"); }
};

/** Trigger resistance that is applied at a single position with the given strength. */
struct FInputDeviceTriggerFeedbackProperty : public FInputDeviceTriggerProperty
{
	FInputDeviceTriggerFeedbackProperty()
		: FInputDeviceTriggerProperty(PropertyName())
	{}

	static FName PropertyName() { return FName("InputDeviceTriggerFeedback"); }

	int32 Position = 0;

	int32 Strengh = 0;
};

/** A generic trigger effect that allows analog triggers to have a resistance curve between two points (Start and End) */
struct FInputDeviceTriggerResistanceProperty : public FInputDeviceTriggerProperty
{
	FInputDeviceTriggerResistanceProperty()
		: FInputDeviceTriggerProperty(PropertyName())
	{}

	static FName PropertyName() { return FName("InputDeviceTriggerResistance"); }

	/** The position that the trigger should start providing resistance */
	int32 StartPosition = 0;

	/** How strong the resistance is */
	int32 StartStrengh = 0;

	/** The position that the trigger should start providing resistance */
	int32 EndPosition = 0;

	/** How strong the resistance is */
	int32 EndStrengh = 0;
};

/** A generic input device property that sets vibration on triggers */
struct FInputDeviceTriggerVibrationProperty : public FInputDeviceTriggerProperty
{
	FInputDeviceTriggerVibrationProperty()
		: FInputDeviceTriggerProperty(PropertyName())
	{}

	static FName PropertyName() { return FName("InputDeviceTriggerVibration"); }

	int32 TriggerPosition = 0;

	int32 VibrationFrequency = 0;

	int32 VibrationAmplitude = 0;
};

/** Base class for device properties that affect Analog Sticks */
struct FInputDeviceAnalogStickProperty : public FInputDeviceProperty
{
	FInputDeviceAnalogStickProperty(FName InName)
		: FInputDeviceProperty(InName)
	{}

	FInputDeviceAnalogStickProperty(FName InName, EInputDeviceAnalogStickMask InAffectedStick)
		: FInputDeviceProperty(InName)
		, AffectedStick(InAffectedStick)
	{}

	/** Which stick this property should effect */
	EInputDeviceAnalogStickMask AffectedStick = EInputDeviceAnalogStickMask::None;
};

/**
 * A generic input device property that sets Deadzone for Analog Sticks.
 * 
 * NOTE: Not all input device libraries do their own deadzone filtering, so the game must still do it.
 * This is just to tell those device libraries that do some filtering not to ignore inputs of this size (circular) or larger.
 */
struct FInputDeviceAnalogStickDeadZoneProperty : public FInputDeviceAnalogStickProperty
{
	FInputDeviceAnalogStickDeadZoneProperty()
		: FInputDeviceAnalogStickProperty(PropertyName())
	{}

	FInputDeviceAnalogStickDeadZoneProperty(EInputDeviceAnalogStickMask InAffectedStick, float InDeadZone)
		: FInputDeviceAnalogStickProperty(PropertyName(), InAffectedStick)
		, DeadZone(InDeadZone)
	{}

	static FName PropertyName() { return FName("InputDeviceAnalogStickDeadZone"); }

	float DeadZone = 0;
};

/**
 * A generic input device property that sets "Dynamic Release DeadZones" for Analog Triggers.
 * 
 * NOTE: Dynamic Triggers allow for Release and Press to be triggered even if the user fails to fully release the trigger before beginning another press.
 * Smaller deadzones increase this effect. A deadzone of 1.0 isn't dynamic at all.
 */
struct FInputDeviceTriggerDynamicReleaseDeadZoneProperty : public FInputDeviceTriggerProperty
{
	FInputDeviceTriggerDynamicReleaseDeadZoneProperty()
		: FInputDeviceTriggerProperty(PropertyName())
	{}

	FInputDeviceTriggerDynamicReleaseDeadZoneProperty(EInputDeviceTriggerMask InAffectedTriggers, float InDeadZone)
		: FInputDeviceTriggerProperty(PropertyName())
		, DeadZone(InDeadZone)
	{
		AffectedTriggers = InAffectedTriggers;
	}

	static FName PropertyName() { return FName("InputDeviceTriggerDynamicReleaseDeadZone"); }

	float DeadZone = 1.0f;
};

/**
 * Interface for the input interface.
 */
class IInputInterface
{
public:

	/** Virtual destructor. */
	virtual ~IInputInterface() = default;

	/**
	* Sets the strength/speed of the given channel for the given controller id.
	* NOTE: If the channel is not supported, the call will silently fail
	*
	* @param ControllerId the id of the controller whose value is to be set
	* @param ChannelType the type of channel whose value should be set
	* @param Value strength or speed of feedback, 0.0f to 1.0f. 0.0f will disable
	*/
	virtual void SetForceFeedbackChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) = 0;

	/**
	* Sets the strength/speed of all the channels for the given controller id.
	* NOTE: Unsupported channels are silently ignored
	*
	* @param ControllerId the id of the controller whose value is to be set
	* @param FForceFeedbackChannelValues strength or speed of feedback for all channels
	*/
	virtual void SetForceFeedbackChannelValues(int32 ControllerId, const FForceFeedbackValues &Values) = 0;

	/**
	* Sets the frequency and amplitude of haptic feedback channels for a given controller id.
	* Some devices / platforms may support just haptics, or just force feedback.
	*
	* @param ControllerId	ID of the controller to issue haptic feedback for
	* @param HandId			Which hand id (e.g. left or right) to issue the feedback for.  These usually correspond to EControllerHands
	* @param Values			Frequency and amplitude to haptics at
	*/
	virtual void SetHapticFeedbackValues(int32 ControllerId, int32 Hand, const FHapticFeedbackValues& Values) {}

	/**
	* Sets a property for a given controller id.
	* Will be ignored for devices which don't support the property.
	*
	* @param ControllerId the id of the controller whose property is to be applied
	* @param Property Base class pointer to property that will be applied
	*/
	virtual void SetDeviceProperty(int32 ControllerId, const FInputDeviceProperty* Property) {};

	/*
	 * Sets the light color for the given controller.  Ignored if controller does not support a color.
	 */
	virtual void SetLightColor(int32 ControllerId, FColor Color) = 0;
	
	/*
	 * Resets the light color for the given controller.  Ignored if controller does not support a color.
	 */
	virtual void ResetLightColor(int32 ControllerId) = 0;
};
