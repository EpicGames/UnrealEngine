// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericWindow.h"

/**
 * Editable texts should implement this class and maintain an object of this type after registering it.
 * Methods of this class are called by the system to query contextual information about the state of the editable text.
 * This information is used by the text input method system to provide appropriate processed input.
 * Methods of this class are also called by the system to provide processed text input.
 */
class ITextInputMethodContext
{
public:
	enum class ECaretPosition
	{
		Beginning,
		Ending
	};

public:
	virtual ~ITextInputMethodContext() = default;

	/**
	 * Returns whether or not this context is currently composing.
	 * @note This should be set to true when BeginComposition is called, and false when EndComposition is called.
	 *
	 * @return	True if we are composing, false otherwise.
	 */
	virtual bool IsComposing() = 0;

	/**
	 * Returns whether or not this text is read-only.
	 *
	 * @return	True if the text is read-only, false otherwise.
	 */
	virtual bool IsReadOnly() = 0;

	/**
	 * Returns the number of code points in the text.
	 *
	 * @return	The number of code points in the text.
	 */
	virtual uint32 GetTextLength() = 0;

	/**
	 * Gets the range of code point indices that are selected and which end of the selection the caret is at.
	 *
	 * @param	OutBeginIndex		The code point index at the beginning of the selection range.
	 * @param	OutLength			The number of code points selected after the beginning index.
	 * @param	OutCaretPosition	A flag indicating whether the caret is at the beginning or ending of the range. Irrelevant if the Length is zero.
	 */
	virtual void GetSelectionRange(uint32& OutBeginIndex, uint32& OutLength, ECaretPosition& OutCaretPosition) = 0;

	/**
	 * Sets the range of code point indices that are selected and which end of the selection the caret is at.
	 *
	 * @param	InBeginIndex	The code point index at the beginning of the selection range.
	 * @param	InLength		The number of code points selected after the beginning index.
	 * @param	InCaretPosition	A flag indicating whether the caret is at the beginning or ending of the range. Irrelevant if the Length is zero.
	 */
	virtual void SetSelectionRange(const uint32 InBeginIndex, const uint32 InLength, const ECaretPosition InCaretPosition) = 0;

	/**
	 * Gets the code points in a range of indices.
	 *
	 * @param	InBeginIndex	The code point index at the beginning of the range to get.
	 * @param	InLength		The number of code points to get after the beginning index.
	 * @param	OutString		A string to store the code points being returned.
	 */
	virtual void GetTextInRange(const uint32 InBeginIndex, const uint32 InLength, FString& OutString) = 0;

	/**
	 * Sets the code points in a range of indices.
	 *
	 * @param	InBeginIndex	The code point index at the beginning of the range to set.
	 * @param	InLength		The number of code points to set after the beginning index.
	 * @param	InString		A string of the code points to be set.
	 */
	virtual void SetTextInRange(const uint32 InBeginIndex, const uint32 InLength, const FString& InString) = 0;

	/**
	 * Gets the index of the code point at the point on the screen.
	 *
	 * @param	InPoint	The 2D point on the screen to test for a code point.
	 *
	 * @return	The index of the code point at the point on the screen. INDEX_NONE if none was found.
	 */
	virtual int32 GetCharacterIndexFromPoint(const FVector2D& InPoint) = 0;
	
	/**
	 * Measures the screen-space bounds of the text in the specified range of code points.
	 *
	 * @param	InBeginIndex	The code point index at the beginning of the range to measure.
	 * @param	InLength		The number of code points to measure after the beginning index.
	 * @param	OutPosition		The screen-space position of the top-left bound of the specified range of code points.
	 * @param	OutSize			The screen-space size of the of the specified range of code points.
	 *
	 * @return	True if the specified range of code points is drawn clipped. False otherwise.
	 */
	virtual bool GetTextBounds(const uint32 InBeginIndex, const uint32 InLength, FVector2D& OutPosition, FVector2D& OutSize) = 0;

	/**
	 * Measures the screen-space bounds of the display area available for text.
	 *
	 * @param	OutPosition		The screen-space position of the top-left bound of the display area.
	 * @param	OutSize			The screen-space size of the of the display area.
	 */
	virtual void GetScreenBounds(FVector2D& OutPosition, FVector2D& OutSize) = 0;

	/**
	 * Returns the window in which the text is displayed.
	 *
	 * @return	The window in which the text is displayed.
	 */
	virtual TSharedPtr<FGenericWindow> GetWindow() = 0;

	/**
	 * Begins composition.
	 */
	virtual void BeginComposition() = 0;

	/**
	 * Updates the range of code point indices being composed.
	 * These code points should be displayed in some manner to communicate they are being composed.
	 * IE: Highlighted and underlined.
	 *
	 * @param	InBeginIndex	The code point index at the beginning of the range being composed.
	 * @param	InLength		The number of code points to measure after the beginning index.
	 */
	virtual void UpdateCompositionRange(const int32 InBeginIndex, const uint32 InLength) = 0;

	/**
	 * Ends composition. May or may not mean the composition was committed.
	 */
	virtual void EndComposition() = 0;
};

/**
 * Platform owners implement this class to react to changes in the view/model of editable text widgets.
 * Methods of this class should be called by the user to notify the system of changes not caused by system calls to 
 * methods of a ITextInputMethodContext implementation.
 */
class ITextInputMethodChangeNotifier
{
public:
	enum class ELayoutChangeType
	{
		Created,
		Changed,
		Destroyed
	};

public:
	virtual ~ITextInputMethodChangeNotifier() = default;

	virtual void NotifyLayoutChanged(const ELayoutChangeType ChangeType) = 0;
	virtual void NotifySelectionChanged() = 0;
	virtual void NotifyTextChanged(const uint32 BeginIndex, const uint32 OldLength, const uint32 NewLength) = 0;
	virtual void CancelComposition() = 0;
};

/**
 * Platform owners implement this class to interface with the platform's input method system.
 */
class ITextInputMethodSystem
{
public:
	virtual ~ITextInputMethodSystem() = default;

	/**
	 * Called when a window is first created to allow the text input system to apply any default settings
	 */
	virtual void ApplyDefaults(const TSharedRef<FGenericWindow>& InWindow) = 0;

	/**
	 * Registers an implemented context interface object with the system to receive callbacks and provides an implemented
	 * notifier interface object. Editable text should call this method after constructing an implemented context 
	 * interface object.
	 *
	 * @param	Context	The implemented context interface object to be registered.
	 *
	 * @return	An implemented notifier interface object
	 */
	virtual TSharedPtr<ITextInputMethodChangeNotifier> RegisterContext(const TSharedRef<ITextInputMethodContext>& Context) = 0;

	/**
	 * Unregisters an implemented context interface object with the system. Editable text should call this method before 
	 * destroying an implemented context interface object.
	 *
	 * @param	Context	The implemented context interface object to be unregistered.
	 */
	virtual void UnregisterContext(const TSharedRef<ITextInputMethodContext>& Context) = 0;

	/**
	 * Activates the provided context. Editable text should call this method when keyboard focus is received.
	 *
	 * @param	Context	The context to be activated.
	 */
	virtual void ActivateContext(const TSharedRef<ITextInputMethodContext>& Context) = 0;

	/**
	 * Deactivates the provided context. Editable text should call this method when keyboard focus is lost.
	 *
	 * @param	Context	The context to be activated.
	 */
	virtual void DeactivateContext(const TSharedRef<ITextInputMethodContext>& Context) = 0;

	/**
	 * Test to see whether the provided context is the currently active context
	 *
	 * @param	Context	The context to be checked.
	 *
	 * @return	True if the context is active, false otherwise
	 */
	virtual bool IsActiveContext(const TSharedRef<ITextInputMethodContext>& Context) const = 0;
};
