// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"	// for int32
#include "UObject/CoreReflectedTypeAccessors.h"

/**
 * The input device mapping policy controls how Human Interface Devices (HID's)
 * are "mapped" or "assigned" to local players in your game. Depending on this
 * policy, the creation of FPlatformUserId's and FInputDeviceId's can change.
 *
 * This policy is only in affect on platforms which do NOT have a managed user login
 * or some kind of dedicated input mapping OS functionality (consoles). 
 */
enum class EInputDeviceMappingPolicy : int32
{
	/**
	 * Invalid mapping policy. Used to validate that the one in the settings is correct.
	 */
	Invalid = -1,
	
	/**
	 * Use the current build target's built in platform login functionality.
	 * This is often times a prompt which is displayed by the target system
	 * to assign an input device to a specific player.
	 *
	 * Note: The behavior of this setting on non-user managed platforms (such as PC targets) is undefined.
	 */
	UseManagedPlatformLogin = 0,
	
	/**
	 * Maps the keyboard/mouse and the first connected controller to the primary platform user.
	 * Any subsequently connected gamepads will be mapped to the next platform user without a
	 * valid input device already assigned to them.
	 *
	 * An example of this policy when you have 3 gamepads and a keyboard/mouse connected would be:
	 *
	 * -- Platform User 0 (Primary)
	 * ---> Keyboard and mouse
	 * ---> Gamepad 0
	 * |
	 * -- Platform User 1
	 * ---> Gamepad 1
	 * |
	 * -- PlatformUser 2
	 * ---> Gamepad 2
	 *
	 * This is the default device mapping policy for PC. 
	 */
	PrimaryUserSharesKeyboardAndFirstGamepad = 1,

	/**
	 * If this policy is set, then every input device that is connected will be mapped
	 * to a unique platform user, meaning that each device is treated as a separate local player.
	 * This is what you want if you want plugging in a new controller to mean that you have a
	 * new local player in your game.
	 *
	 * This only defines the default behavior upon connection of a new device, you can still remap
	 * these input devices to new platform users later in the application lifecycle.
	 *
	 * An example of this policy when you have 3 gamepads and a keyboard/mouse connected would be:
	 *
	 * -- Platform User 0 (Primary)
	 * ---> Keyboard and mouse
	 * |
	 * -- Platform User 1
	 * ---> Gamepad 0
	 * |
	 * -- PlatformUser 2
	 * ---> Gamepad 1
	 * |
	 * -- PlatformUser 3
	 * ---> Gamepad 2
	 */
	CreateUniquePlatformUserForEachDevice = 2,

	/**
	 * If this policy is set, then every connected input device will be mapped to the
	 * primary platform user and new platform user Id's will not be created from input
	 * devices.
	 *
	 * An example of this policy when you have 3 gamepads and a keyboard/mouse connected would be:
	 * -- Platform User 0 (Primary)
	 * ---> Keyboard and mouse
	 * ---> Gamepad 0
	 * ---> Gamepad 1
	 * ---> Gamepad 2
	 */
	MapAllDevicesToPrimaryUser = 3
};

#ifdef COREUOBJECT_API
template<> COREUOBJECT_API UEnum* StaticEnum<EInputDeviceMappingPolicy>();
#endif
