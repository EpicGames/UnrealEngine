// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "HAL/CriticalSection.h"
#include "Misc/ScopeRWLock.h"
#include "Containers/Map.h"
#include "Misc/Optional.h"
#include "UObject/NameTypes.h"

#define UE_API APPLICATIONCORE_API

/**
 * Stable per-device descriptor stored in FInputDeviceRegistry.
 * Mirrors the data held by FInputDeviceScope but is keyed on FInputDeviceId
 * and lives for the lifetime of the device rather than just the dispatch call.
 */
struct FInputDeviceDescriptor final
{
	/** The engine-assigned device ID. INPUTDEVICEID_NONE if unknown. */
	FInputDeviceId HardwareDeviceHandle = INPUTDEVICEID_NONE;
	
	/**
	 * Logical name of the input device interface or module which created this descriptor.
	 * 
	 * This is often times the name of the C++ class which is being used to create this event, like "XInputInterface".
	 *
	 * It is a generic grouping for input devices.
	 */
	FName InputDeviceName = NAME_None;

	/**
	 * Logical identifier for the hardware device. Platform-specific, not translated.
	 * Stored as FName (replaces the FString used in FInputDeviceScope)
	 *
	 * This is a more specific name that represents a specific human interface device such as
	 * "<PlatformName>_Controller". 
	 */
	FName HardwareDeviceIdentifier = NAME_None;

	/** Returns a human-readable string representation of this descriptor, suitable for logging. */
	UE_API FString ToString() const;

	bool operator==(const FInputDeviceDescriptor& Other) const
	{
		return HardwareDeviceHandle == Other.HardwareDeviceHandle
			&& InputDeviceName == Other.InputDeviceName
			&& HardwareDeviceIdentifier == Other.HardwareDeviceIdentifier;
	}

	bool operator!=(const FInputDeviceDescriptor& Other) const
	{
		return !(*this == Other);
	}
};

/**
 * Broadcast when RegisterDevice() stores a descriptor that DIFFERS from the previously-registered
 * descriptor for the same FInputDeviceId.
 *
 * Threading: broadcast runs on whichever thread called RegisterDevice and AFTER the registry's
 * write lock is released, so handlers may safely call back into FInputDeviceRegistry. Game-thread
 * work must be marshalled by the subscriber.
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnInputDeviceDescriptorChanged, const FInputDeviceDescriptor& /*NewDescriptor*/);

/**
 * Thread-safe singleton registry mapping FInputDeviceId to FInputDeviceDescriptor.
 *
 * This is the intended replacement for FInputDeviceScope::GetCurrent() and is safe to call from
 * multiple threads.
 */
class FInputDeviceRegistry final
{
private:
	
	/** Returns the process-wide singleton instance. */
	static UE_API FInputDeviceRegistry& Get();

public:
	/**
	 * Register or update the descriptor for a device.
	 * Called by FInputDeviceScope's constructor and may be called directly at device-connect
	 * time once all call sites have migrated away from FInputDeviceScope.
	 * @return True if the descriptor was registered successfully, false if the descriptor's
	 *         HardwareDeviceHandle is not valid (INPUTDEVICEID_NONE).
	 */
	static UE_API bool RegisterDevice(const FInputDeviceDescriptor& Descriptor);

	/**
	 * Explicitly remove the descriptor for a device from the registry.
	 * The registry is intentionally persistent across connect/disconnect cycles so that
	 * hardware metadata remains queryable for disconnected devices. Only call this when
	 * the device ID itself is being permanently retired (e.g., device ID recycled by the
	 * mapper, or engine shutdown cleanup). No-op if the device was never registered.
	 * Safe to call from any thread.
	 */
	static UE_API void RemoveDevice(const FInputDeviceId DeviceId);

	/**
	 * Look up the descriptor for a device by its FInputDeviceId.
	 * Returns an empty optional if the device has not been registered yet.
	 *
	 * If a simulated descriptor has been set for this device (see SetSimulatedDescriptor),
	 * that takes priority over the real descriptor.
	 * Safe to call from any thread.
	 */
	static UE_API TOptional<FInputDeviceDescriptor> FindDescriptor(const FInputDeviceId DeviceId);

#if !UE_BUILD_SHIPPING
	/**
	 * Override the descriptor returned by FindDescriptor for a specific device ID with a
	 * simulated one. Intended for tooling and non-shipping testing (e.g. input preview,
	 * device simulation) that needs to pretend a particular device is connected without
	 * affecting the real runtime registry.
	 *
	 * The simulated descriptor is stored separately from the real registry and is never
	 * written to by the platform layer. Call ClearSimulatedDescriptor to remove it.
	 *
	 * Not available in Shipping builds.
	 */
	static UE_API void SetSimulatedDescriptor(const FInputDeviceId DeviceId, const FInputDeviceDescriptor& SimulatedDescriptor);

	/**
	 * Remove the simulated descriptor for a device, restoring FindDescriptor to return
	 * the real registered descriptor (if any).
	 *
	 * Not available in Shipping builds.
	 */
	static UE_API void ClearSimulatedDescriptor(const FInputDeviceId DeviceId);

	/**
	 * Remove all simulated descriptors at once (e.g. on PIE end or tool teardown).
	 *
	 * Not available in Shipping builds.
	 */
	static UE_API void ClearAllSimulatedDescriptors();
#endif // !UE_BUILD_SHIPPING

	/**
	 * Log the current contents of the registry to LogInputDeviceRegistry.
	 * In non-shipping builds this also includes any simulated descriptors.
	 * Safe to call from any thread.
	 */
	static UE_API void LogRegistryContents();

	/**
	 * Returns the singleton's "descriptor changed" multicast delegate. See FOnInputDeviceDescriptorChanged
	 * above for broadcast semantics — fires only when a registration ACTUALLY changes the stored
	 * descriptor (first registration counts as a change).
	 */
	static UE_API FOnInputDeviceDescriptorChanged& OnInputDeviceDescriptorChanged();

private:
	mutable FRWLock Lock;
	TMap<FInputDeviceId, FInputDeviceDescriptor> Descriptors;
	FOnInputDeviceDescriptorChanged DescriptorChangedDelegate;

#if !UE_BUILD_SHIPPING
	/** Descriptors set via SetSimulatedDescriptor. Checked first by FindDescriptor. */
	TMap<FInputDeviceId, FInputDeviceDescriptor> SimulatedDescriptors;
#endif // !UE_BUILD_SHIPPING
};

#undef UE_API
