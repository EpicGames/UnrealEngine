// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"

#if PLATFORM_IOS
#include "IOS/IOSPlatformFramePacer.h"
#elif PLATFORM_MAC
#include "Mac/MacPlatformFramePacer.h"
#elif PLATFORM_ANDROID
#include "Android/AndroidPlatformFramePacer.h"
#else
#include "GenericPlatform/GenericPlatformFramePacer.h"
typedef FGenericPlatformRHIFramePacer FPlatformRHIFramePacer;
#endif
