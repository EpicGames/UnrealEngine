// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_ACCESSIBILITY

#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"

#import <Foundation/Foundation.h> 

@class FIOSAccessibilityLeaf;

/**
 * This class is a singleton and should be accessed through [FIOSAccessibilityCache AccessibilityElementCache].
 * Stores a list of accessible containers that map to AccessibleWidgetIds for lookup.
 * The cache is also responsible for polling attributes from the underlying
 * IAccessibleWidgets that are too expensive to be done when requested by IOS due to
 * needing to be accessed from a different thread.
 *
 * Leaf elements can be accessed by getting their container from the cache and calling
 * [container GetLeaf] on it.
 */
@interface FIOSAccessibilityCache : NSObject
{
@private
	/** AccessibleWidgetId(String)->FIOSAccessibilityLeaf map for all created containers. */
	NSMutableDictionary* Cache;
}
/** The Id of the root IAccessibleWindow that backs the IOS application */
@property (nonatomic) AccessibleWidgetId RootWindowId;
/** Retrieve a cached leaf, or create one if it doesn't exist yet. */
-(FIOSAccessibilityLeaf*)GetAccessibilityElement:(AccessibleWidgetId)Id;
/** Returns true if the Cache contains the Id. Does not create one if it doesn't exist. */
-(bool)AccessibilityElementExists:(AccessibleWidgetId)Id;
/** Removes an entry from the Cache. */
-(void)RemoveAccessibilityElement:(AccessibleWidgetId)Id;
/** Completely empties the cache. */
-(void)Clear;
/** Loop over all cached elements and update any properties necessary on the Game thread. */
-(void)UpdateAllCachedProperties;
/** Singleton accessor */
+(FIOSAccessibilityCache*)AccessibilityElementCache;

#if !UE_BUILD_SHIPPING
-(void)DumpAccessibilityStats;
#endif

@end

#endif
