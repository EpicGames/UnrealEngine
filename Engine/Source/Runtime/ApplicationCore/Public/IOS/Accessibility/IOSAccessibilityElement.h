// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_ACCESSIBILITY

#include "CoreMinimal.h"
#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"
#import <UIKit/UIKit.h>

@class FIOSAccessibilityLeaf;

/**
 * UIAccessibilityElements cannot be both accessible and have children. While
 * the same class can be used in both cases, the value they return for
 * isAccessibilityElement determines what type of widget they are. If false,
 * the widget is a container and only functions regarding children will be called.
 * If true, only functions regarding the value of the wigdet will be called.
 *
 * Because of this, all IAccessibleWidgets have both a corresponding container
 * and leaf. The leaf is always reported as the first child of the container. This
 * is our workaround for a widget being accessible and having children at the same time.
 * @see The IOS Accessibility Hierarchy in IOSAccessibilityElement.cpp
 */
@interface FIOSAccessibilityContainer : UIAccessibilityElement
{
@private
	/**
	 * Holds the accessible children of this container.
	 * Can contain both FIOSAccessibilityContainers and FIOSAccessibilityLeaf as elements.
	 * @see [FIOSAccessibilityContainer accessibilityElements] container.
	 */
	NSMutableArray* _AccessibilityChildren;
}

/** This must be used instead of initWithAccessibilityContainer in order to work properly. */
-(id)initWithLeaf:(FIOSAccessibilityLeaf*)InLeaf;

/** A matching Leaf element that shares the same AccessibleWidgetId at this container. */
@property(nonatomic, assign) FIOSAccessibilityLeaf* Leaf;
@end

/**
 * The accessible version of a widget for a given AccessibleWidgetId.
 *  Holds the cached data to be returned to the iOS accessibility API.
 * An FIOSAccessibilityLeaf returns an FIOSAccessibilityContainer as an accessibilityContainer
  * If the leaf has children. This is used to enforce the accessibility hierarchy
 */
@interface FIOSAccessibilityLeaf : UIAccessibilityElement
{
	@private
	/**
	 *The container used to hold this FIOSAccessibilityLeaf if this leaf has children.
	 * Used to enforce The iOS Accessibility Hierarchy
	 *  @see The iOS Accessibility Hierarchy
	 */
	FIOSAccessibilityContainer* _Container;
}

/** This must be used instead of initWithAccessibilityContainer in order to work properly. */
-(id)initWithId:(AccessibleWidgetId)InId;
/**  Used to update the current parent Id of the  Leaf */
-(void)SetParent:(AccessibleWidgetId)InParentId;
/** Check if LastCachedStringTime was updated recently. */
-(bool)ShouldCacheStrings;
/** Check to see if cached strings are stale and should be emptied to save memory. */
-(bool)ShouldEmptyCachedStrings;
/** Empty all cached strings. */
-(void)EmptyCachedStrings;
/** Cache the strings by retrieving the data from the Slate thread. */
-(void)CacheStrings;
/**  Toggle an individual trait on or off */
-(void)SetAccessibilityTrait:(UIAccessibilityTraits)Trait Set:(bool)IsEnabled;
/** A cached version of the name of the widget. */
@property (nonatomic) FString Label;
/** A cached version of the help text of the widget. */
@property (nonatomic) FString Hint;
/** A cached version of the value of property widgets. */
@property (nonatomic) FString Value;
/** Bitflag of traits that describe the widget. Most are set once on initialization. */
@property (nonatomic) UIAccessibilityTraits Traits;
/** Timestamp for when Label, Hint, and Value were last cached. */
@property (nonatomic) double LastCachedStringTime;
/** The identifier used to access this widget through the accessible API. */
@property (nonatomic) AccessibleWidgetId Id;
/** The Id of the parent of this widget in the widget tree */
@property (nonatomic) AccessibleWidgetId ParentId;
/** A list of identifiers for all current children of this container. */
@property (nonatomic) TArray<AccessibleWidgetId> ChildIds;
/** The bounding rect of the container. */
@property (nonatomic) FBox2D Bounds;
/** Whether or not the widget is currently visible. */
@property (nonatomic) bool bIsVisible;
@end

#endif
