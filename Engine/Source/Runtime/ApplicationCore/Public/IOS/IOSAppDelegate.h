// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#import <UIKit/UIKit.h>
#import <GameKit/GKGameCenterViewController.h>

#ifndef SWIFT_IMPORT
#import <AVFoundation/AVAudioSession.h>
#include "Delegates/Delegate.h"
#include "Logging/LogMacros.h"
#include "Containers/UnrealString.h"
#include "IOS/IOSPlatformMisc.h"
#endif

#if PLATFORM_VISIONOS
#import <CompositorServices/CompositorServices.h>
#else
#import <UserNotifications/UserNotifications.h>
#endif

#define USE_MUTE_SWITCH_DETECTION 0

#ifndef SWIFT_IMPORT

#if !PLATFORM_TVOS
extern bool GIOSDelayRotationUntilPresent;
#endif

#ifndef WITH_IOS_UNIVERSAL_LINKS
#define WITH_IOS_UNIVERSAL_LINKS 0
#endif

enum class EAudioFeature : uint8
{
	Playback, // Audio Not affected by the ringer switch
	Record, // Recording only, unless Playback or VoiceChat is also set

	DoNotMixWithOthers, // Do not mix audio with other applications

	VoiceChat, // set AVAudioSessionModeVoiceChat when both Playback and Record are enabled
	UseReceiver, // use receiver instead of speaker when both Playback and Record are enabled. Headsets will still be preferred if they are present
	DisableBluetoothSpeaker, // disable the use of Bluetooth A2DP speakers when both Playback and Record are enabled

	BluetoothMicrophone, // enable the use of Bluetooth HFP headsets when Record is enabled

	BackgroundAudio, // continue to play audio in the background. Requires an appropriate background mode to be set in Info.plist

	AllowHapticsAndSystemSoundsDuringRecording, // allow haptic feedback and system sounds when Record or PlayAndRecord category is active

	NumFeatures,
};

void LexFromString(EAudioFeature& OutFeature, const TCHAR* String);
FString LexToString(EAudioFeature Feature);

// Predicate to decide whether a push notification message should be processed
DECLARE_DELEGATE_RetVal_OneParam(bool, FPushNotificationFilter, NSDictionary*);

class APPLICATIONCORE_API FIOSCoreDelegates
{
public:
	// Broadcast when this application is opened from an external source.
	DECLARE_MULTICAST_DELEGATE_FourParams(FOnOpenURL, UIApplication*, NSURL*, NSString*, id);
	static FOnOpenURL OnOpenURL;

	/** Add a filter to decide whether each push notification should be processed */
	static FDelegateHandle AddPushNotificationFilter(const FPushNotificationFilter& FilterDel);

	/** Remove a previously processed push notification filter */
	static void RemovePushNotificationFilter(FDelegateHandle Handle);

	/** INTERNAL - check if a push notification payload passes all registered filters */
	static bool PassesPushNotificationFilters(NSDictionary* Payload);

	/** INTERNAL - called when entering the background - this is not thread-safe with the game thread or render thread as it is called from the app's Main thread */
	DECLARE_MULTICAST_DELEGATE(FOnWillResignActive);
	static FOnWillResignActive OnWillResignActive;

	/** INTERNAL - called when becoming active - this is not thread-safe with the game thread or render thread as it is called from the app's Main thread */
	DECLARE_MULTICAST_DELEGATE(FOnDidBecomeActive);
	static FOnWillResignActive OnDidBecomeActive;
	
private:
	struct FFilterDelegateAndHandle
	{
		FPushNotificationFilter Filter;
		FDelegateHandle Handle;
	};

	static TArray<FFilterDelegateAndHandle> PushNotificationFilters;
};

DECLARE_LOG_CATEGORY_EXTERN(LogIOSAudioSession, Log, All);

@class IOSAppDelegate;

namespace FAppEntry
{
	void PlatformInit();
	void PreInit(IOSAppDelegate* AppDelegate, UIApplication* Application);
	void Init();
	void Tick();
    void SuspendTick();
	void ResumeAudioContext();
	void ResetAudioContextResumeTime();
	void Shutdown();
    void Suspend(bool bIsInterrupt = false);
    void Resume(bool bIsInterrupt = false);
	void RestartAudio();
    void IncrementAudioSuspendCounters();
    void DecrementAudioSuspendCounters();

	bool IsStartupMoviePlaying();

	extern bool	gAppLaunchedWithLocalNotification;
	extern FString	gLaunchLocalNotificationActivationEvent;
	extern int32	gLaunchLocalNotificationFireDate;
}

#endif

@class FIOSView;
@class IOSViewController;

#ifndef SWIFT_IMPORT
APPLICATIONCORE_API
#endif
@interface IOSAppDelegate : UIResponder <
	UIApplicationDelegate,
#if !UE_BUILD_SHIPPING
	UIGestureRecognizerDelegate,
#endif
	GKGameCenterControllerDelegate,
#if !PLATFORM_VISIONOS
	UNUserNotificationCenterDelegate,
#endif
	UITextFieldDelegate>
{
    bool bForceExit;
}

/** Window object */
@property (strong, retain, nonatomic) UIWindow *Window;

// support Compositor on other platforms?
#if PLATFORM_VISIONOS
@property (strong) CP_OBJECT_cp_layer_renderer* SwiftLayer;
@property (retain) NSArray* SwiftLayerViewports;
#endif

/** The view containing the game contents */
@property (retain) FIOSView* IOSView;

#ifndef SWIFT_IMPORT
@property class FIOSApplication* IOSApplication;
#endif

/** The controller for the main view */
@property (retain) IOSViewController* IOSController;

/** The view controlled by the auto-rotating controller */
@property (retain) UIView* RootView;

/** The value of the alert response (atomically set since main thread and game thread use it */
@property (assign) int AlertResponse;

/** Version of the OS we are running on (NOT compiled with) */
@property (readonly) float OSVersion;

@property bool bDeviceInPortraitMode;

/** splash screen view controller */
@property (retain) UIViewController* viewController;

@property (retain) NSTimer* timer;

@property (retain) NSTimer* PeakMemoryTimer;

/** Timer used for re-enabling the idle timer */
@property (retain) NSTimer* IdleTimerEnableTimer;

/** The time delay (in seconds) between idle timer enable requests and actually enabling the idle timer */
@property (readonly) float IdleTimerEnablePeriod;

#if WITH_ACCESSIBILITY
/** Timer used for updating cached data from game thread for all accessible widgets. */
@property (nonatomic, retain) NSTimer* AccessibilityCacheTimer;
/** Callback for IOS notification of VoiceOver being enabled or disabled. */
-(void)OnVoiceOverStatusChanged;
#endif

// parameters passed from openURL
@property (nonatomic, retain) NSMutableArray* savedOpenUrlParameters;

#if !UE_BUILD_SHIPPING && !PLATFORM_TVOS
	/** Properties for managing the console */
	@property (nonatomic, retain) UIAlertController* ConsoleAlertController;
	@property (nonatomic, retain) NSMutableArray*	ConsoleHistoryValues;
	@property (nonatomic, assign) int				ConsoleHistoryValuesIndex;
#endif

/** True if the engine has been initialized */
@property (atomic) bool bEngineInit;
/** True after the end of FAppEntry::PlatformInit */
@property (atomic) bool bPlatformInit;


/** Delays game initialization slightly in case we have a URL launch to handle */
@property (retain) NSTimer* CommandLineParseTimer;
@property (atomic) bool bCommandLineReady;

#if !PLATFORM_TVOS
@property (assign) UIInterfaceOrientation InterfaceOrientation;
#endif

/** initial launch options */
@property (retain) NSDictionary* launchOptions;

@property (assign) NSProcessInfoThermalState ThermalState;
@property (assign) bool bBatteryState;
@property (assign) int BatteryLevel;

@property (assign) float ScreenScale;	// UIScreen.scale
@property (assign) float NativeScale;	// UIWindow.screen.nativeScale
@property (assign) float MobileContentScaleFactor;
@property (assign) int RequestedResX;
@property (assign) int RequestedResY;
@property (assign) bool bUpdateAvailable;
@property (retain) NSString* PendingActivationProtocol;
/**
 * @return the single app delegate object
 */
+ (IOSAppDelegate*)GetDelegate;

-(bool)IsIdleTimerEnabled;
-(void)EnableIdleTimer:(bool)bEnable;
-(void)StartGameThread;
#ifndef SWIFT_IMPORT
/** Uses the TaskGraph to execute a function on the game thread, and then blocks until the function is executed. */
+(bool)WaitAndRunOnGameThread:(TUniqueFunction<void()>)Function;
#endif
-(void)NoUrlCommandLine;

-(void)LoadScreenResolutionModifiers;

-(int)GetAudioVolume;
-(bool)AreHeadphonesPluggedIn;
-(int)GetBatteryLevel;
-(bool)IsRunningOnBattery;
-(NSProcessInfoThermalState)GetThermalState;
-(void)CheckForZoomAccessibility;
-(float)GetBackgroundingMainThreadBlockTime;
-(void)OverrideBackgroundingMainThreadBlockTime:(float)BlockTime;
-(NSString*)GetPendingActivationProtocol;
-(void)SetPendingActivationProtocol:(NSString *)NewProtocol;

#if !PLATFORM_TVOS && !PLATFORM_VISIONOS && !defined(SWIFT_IMPORT)
  +(EDeviceScreenOrientation) ConvertFromUIInterfaceOrientation:(UIInterfaceOrientation)Orientation;
  +(UIInterfaceOrientation) GetUIInterfaceOrientationFromUIDeviceOrientation:(UIDeviceOrientation)Orientation;
  +(UIInterfaceOrientationMask) GetMaskFromUIInterfaceOrientation:(UIInterfaceOrientation)Orientation;
  +(UIInterfaceOrientationMask) GetMaskFromEDeviceScreenOrientation:(EDeviceScreenOrientation)Orientation;
#endif

#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
  +(void)UpdateSupportedInterfaceOrientations;
#endif

-(bool)IsUpdateAvailable;

-(void) didRotate:(NSNotification*)notification;

@property (assign) bool bAudioSessionInitialized;

/** TRUE if the device is playing background music and we want to allow that */
@property (assign) bool bUsingBackgroundMusic;
@property (assign) bool bLastOtherAudioPlaying;
@property (assign) bool bForceEmitOtherAudioPlaying;

#if USE_MUTE_SWITCH_DETECTION
@property (assign) bool bLastMutedState;
@property (assign) bool bForceEmitMutedState;
#endif

@property (assign) float LastVolume;
@property (assign) bool bForceEmitVolume;

- (void)InitializeAudioSession;
- (void)ToggleAudioSession:(bool)bActive;
- (bool)IsBackgroundAudioPlaying;
- (bool)HasRecordPermission;
#ifndef SWIFT_IMPORT
- (void)RequestRecordPermission:(TFunction<void(bool)>)OnComplete;
- (FIOSPlatformMisc::EIOSAuthMicrophonePermissionStatus)GetMicrophonePermissionStatus;
#endif //SWIFT_IMPORT
- (void)EnableVoiceChat:(bool)bEnable;
- (void)EnableHighQualityVoiceChat:(bool)bEnable;
- (bool)IsVoiceChatEnabled;

#ifndef SWIFT_IMPORT
/** Enable/Disable an EAudioFeature. This is reference counted, so a feature must be disabled as many times as it has been enabled to actually be disabled. */
- (void)SetFeature:(EAudioFeature)Feature Active:(bool)bIsActive;
- (bool)IsFeatureActive:(EAudioFeature)Mode;
#endif

@property (atomic) bool bAudioActive;
@property (atomic) bool bVoiceChatEnabled;

@property (atomic) bool bIsSuspended;
@property (atomic) bool bHasSuspended;
@property (atomic) bool bHasStarted;
- (void)ToggleSuspend:(bool)bSuspend;

- (void)ForceExit;

// Shared lifecycle handlers. UIApplicationDelegate (legacy) and UIWindowSceneDelegate (TN3187)
// both forward into these so the engine sees identical Suspend/Resume/Foreground/Background flow
// regardless of which lifecycle is selected at compile time.
- (void)handleWillResignActive;
- (void)handleDidEnterBackground;
- (void)handleWillEnterForeground;
- (void)handleDidBecomeActive;
- (BOOL)handleOpenURL:(NSURL *)url sourceApplication:(NSString *)sourceApplication annotation:(id)annotation;
#if WITH_IOS_UNIVERSAL_LINKS
- (BOOL)handleContinueUserActivity:(NSUserActivity *)userActivity;
#endif

static void interruptionListener(void* ClientData, UInt32 Interruption);

-(UIWindow*)window;

@end

void InstallSignalHandlers();
