// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"
#include "GenericPlatform/GenericApplication.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#elif PLATFORM_IOS
#include "IOS/IOSSystemIncludes.h"
#endif

// Set to 1 to enable [rotation] debug logging across IOS files
#ifndef IOS_ROTATION_DEBUG_LOGGING
#define IOS_ROTATION_DEBUG_LOGGING 0
#endif

class FIOSInputInterface;
class FIOSWindow;
class IModularFeature;

class FIOSApplication : public GenericApplication
{
public:

	static FIOSApplication* CreateIOSApplication();


public:

	virtual ~FIOSApplication();
	
	void HandlePhysicalKeyboardConnectionChanged();
	
	void HandlePhysicalMouseConnectionChanged();
	
	void HandleMouseMovement(FIntVector2 MovementDelta);
	
	virtual FModifierKeysState GetModifierKeys() const override;
	
	virtual bool IsMouseAttached() const override;

	void SetMessageHandler( const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler );
#if WITH_ACCESSIBILITY
	virtual void SetAccessibleMessageHandler(const TSharedRef<FGenericAccessibleMessageHandler>& InAccessibleMessageHandler) override;
#endif

	virtual void PollGameDeviceState( const float TimeDelta ) override;

	virtual FPlatformRect GetWorkArea( const FPlatformRect& CurrentWindow ) const override;

	virtual TSharedRef< FGenericWindow > MakeWindow() override;

	virtual void AddExternalInputDevice(TSharedPtr<class IInputDevice> InputDevice);

#if !PLATFORM_TVOS
#if !PLATFORM_VISIONOS
	static void OrientationChanged(UIInterfaceOrientation NewOrientation);
	static void UpdateSafeZoneAfterRotation();
#endif
	static UIInterfaceOrientation CachedOrientation;
#endif

	virtual IInputInterface* GetInputInterface() override { return (IInputInterface*)InputInterface.Get(); }
	FIOSInputInterface* GetIOSInputInterface() { return InputInterface.Get(); }

	virtual bool IsGamepadAttached() const override;

	TSharedRef<FIOSWindow> FindWindowByAppDelegateView();

protected:
	virtual void InitializeWindow( const TSharedRef< FGenericWindow >& Window, const TSharedRef< FGenericWindowDefinition >& InDefinition, const TSharedPtr< FGenericWindow >& InParent, const bool bShowImmediately ) override;

private:
	FIOSApplication();
#if WITH_ACCESSIBILITY
	void OnAccessibleEventRaised(const FAccessibleEventArgs& Args);
#endif

	void OnInputDeviceModuleRegistered(const FName& Type, IModularFeature* ModularFeature);

private:

	TSharedPtr< FIOSInputInterface > InputInterface;

	/** List of input devices implemented in external modules. */
	TArray< TSharedPtr<class IInputDevice> > ExternalInputDevices;
	bool bHasLoadedInputPlugins;

	TArray< TSharedRef< FIOSWindow > > Windows;
#if WITH_ACCESSIBILITY
	/**
	 * A timer used to introduce a small delay before accessibility announcement requests to
	 * Avoid our requested accessibility announcement from being stompd by system accessibility requests.
	 */
	NSTimer* AccessibilityAnnouncementDelayTimer;
#endif
	
	static FCriticalSection CriticalSection;
	static bool bOrientationChanged;
	
	std::atomic<bool> bIsMouseAttached = false;

#if !PLATFORM_TVOS
	void CacheDisplayMetrics(UIInterfaceOrientation Orientation);
#endif
};
