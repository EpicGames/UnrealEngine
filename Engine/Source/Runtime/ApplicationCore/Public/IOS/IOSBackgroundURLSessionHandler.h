// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

UE_DEPRECATED_HEADER(5.8, "Use BackgroundURLSessionHandler.h and add the BackgroundHttpIOS module as dependency.")

#include "BackgroundURLSessionHandler.h" 
