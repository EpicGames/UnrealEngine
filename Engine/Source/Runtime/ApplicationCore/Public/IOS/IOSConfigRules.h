// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/GenericConfigRules.h"

#import <Foundation/NSObject.h>

// iOSConfigRuleProvider represents the interface used to return the info required to load multiple config files
// instances are registered with iOSConfigRuleProviders during +load time and queried during FIOSConfigRules::InitRules
@protocol iOSConfigRuleProvider <NSObject>
// getRuleData returns an array of arrays, the sub arrays must contain 3 elements
// of filename, key, iv. e.g. [["filename", "key", "IV"],["filename2", "key2", "IV2"], etc.. 
- (NSArray *)getRuleData;
@end

// iOSConfigRuleProviders contains the list of iOSConfigRuleProviders
// instances of iOSConfigRuleProvider are registered here to be 
// queried during Init
@interface iOSConfigRuleProviders : NSObject
+ (void)load;
+ (void)registerRuleProvider:(NSObject<iOSConfigRuleProvider>*)newProvider;
+ (NSArray *)getRuleProviders;
+ (void)empty;
@end

class FIOSConfigRules : public FGenericConfigRules
{
	struct FConfigRuleParams
	{
		FString Path;
		FString Key;
		FString IV;
	};
    static inline TArray<FConfigRuleParams> ConfigRulesParams;
	
	static void InitRules();	
	static inline TMap<FString, FString> ConfigRuleVariablesMap;
	
public:
	// Initialize the config rules, 
	// PredefinedVariables contains a set of K/V pairs that are used while
	// processing the rules. They are included in the map returned by GetConfigRulesMap.
	// returns true if a config rule file was processed.
	static APPLICATIONCORE_API bool Init(TMap<FString, FString>&& PredefinedVariables);
	static APPLICATIONCORE_API const TMap<FString, FString>& GetConfigRulesMap() { return ConfigRuleVariablesMap; }
};
