// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformSoftwareCursor.h"
#include "GenericPlatform/ICursor.h"
#include "Math/IntRect.h"

class FIOSCursor : public FGenericPlatformSoftwareCursor
{
public:

	FIOSCursor();

	virtual ~FIOSCursor();
};
