// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/OutputDevice.h"

/**
 * Generic Output device that writes to Windows Event Log
 */
class FIOSDebugOutputDevice : public FOutputDevice
{
public:
	/** Constructor, initializing member variables */
	APPLICATIONCORE_API FIOSDebugOutputDevice();

	/**
	 * Serializes the passed in data unless the current event is suppressed.
	 *
	 * @param	Data	Text to log
	 * @param	Event	Event name used for suppression purposes
	 */
	virtual void Serialize( const TCHAR* Msg, ELogVerbosity::Type Verbosity, const class FName& Category ) override;
};
