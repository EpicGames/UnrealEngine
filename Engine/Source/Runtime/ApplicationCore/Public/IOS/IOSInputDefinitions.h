// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace IOS
{
	/** Unreal engine key codes used for key mapping in IOS Platforms. 
	 * Apple has GCKeyCodes and UIKit USB Hid codes. We use these to map to a single key code in the UE side */
	enum class EUnrealKeyCode : uint32
	{
		Unknown = 0,
		
		Enter = 1000,
		Backspace,
		Escape,
		Tab,
		Left,
		Right,
		Down,
		Up,
		LeftControl,
		LeftShift,
		LeftAlt,
		LeftCommand,
		CapsLock,
		RightControl,
		RightShift,
		RightAlt,
		RightCommand,
		F1,
		F2,
		F3,
		F4,
		F5,
		F6,
		F7,
		F8,
		F9,
		F10,
		F11,
		F12,
		F13,
		F14,
		F15,
		F16,
		F17,
		F18,
		F19,
		F20,

		PageUp,
		PageDown,
		Home,
		End,
		Insert,

		Multiply,
		Add,
		Subtract,
		Divide,

		ScrollLock,
		Pause,

		NumLock,
		NumPad0,
		NumPad1,
		NumPad2,
		NumPad3,
		NumPad4,
		NumPad5,
		NumPad6,
		NumPad7,
		NumPad8,
		NumPad9,
		Delete,
		Decimal,
		SpaceBar,
	};	
}