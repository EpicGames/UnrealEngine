// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/IInputInterface.h"
#if !PLATFORM_TVOS
#import <CoreMotion/CoreMotion.h>
#endif
#import <GameController/GameController.h>
#include "Misc/CoreMisc.h"
#include "Math/Quat.h"
#include "Math/Vector.h"
#include "Math/Color.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "AppleControllerInterface.h"
#include "AppleKeyboardController.h"
#include "AppleMouseController.h"


#define KEYCODE_ENTER UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1000
#define KEYCODE_BACKSPACE UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1001
#define KEYCODE_ESCAPE UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1002
#define KEYCODE_TAB UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1003
#define KEYCODE_LEFT UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1004
#define KEYCODE_RIGHT UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1005
#define KEYCODE_DOWN UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1006
#define KEYCODE_UP UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1007
#define KEYCODE_LEFT_CONTROL UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1008
#define KEYCODE_LEFT_SHIFT UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1009
#define KEYCODE_LEFT_ALT UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1010
#define KEYCODE_LEFT_COMMAND UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1011
#define KEYCODE_CAPS_LOCK UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1012
#define KEYCODE_RIGHT_CONTROL UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1013
#define KEYCODE_RIGHT_SHIFT UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1014
#define KEYCODE_RIGHT_ALT UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1015
#define KEYCODE_RIGHT_COMMAND UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1016
#define KEYCODE_F1 UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1017
#define KEYCODE_F2 UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1018
#define KEYCODE_F3 UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1019
#define KEYCODE_F4 UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1020
#define KEYCODE_F5 UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1021
#define KEYCODE_F6 UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1022
#define KEYCODE_F7 UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1023
#define KEYCODE_F8 UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1024
#define KEYCODE_F9 UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1025
#define KEYCODE_F10 UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1026
#define KEYCODE_F11 UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1027
#define KEYCODE_F12 UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1028
#define KEYCODE_F13 UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1029
#define KEYCODE_F14 UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1030
#define KEYCODE_F15 UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1031
#define KEYCODE_F16 UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1032
#define KEYCODE_F17 UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1033
#define KEYCODE_F18 UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1034
#define KEYCODE_F19 UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1035
#define KEYCODE_F20 UE_DEPRECATED_MACRO(5.8, "IOS KEYCODE Define has been deprecated. Use enum IOS::EUnrealKeyCode") 1036

enum TouchType
{
	TouchBegan,
	TouchMoved,
	TouchEnded,
	ForceChanged,
	FirstMove,
};

struct TouchInput
{
	int Handle;
	TouchType Type;
	FVector2D LastPosition;
	FVector2D Position;
	float Force;
};

struct GestureInput
{
    EGestureEvent Type;
    FVector2D     Delta;       // gesture delta (pan offset, swipe direction)
    bool          bIsStart;    // UIGestureRecognizerStateBegan
    bool          bIsEnd;      // UIGestureRecognizerStateEnded/Cancelled
};

enum class EIOSEventType : int32
{
    Invalid = 0,
    LeftMouseDown = 1,
    LeftMouseUp = 2,
    RightMouseDown = 3,
    RightMouseUp = 4,
    KeyDown = 10,
    KeyUp = 11,
    MiddleMouseDown = 25,
    MiddleMouseUp = 26,
    ThumbDown = 50,
    ThumbUp = 70,
};

struct FDeferredIOSEvent
{
    EIOSEventType type;
    uint32 keycode;
    uint32 charcode;
};

/**
 * Interface class for IOS input devices
 */
class FIOSInputInterface : public FAppleControllerInterface, FSelfRegisteringExec
{
public:

	static TSharedRef< FIOSInputInterface > Create(  const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler );
	void MapKeyboardModifierKeys();
	static TSharedPtr< FIOSInputInterface > Get();

public:

	virtual ~FIOSInputInterface() {}
	
	/**
	 * Poll for controller state and send events if needed
	 */
	void SendControllerEvents();
	
	virtual void SetMessageHandler( const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler );
	
	/**
	 * IInputInterface implementation
	 */
	virtual void SetForceFeedbackChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) override;
	virtual void SetForceFeedbackChannelValues(int32 ControllerId, const FForceFeedbackValues &values) override;

	static APPLICATIONCORE_API void QueueTouchInput(const TArray<TouchInput>& InTouchEvents);
	static APPLICATIONCORE_API void QueueGestureInput(const TArray<GestureInput>& InGestureInputs);

	void SetGamepadsAllowed(bool bAllowed) { bAllowControllers = bAllowed; }
	void SetGamepadsBlockDeviceFeedback(bool bBlock) { bControllersBlockDeviceFeedback = bBlock; }

	void EnableMotionData(bool bEnable);
	bool IsMotionDataEnabled() const;
    
	UE_DEPRECATED(5.8, "Please use SetKeyboardControllerInhibited")
	static void SetKeyboardInhibited(bool bInhibited)
	{
	}
	UE_DEPRECATED(5.8, "Please use IsKeyboardControllerInhibited")
	static bool IsKeyboardInhibited()
	{
		return false;
	}
	
	TWeakPtr<FAppleKeyboardController> GetKeyboardController();
	
	TWeakPtr<FAppleMouseController> GetMouseController();
	
	void SetKeyboardControllerInhibited(bool bInhibited);

	bool IsKeyboardControllerInhibited() const;
	
	bool IsAnyPhysicalKeyboardConnected() const;
    
    NSData* GetGamepadGlyphRawData(const FGamepadKeyNames::Type& ButtonKey, uint32 ControllerIndex);

protected:

	//~ Begin Exec Interface
	virtual bool Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	//~ End Exec Interface

private:

	FIOSInputInterface( const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler );

    /**
	 * Get the current Movement data from the device
	 *
	 * @param Attitude The current Roll/Pitch/Yaw of the device
	 * @param RotationRate The current rate of change of the attitude
	 * @param Gravity A vector that describes the direction of gravity for the device
	 * @param Acceleration returns the current acceleration of the device
	 */
	void GetMovementData(FVector& Attitude, FVector& RotationRate, FVector& Gravity, FVector& Acceleration);

	/**
	 * Calibrate the devices motion
	 */
	void CalibrateMotion(uint32 PlayerIndex);

private:
	void ProcessTouchesAndKeys(uint32 ControllerId, const TArray<TouchInput>& InTouchInputStack);

	void ProcessGestures(uint32 ControllerId, const TArray<GestureInput>& InGestureInputStack);
	
	// can the remote be rotated to landscape
	bool bAllowRemoteRotation;

	// can the game handle multiple gamepads at the same time (siri remote is a gamepad) ?
	bool bGameSupportsMultipleActiveControllers;

	// bluetooth connected controllers will block force feedback.
	bool bControllersBlockDeviceFeedback;
	
	/** Is motion paused or not? */
	bool bPauseMotion;

#if !PLATFORM_TVOS
	/** Access to the ios devices motion */
	CMMotionManager* MotionManager;

	/** Access to the ios devices tilt information */
	CMAttitude* ReferenceAttitude;
#endif

	/** Last frames roll, for calculating rate */
	float LastRoll;

	/** Last frames pitch, for calculating rate */
	float LastPitch;

	/** True if a calibration is requested */
	bool bIsCalibrationRequested;

	/** The center roll value for tilt calibration */
	float CenterRoll;

	/** The center pitch value for tilt calibration */
	float CenterPitch;

	/** When using just acceleration (without full motion) we store a frame of accel data to filter by */
	FVector FilteredAccelerometer;

	/** Last value sent to mobile haptics */
	float LastHapticValue;

	// Cumulative pinch scale ratio for native gesture value conversion.
	float CumulativePinchScale = 1.0f;

	int HapticFeedbackSupportLevel;
	
	static FName InputClassName_DefaultMobileTouch;
	static FString HardwareDeviceIdentifier_DefaultMobileTouch;
	
	TSharedRef<FAppleKeyboardController> KeyboardController;
	TSharedRef<FAppleMouseController> MouseController;
};
