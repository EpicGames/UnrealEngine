// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#import <UIKit/UIKit.h>
#import <UIKit/UIKeyConstants.h>

namespace IOS
{
	enum class EUnrealKeyCode : uint32;
	
	/** Translate UI Kit USB Hid keycodes into Unreal Keycodes */
	EUnrealKeyCode TranslateUSBHIDToUnrealKeycode(UIKeyboardHIDUsage HIDKeycode);
}

