// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformApplicationMisc.h"
#include "Delegates/DelegateCombinations.h"

DECLARE_DELEGATE_RetVal_TwoParams(class UTexture2D*, FGetGamePadGlyphDelegate, const FGamepadKeyNames::Type&, uint32);

struct APPLICATIONCORE_API FIOSPlatformApplicationMisc : public FGenericPlatformApplicationMisc
{
	static void LoadPreInitModules();

	static class FFeedbackContext* GetFeedbackContext();
	static class FOutputDeviceError* GetErrorOutputDevice();
	static class GenericApplication* CreateApplication();
	static bool IsScreensaverEnabled();
	static bool ControlScreensaver(EScreenSaverAction Action);

	static void SetGamepadsAllowed(bool bAllowed);
	static void SetGamepadsBlockDeviceFeedback(bool bBlock);
	static void ResetGamepadAssignments();
	static void ResetGamepadAssignmentToController(int32 ControllerId);
	static bool IsControllerAssignedToGamepad(int32 ControllerId);
	static class UTexture2D* GetGamepadButtonGlyph(const FGamepadKeyNames::Type& ButtonKey, uint32 ControllerIndex);

	static void EnableMotionData(bool bEnable);
	static bool IsMotionDataEnabled();

	static void ClipboardCopy(const TCHAR* Str);
	static void ClipboardPaste(class FString& Dest);

	static EScreenPhysicalAccuracy ComputePhysicalScreenDensity(int32& ScreenDensity);
	static EScreenPhysicalAccuracy ComputePhysicalScreenDimensions(FVector2f& OutScreenDimensions, float& OutScreenSize);

    static bool RequiresVirtualKeyboard();
    
private:
	static class FIOSApplication* CachedApplication;

	friend class FIOSGamepadUtils;
	static FGetGamePadGlyphDelegate GetGamePadGlyphDelegate;
};

typedef FIOSPlatformApplicationMisc FPlatformApplicationMisc;
