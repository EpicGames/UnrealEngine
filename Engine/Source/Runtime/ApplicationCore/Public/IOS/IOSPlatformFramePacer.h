// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	IOSPlatformFramePacer.h: Apple iOS platform frame pacer classes.
==============================================================================================*/


#pragma once
#include "GenericPlatform/GenericPlatformFramePacer.h"

// Forward declare the ios frame pacer class we will be using.
@class FIOSFramePacer;

typedef void (^FIOSFramePacerHandler)(uint32 IgnoredId, double OutputSeconds, double OutputDuration);

/**
 * iOS implementation of FGenericPlatformRHIFramePacer
 **/
struct FIOSPlatformRHIFramePacer : public FGenericPlatformRHIFramePacer
{
    // FGenericPlatformRHIFramePacer interface
    static bool IsEnabled();
    static void Destroy();
	static int32 SetFramePace(int32 FramePace);
	static int32 GetFramePace();
	static bool SupportsFramePace(int32 QueryFramePace);

	// FIOSPlatformRHIFramePacer interface
	static void InitWithEvent(class FEvent* TriggeredEvent);
	static void AddHandler(FIOSFramePacerHandler Handler);
	static void RemoveHandler(FIOSFramePacerHandler Handler);
	static uint32 GetMaxRefreshRate();

    /** Access to the IOS Frame Pacer: CADisplayLink */
    static FIOSFramePacer* FramePacer;
    
    /** Number of frames before the CADisplayLink triggers it's readied callback */
    static uint32 FrameInterval;

	 /** The minimum frame interval dictated by project settings on startup */
	static uint32 MinFrameInterval;
	
	/** Frame rate we are pacing to */
	static uint32 Pace;
    
    /** Suspend the frame pacer so we can enter the background state */
    static void Suspend();
    
    /** Resume the frame pacer so we can enter the foreground state */
    static void Resume();
};


typedef FIOSPlatformRHIFramePacer FPlatformRHIFramePacer;
typedef FIOSFramePacerHandler FPlatformRHIFramePacerHandler;
