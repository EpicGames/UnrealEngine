// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	IOSPlatformSplash.h: iOS platform splash screen...
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformSplash.h"

/**
* IOS splash implementation 
**/
struct APPLICATIONCORE_API FIOSPlatformSplash : public FGenericPlatformSplash
{
	// default implementation for now
};

typedef FIOSPlatformSplash FPlatformSplash;

