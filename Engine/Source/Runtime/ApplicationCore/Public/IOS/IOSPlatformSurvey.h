// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	IOSPlatformSurvey.h: Apple iOS platform hardware-survey classes
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformSurvey.h"

/**
* iOS implementation of FGenericPlatformSurvey
**/
struct FIOSPlatformSurvey : public FGenericPlatformSurvey
{
	static bool GetSurveyResults(FHardwareSurveyResults& OutResults, bool bWait = false);
};

typedef FIOSPlatformSurvey FPlatformSurvey;