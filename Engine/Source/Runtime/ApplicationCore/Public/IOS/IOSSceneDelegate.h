// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#import <UIKit/UIKit.h>

#if UE_IOS_SCENE_LIFECYCLE

#ifndef SWIFT_IMPORT
APPLICATIONCORE_API
#endif
@interface IOSSceneDelegate : UIResponder <UIWindowSceneDelegate>

@property (strong, nonatomic) UIWindow *window;

@end

#endif // UE_IOS_SCENE_LIFECYCLE
