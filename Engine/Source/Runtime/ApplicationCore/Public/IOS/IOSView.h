// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef SWIFT_IMPORT
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "IOS/IOSInputInterface.h"
#include "Templates/SharedPointer.h"
#endif

#import <UIKit/UIKit.h>

#import <Metal/Metal.h>
#import <QuartzCore/CAMetalLayer.h>

#if WITH_ACCESSIBILITY
#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"
#endif

#ifndef SWIFT_IMPORT
struct FKeyboardConfig
{
	UIKeyboardType KeyboardType;
	UITextAutocorrectionType AutocorrectionType;
	UITextAutocapitalizationType AutocapitalizationType;
	BOOL bSecureTextEntry;
	
	inline FKeyboardConfig() :
		KeyboardType(UIKeyboardTypeDefault),
		AutocorrectionType(UITextAutocorrectionTypeNo),
		AutocapitalizationType(UITextAutocapitalizationTypeNone),
		bSecureTextEntry(NO) {}
};

namespace MTL
{
    class Device;
}
#endif

#ifndef SWIFT_IMPORT
APPLICATIONCORE_API
#endif
#if !PLATFORM_TVOS
@interface FIOSView : UIView  <UIKeyInput, UITextInput, UIGestureRecognizerDelegate>
#else
@interface FIOSView : UIView  <UIKeyInput, UITextInput>
#endif
{
@public
	// are we initialized?
	bool bIsInitialized;

//@private
	// keeps track of the number of active touches
	// used to bring up the three finger touch debug console after 3 active touches are registered
	int NumActiveTouches;

	// track the touches by pointer (which will stay the same for a given finger down) - note we don't deref the pointers in this array
	UITouch* AllTouches[10];
	float PreviousForces[10];
	bool HasMoved[10];

	// global metal device
#ifndef SWIFT_IMPORT
	MTL::Device* MetalDevice;
#endif
	id<CAMetalDrawable> PanicDrawable;
	// Might be null if caching is disabled.
	CAMetalLayer* CachedMetalLayer;

	//// KEYBOARD MEMBERS
	
	// whether or not to use the new style virtual keyboard that sends events to the engine instead of using an alert
	bool bIsUsingIntegratedKeyboard;
	bool bSendEscapeOnClose;

	// caches for the TextInput
	NSString* CachedMarkedText;
	
	UIKeyboardType KeyboardType;
	UITextAutocorrectionType AutocorrectionType;
	UITextAutocapitalizationType AutocapitalizationType;
	BOOL bSecureTextEntry;
	
	volatile int KeyboardShowCount;

#if !PLATFORM_TVOS
	UIInterfaceOrientationMask SupportedInterfaceOrientations;
	// the orientation for the current frame we are displaying, or 0 if none is available
	UIInterfaceOrientationMask LastFrameInterfaceOrientationMask;
#endif
}

#if WITH_ACCESSIBILITY
/** Repopulate _accessibilityElements when the accessible window's ID has changed. */
-(void)SetAccessibilityWindow:(AccessibleWidgetId)WindowId;
#endif

//// SHARED FUNCTIONALITY
@property (nonatomic) uint SwapCount;
@property (assign, nonatomic) CGSize ViewSize;

-(bool)CreateFramebuffer;
-(void)DestroyFramebuffer;
-(void)UpdateRenderWidth:(unsigned int)Width andHeight:(unsigned int)Height;
-(void)CalculateContentScaleFactor:(int)ScreenWidth ScreenHeight:(int)ScreenHeight;
-(void)forceLayoutSubviews;

- (void)SwapBuffers;

//// METAL FUNCTIONALITY
// Return a drawable object (ie a back buffer texture) for the RHI to render to
- (nullable id<CAMetalDrawable>)MakeDrawable;

#ifndef SWIFT_IMPORT
//// KEYBOARD FUNCTIONALITY
-(void)InitKeyboard;
-(void)ActivateKeyboard:(bool)bInSendEscapeOnClose;
-(void)ActivateKeyboard:(bool)bInSendEscapeOnClose keyboardConfig:(FKeyboardConfig)KeyboardConfig;
-(void)DeactivateKeyboard;
-(bool)ShouldUseIntegratedKeyboard;
-(TWeakPtr<FAppleKeyboardController>) GetKeyboardController;

// callable from outside to fake locations
-(void)HandleTouchAtLoc:(CGPoint)Loc PrevLoc:(CGPoint)PrevLoc TouchIndex:(int)TouchIndex Force:(float)Force Type:(TouchType)Type TouchesArray:(TArray<TouchInput>&)TouchesArray;
#endif

#if BUILD_EMBEDDED_APP
// startup UE before we have a view - so that we don't need block on Metal device creation, which can take .5-1.5 seconds!
+(void)StartupEmbeddedUnreal;
#endif

#if !PLATFORM_TVOS
// Gesture recognizer setup (called during view init)
- (void)SetupGestureRecognizers;

// Gesture callbacks
- (void)HandlePanGesture:(nonnull UIPanGestureRecognizer*)Recognizer;
- (void)HandlePinchGesture:(nonnull UIPinchGestureRecognizer*)Recognizer;
- (void)HandleRotateGesture:(nonnull UIRotationGestureRecognizer*)Recognizer;
- (void)HandleTapGesture:(nonnull UITapGestureRecognizer*)Recognizer;
- (void)HandleLongPressGesture:(nonnull UILongPressGestureRecognizer*)Recognizer;
#endif

@end


/**
 * A view controller subclass that handles loading our IOS view as well as autorotation
 */
#import <GameController/GameController.h>
// We use a GCEvent view controller to dynamically control how Game Controller API events flow in the responder chain, based on the presence of a physical keyboard or not
@interface IOSViewController : GCEventViewController

{
}

#if !PLATFORM_TVOS && !PLATFORM_VISIONOS
-(void)notifyPresentAfterRotateOrientationMask : (UIInterfaceOrientationMask)NewOrientationMask withSizeX : (unsigned int)SizeX withSizeY : (unsigned int)SizeY;

- (UIInterfaceOrientationMask)supportedInterfaceOrientations_Internal;
- (UIInterfaceOrientationMask)supportedInterfaceOrientations;
#endif

#ifndef SWIFT_IMPORT
- (TWeakPtr<FAppleKeyboardController>)GetKeyboardController;

- (bool)IsAnyPhysicalKeyboardConnected;

- (bool)ProcessKeyPresses:(nullable NSSet<UIPress *> *)keypresses eventType:(FAppleKeyboardController::EKeyEvent)eventType;

#endif

- (void)pressesBegan:(nullable NSSet<UIPress *> *)presses withEvent:(nullable UIPressesEvent *)event;
- (void)pressesChanged:(nullable NSSet<UIPress *> *)presses withEvent:(nullable UIPressesEvent *)event;
- (void)pressesEnded:(nullable NSSet<UIPress *> *)presses withEvent:(nullable UIPressesEvent *)event;
- (void)pressesCancelled:(nullable NSSet<UIPress *> *)presses withEvent:(nullable UIPressesEvent *)event;
- (bool)canBecomeFirstResponder;

@end
