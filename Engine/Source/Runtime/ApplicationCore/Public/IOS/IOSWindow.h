// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericWindow.h"
#include "GenericPlatform/GenericApplication.h"
#import <UIKit/UIKit.h>

/**
 * A platform specific implementation of FNativeWindow.
 * Native windows provide platform-specific backing for and are always owned by an SWindow.
 */
class APPLICATIONCORE_API FIOSWindow : public FGenericWindow
{
public:
	~FIOSWindow();

	/** Create a new FIOSWindow.
	 *
	 * @param OwnerWindow		The SlateWindow for which we are crating a backing IOSWindow
	 * @param InParent			Parent iOS window; usually NULL.
	 */
	static TSharedRef<FIOSWindow> Make();

	/** Returns a rect describing the main screen */
	static FPlatformRect GetScreenRect();

	/** Returns a void pointer to the hWnd (for other apis)*/
	virtual void* GetOSWindowHandle() const override { return Window; }

	void Initialize( class FIOSApplication* const Application, const TSharedRef< FGenericWindowDefinition >& InDefinition, const TSharedPtr< FIOSWindow >& InParent, const bool bShowImmediately );

	/** Returns the rectangle of the screen the window is associated with */
	virtual bool GetFullScreenInfo( int32& X, int32& Y, int32& Width, int32& Height ) const override;

	/** Callbacks for Cvar changes "r.MobileContentScaleFactor" and "r.Mobile.DesiredResX/Y" respectively */
	static void OnScaleFactorChanged(IConsoleVariable* CVar);
	static void OnConsoleResolutionChanged(IConsoleVariable* CVar);
		
protected:
	/** @return true if the native window is currently in fullscreen mode, false otherwise */
	virtual EWindowMode::Type GetWindowMode() const override { return EWindowMode::Fullscreen; }

private:
	/**
	 * Protect the constructor; only TSharedRefs of this class can be made.
	 */
	FIOSWindow();

	FIOSApplication* OwningApplication;

	/** iOS window handle, typically, only one should ever exist */
	UIWindow *Window;

	/** Store the window region size for querying whether a point lies within the window */
	int32 RegionX;
	int32 RegionY;
};
