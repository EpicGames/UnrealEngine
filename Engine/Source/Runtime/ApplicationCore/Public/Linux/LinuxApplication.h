// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Containers/Map.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "Misc/CoreMisc.h"
#include "GenericPlatform/GenericWindow.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "GenericPlatform/GenericWindowDefinition.h"
#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/IInputInterface.h"
#include "Linux/LinuxWindow.h"
#include "Linux/LinuxCursor.h"

class IInputDevice;
class IModularFeature;

class FLinuxApplication : public GenericApplication, public FSelfRegisteringExec, public IInputInterface
{
public:
	/**
	 * Stores window properties at the beginning of handling the events.
	 */
	struct FWindowProperties
	{
		/** Location at the moment of receiving events */
		FVector2D Location;

		/** Size at the moment of receiving events */
		FVector2D Size;
	};

	static FLinuxApplication* CreateLinuxApplication();

	enum UserDefinedEvents
	{
		CheckForDeactivation
	};

public:	
	virtual ~FLinuxApplication();
	
	virtual void DestroyApplication() override;

	// FSelfRegisteringExec
	virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;

public:
	virtual void SetMessageHandler( const TSharedRef< class FGenericApplicationMessageHandler >& InMessageHandler ) override;

	virtual void PollGameDeviceState( const float TimeDelta ) override;

	virtual void PumpMessages( const float TimeDelta ) override;

	virtual void ProcessDeferredEvents( const float TimeDelta ) override;

	virtual TSharedRef< FGenericWindow > MakeWindow() override;

	virtual void InitializeWindow( const TSharedRef< FGenericWindow >& Window, const TSharedRef< FGenericWindowDefinition >& InDefinition, const TSharedPtr< FGenericWindow >& InParent, const bool bShowImmediately ) override;

	virtual void SetCapture( const TSharedPtr< FGenericWindow >& InWindow ) override;

	virtual void* GetCapture( void ) const override;

	virtual void SetHighPrecisionMouseMode( const bool Enable, const TSharedPtr< FGenericWindow >& InWindow ) override;

	virtual bool IsUsingHighPrecisionMouseMode() const override { return bUsingHighPrecisionMouseInput; }

	virtual bool IsGamepadAttached() const override;
	
	virtual FModifierKeysState GetModifierKeys() const override;

	virtual FPlatformRect GetWorkArea( const FPlatformRect& CurrentWindow ) const override;

	virtual EWindowTransparency GetWindowTransparencySupport() const override
	{
		return EWindowTransparency::PerWindow;
	}

	void AddPendingEvent( SDL_Event event );

	void RemoveEventWindow(SDL_HWindow Window);

	void RemoveRevertFocusWindow(SDL_HWindow HWnd);

	void RaiseNotificationWindows( const TSharedPtr< FLinuxWindow >& ParentWindow);

	void RemoveNotificationWindow(SDL_HWindow HWnd);

	void CheckIfApplicatioNeedsDeactivation();

	EWindowZone::Type WindowHitTest( const TSharedPtr< FLinuxWindow > &window, int x, int y );

	TSharedPtr< FLinuxWindow > FindWindowBySDLWindow( SDL_Window *win );

	virtual bool IsCursorDirectlyOverSlateWindow() const override;

	APPLICATIONCORE_API virtual bool RequiresPerWindowTooltips() const override;

 	virtual TSharedPtr<FGenericWindow> GetWindowUnderCursor() override;

	/** Returns true if this application is foreground */
	inline bool IsForeground()
	{
		// if there are no windows, consider ourselves foreground so servers and commandlets aren't impacted
		return (Windows.Num() > 0) ? bActivateApp : true;
	}

	/**
	 * Windows can move during an event loop (like in
	 * FLinuxPlatformMisc::PumpMessages(), but SDL queues many events
	 * up before any windows move. This can lead to the screen-space
	 * position of the mouse cursor being calculated incorrectly with
	 * the old event data and new window location data. Use this to
	 * save the window locations for use during the loop.
	 */
	void SaveWindowPropertiesForEventLoop(void);

	/** Clear out data saved in SaveWindowLocationsForEventLoop(). */
	void ClearWindowPropertiesAfterEventLoop(void);

	/**
	 * Get a window position inside the event loop. Fall back on
	 * SDL_GetWindowPosition if not present in the saved window
	 * locations.
	 *
	 * @param NativeWindow The native SDL_HWindow to look up. This is
	 *					 a drop-in replacement for
	 *					 SDL_GetWindowPosition, so it uses the
	 *					 native window type.
	 * @param WindowProperties link to properties structure
	 */
	void GetWindowPropertiesInEventLoop(SDL_HWindow NativeWindow, FWindowProperties& Properties);

	virtual bool IsMouseAttached() const override;

	/**
	 * Returns the current active foreground window.
	 *
	 * @return pointer to the window, if any
	 */
	TSharedPtr< FLinuxWindow > GetCurrentActiveWindow() 
	{
		return CurrentlyActiveWindow;
	}

	/**
	 * Returns the current active foreground window.
	 *
	 * @return pointer to the window, if any
	 */
	TSharedPtr< FLinuxWindow > GetCurrentFocusWindow() 
	{
		return CurrentFocusWindow;
	}

private:

	FLinuxApplication();

	/**
	 * Finds a window associated with the event (if there is such an association).
	 *
	 * @param Event event in question
	 * @param bOutWindowlessEvent whether or not the event needs to have a window at all
	 *
	 * @return pointer to the window, if any
	 */
	TSharedPtr< FLinuxWindow > FindEventWindow(SDL_Event *Event, bool& bOutWindowlessEvent);

	void UpdateMouseCaptureWindow( SDL_HWindow TargetWindow );

	void ProcessDeferredMessage( SDL_Event Event );

	/** 
	 * Determines whether this particular SDL_KEYDOWN event should also be routed to OnKeyChar()
	 *
	 * @param KeyDownEvent event in question
	 *
	 * @return true if character needs to be passed to OnKeyChar
	 */
	static bool GeneratesKeyCharMessage(const SDL_KeyboardEvent & KeyDownEvent);

	/** Activate this Slate application */
	void ActivateApplication();

	/** Deactivate this Slate application */
	void DeactivateApplication();

	/** 
	 * Acivate the specified Window. That includes the deactivation of the previous window
	 * if one was active.
	 */
	void ActivateWindow(const TSharedPtr< FLinuxWindow >& Window);

	void ActivateRootWindow(const TSharedPtr< FLinuxWindow >& Window);

	TSharedPtr< FLinuxWindow > GetRootWindow(const TSharedPtr< FLinuxWindow >& Window);

	/** Handles "Cursor" exec commands" */
	bool HandleCursorCommand(const TCHAR* Cmd, FOutputDevice& Ar);

	/** Handles "Window" exec commands" */
	bool HandleWindowCommand(const TCHAR* Cmd, FOutputDevice& Ar);

private:

	void RefreshDisplayCache();

	/** Gets the location from a given touch event. */
	FVector2D GetTouchEventLocation(SDL_HWindow NativeWindow, SDL_Event TouchEvent);

	/** Searches for a free touch index. */
	int GetFirstFreeTouchId();

public:
	virtual IInputInterface* GetInputInterface() override
	{
		return this;
	}
	// IInputInterface overrides
	virtual void SetForceFeedbackChannelValue (int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) override;
	virtual void SetForceFeedbackChannelValues(int32 ControllerId, const FForceFeedbackValues &Values) override;
	virtual void SetHapticFeedbackValues(int32 ControllerId, int32 Hand, const FHapticFeedbackValues& Values) override;
	virtual void SetLightColor(int32 ControllerId, FColor Color) override { }
	virtual void ResetLightColor(int32 ControllerId) override { }

private:
	void AddGameController(SDL_JoystickID Id);
	void RemoveGameController(SDL_JoystickID Id);

	void OnInputDeviceModuleRegistered(const FName& Type, IModularFeature* ModularFeature);

	uint32 SDLDoubleClickTime;
	uint32 SDLDoubleClickRadius;

	const uint32 GetSDLDoubleClickTime();
	const uint32 GetSDLDoubleClickRadius();

	/** Stores context information about a currently active touch. */
	struct FTouchContext
	{
		/** Internal touch index (0-9 normally). */
		int TouchIndex;

		/** Device id */
		SDL_TouchID DeviceId;

		/** Last known location. */
		FVector2D Location;
	};

	/** Holds currently active touches (i.e. fingers pressed but not released) */
	TMap<uint64, FTouchContext> Touches;

	/** Maps touch indexes to SDL touch IDs. */
	TArray<TOptional<uint64>> TouchIds;

	struct SDLControllerState
	{
		/** SDL controller */
		SDL_Gamepad* Controller;

		/** Tracks whether the "button" was previously pressed so we don't generate extra events */
		bool AnalogOverThreshold[10];

		/** The input device Id of the controller that can be used to find the matching ULocalPlayer */
		FInputDeviceId DeviceId;

		/** Store axis values from events here to be handled once per frame. */
		TMap<FGamepadKeyNames::Type, float> AxisEvents;

		/** Current force feedback values */
		FForceFeedbackValues ForceFeedbackValues;

		SDLControllerState()
			:	Controller(nullptr)
			,	DeviceId(INPUTDEVICEID_NONE)
		{
			FMemory::Memzero(AnalogOverThreshold);
		}

		void UpdateHapticEffect();
	};

	TArray< SDL_Event > PendingEvents;

	TArray< TSharedRef< FLinuxWindow > > Windows;

	/** Array of notificaion windows to raise when activating toplevel window */ 
	TArray< TSharedRef< FLinuxWindow > > NotificationWindows;

	/** Array of windows to focus when current gets removed. */
	TArray< TSharedRef< FLinuxWindow > > RevertFocusStack;

	/**
	 * Saved window properties used for event loop, because the 
	 * These keys should not be de-referenced
	 * (but comparison is okay).
	 */
	TMap<SDL_HWindow, FWindowProperties> SavedWindowPropertiesForEventLoop;

	int32 bAllowedToDeferMessageProcessing;

	/** Using high precision mouse input */
	bool bUsingHighPrecisionMouseInput;

	/** TODO: describe */
	bool bIsMouseCursorLocked;

	/** TODO: describe */
	bool bIsMouseCaptureEnabled;

	/** True after every SDL_WINDOWEVENT_HIT_TEST until a following SDL_WINDOWEVENT_MOVED */
	bool bFirstFrameOfWindowMove = false;

	/** Used to detect a double click on the title bar area of the window */
	float LastTitleBarClickX;
	float LastTitleBarClickY;
	uint64 LastTitleBarClickTimestampNS;
	TSharedPtr< FLinuxWindow > LastTitleBarClickWindow;

	/**
	 * Timestamp (ns) of the last SDL_EVENT_WINDOW_HIT_TEST. Used to coalesce
	 * the duplicate HIT_TEST events that SDL3 X11 XInput2 emits for every
	 * Button1 press
	 */
	uint64 LastHitTestTimestampNS;

	/** Window that we think has been activated last. */
	TSharedPtr< FLinuxWindow > CurrentlyActiveWindow;
	TSharedPtr< FLinuxWindow > CurrentFocusWindow;
	TSharedPtr< FLinuxWindow > CurrentClipWindow;
	TSharedPtr< FLinuxWindow > CurrentUnderCursorWindow;

	/** Stores (unescaped) file URIs received during current drag-n-drop operation. */
	TArray<FString> DragAndDropQueue;

	/** Stores text received during current drag-n-drop operation. */
	TArray<FString> DragAndDropTextQueue;

	/** Window that we think has been previously active. */
	TSharedPtr< FLinuxWindow > PreviousActiveWindow;

	SDL_HWindow MouseCaptureWindow;

	TMap<SDL_JoystickID, SDLControllerState> ControllerStates;

	float fMouseWheelScrollAccel;

	/** List of input devices implemented in external modules. */
	TArray< TSharedPtr<class IInputDevice> > ExternalInputDevices;

	/** Whether input plugins have been loaded */
	bool bHasLoadedInputPlugins;

	/** Whether we entered one of our own windows */
	bool bInsideOwnWindow;

	/** This is used to assist the drag/drop on tabs. */
	bool bIsDragWindowButtonPressed;

	/** Used to check if the application is active or not. */
	bool bActivateApp;

	/** Time before deactivating the application if no FocusIn event happens on any of our Windows */
	double FocusOutDeactivationTime;

	struct FCachedDisplay
	{
		SDL_DisplayID	Id;
		SDL_Rect	UsableBounds;
	};

	// SDL caches full display bounds in-memory but re-fetches _NET_WORKAREA per call for usable bounds.
	mutable TArray<FCachedDisplay>	CachedDisplays;
	mutable double			LastTimeCachedDisplays;
};

extern FLinuxApplication* LinuxApplication;
