// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/OutputDeviceConsole.h"

/**
* Linux implementation of console log window, just a printf to the terminal for now
*/
class FLinuxConsoleOutputDevice : public FOutputDeviceConsole
{

private:

	/** true if the color is currently set by the caller */
	bool bOverrideColorSet;

	/** true if we are outputting to a terminal */
	bool bOutputtingToTerminal;

	/** true if window is visible */
	bool bIsWindowShown;

	/** true if stdout is set via the command line */
	bool bIsStdoutSet;

	/** maximum allowed log verbosity */
	ELogVerbosity::Type AllowedLogVerbosity = ELogVerbosity::All;

public:

	/**
	 * Constructor, setting console control handler.
	 */
	APPLICATIONCORE_API FLinuxConsoleOutputDevice();
	APPLICATIONCORE_API ~FLinuxConsoleOutputDevice();

	/**
	 * Shows or hides the console window.
	 *
	 * @param ShowWindow	Whether to show (true) or hide (false) the console window.
	 */
	APPLICATIONCORE_API virtual void Show(bool bShowWindow);

	/**
	 * Returns whether console is currently shown or not
	 *
	 * @return true if console is shown, false otherwise
	 */
	APPLICATIONCORE_API virtual bool IsShown();

	virtual bool IsAttached() {return false;}

	APPLICATIONCORE_API virtual bool CanBeUsedOnAnyThread() const override;
	APPLICATIONCORE_API virtual bool CanBeUsedOnPanicThread() const override;

	/**
	 * Displays text on the console and scrolls if necessary.
	 *
	 * @param Data	Text to display
	 * @param Event	Event type, used for filtering/ suppression
	 */
	APPLICATIONCORE_API void Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category);

};
