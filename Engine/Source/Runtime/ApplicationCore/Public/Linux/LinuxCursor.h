// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Vector2D.h"
#include "GenericPlatform/ICursor.h"
#include <SDL3/SDL.h>

typedef SDL_Cursor*		SDL_HCursor;

class FLinuxCursor : public ICursor
{
public:
	FLinuxCursor();

	virtual ~FLinuxCursor();

	virtual bool IsCreateCursorFromRGBABufferSupported() const override
	{
		return true;
	}

	virtual void* CreateCursorFromRGBABuffer(const FColor* Pixels, int32 Width, int32 Height, FVector2D InHotSpot) override;

	virtual FVector2D GetPosition() const override;

	virtual void SetPosition( const int32 X, const int32 Y ) override;

	virtual void SetType( const EMouseCursor::Type InNewCursor ) override;

	virtual EMouseCursor::Type GetType() const override
	{
		return CurrentType;
	}

	virtual void GetSize( int32& Width, int32& Height ) const override;

	virtual void Show( bool bShow ) override;

	virtual void Lock( const RECT* const Bounds ) override;

	virtual void SetTypeShape(EMouseCursor::Type InCursorType, void* CursorHandle) override;

public:

	/**
	 * Defines a custom cursor shape for the EMouseCursor::Custom type.
	 * 
	 * @param CursorHandle	A native cursor handle to show when EMouseCursor::Custom is selected.
	 */
	virtual void SetCustomShape( SDL_HCursor CursorHandle );

	bool IsHidden();

	/**
	 * Invalidates whatever cached data cursor may have
	 */
	void InvalidateCaches();

	/** Set the internal cached position, setting the cache to valid */
	void SetCachedPosition( const float X, const float Y );

private:

	EMouseCursor::Type CurrentType;
	bool bHidden;

	/** Cursors */
	SDL_HCursor CursorHandles[ EMouseCursor::TotalCursorCount ];

	/** Override Cursors */
	SDL_HCursor CursorOverrideHandles[ EMouseCursor::TotalCursorCount ];

	SDL_Rect CursorClipRect;

	/** Cached global X position */
	mutable float CachedGlobalXPosition;

	/** Cached global Y position */
	mutable float CachedGlobalYPosition;

	/** Whether mouse position cache is valid */
	mutable bool bPositionCacheIsValid;
};
