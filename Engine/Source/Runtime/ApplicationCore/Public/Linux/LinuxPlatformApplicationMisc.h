// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformApplicationMisc.h"

struct FLinuxPlatformApplicationMisc : public FGenericPlatformApplicationMisc
{
	static APPLICATIONCORE_API void PreInit();
	static APPLICATIONCORE_API void Init();
	static APPLICATIONCORE_API bool InitSDL();
	static APPLICATIONCORE_API void TearDown();
	static APPLICATIONCORE_API void LoadPreInitModules();
	static APPLICATIONCORE_API void LoadStartupModules();
	static APPLICATIONCORE_API uint64 WindowStyle();
	static APPLICATIONCORE_API class FOutputDeviceConsole* CreateConsoleOutputDevice();
	static APPLICATIONCORE_API class FOutputDeviceError* GetErrorOutputDevice();
	static APPLICATIONCORE_API class FFeedbackContext* GetFeedbackContext();
	static APPLICATIONCORE_API class GenericApplication* CreateApplication();
	static APPLICATIONCORE_API bool IsThisApplicationForeground();
	static APPLICATIONCORE_API void PumpMessages(bool bFromMainLoop);
	static APPLICATIONCORE_API bool IsScreensaverEnabled();
	static APPLICATIONCORE_API bool ControlScreensaver(EScreenSaverAction Action);
	static APPLICATIONCORE_API float GetDPIScaleFactorAtPoint(float X, float Y);
	static APPLICATIONCORE_API void ClipboardCopy(const TCHAR* Str);
	static APPLICATIONCORE_API void ClipboardPaste(class FString& Dest);
	static bool FullscreenSameAsWindowedFullscreen() { return true; }
	static APPLICATIONCORE_API bool IsUsingWayland();			// Is SDL using native Wayland video driver
	static APPLICATIONCORE_API bool IsUsingX11();			    // Is SDL using X11 video driver
	static APPLICATIONCORE_API bool RequiresParentedNotifications();	// True on pure Wayland (cannot position top-level windows)

	// Unix specific
	static APPLICATIONCORE_API void EarlyUnixInitialization(class FString& OutCommandLine);
	static bool ShouldIncreaseProcessLimits() { return true; }

	/** Mark a window for destruction once all RHI surface refs are released. The actual
	 *  SDL_DestroyWindow runs on the game thread in PumpMessages once the refcount reaches
	 *  zero (or a wall-clock backstop elapses). Replaces the old fixed-frame defer. */
	static APPLICATIONCORE_API void DeferDestroyWindow(struct SDL_Window* Window);

	/** Register that an RHI surface now backs this window. Called from the RHI right after a
	 *  Vulkan surface is created for the window. Thread-safe (RHI thread). Returns the SDL window
	 *  id the ref was recorded under (0 on failure). Release must be keyed on this immutable,
	 *  never-recycled id (the RHI carries it on the window context / swapchain) rather than on a
	 *  pointer that may be stale or reused by destroy time. */
	static APPLICATIONCORE_API uint32 AcquireWindowForRHI(void* SdlWindow);

	/** Release an RHI surface ref for the window previously acquired under WindowId. Called from
	 *  the RHI right after the Vulkan surface is destroyed. Thread-safe (RHI thread). Never destroys
	 *  the window itself. Keyed on the immutable, never-recycled SDL window id, so a release that
	 *  arrives after the window is already gone (e.g. via the destroy backstop) is a safe no-op and
	 *  can never decrement a different window's refcount. */
	static APPLICATIONCORE_API void ReleaseWindowForRHI(uint32 WindowId);

	/** Queue SDL_HideWindow for 3 PumpMessages ticks. Caller must set
	 *  UE.window.present_enabled=false on the window first. */
	static APPLICATIONCORE_API void DeferHideWindow(struct SDL_Window* Window);

	/** Remove a window from the deferred-hide queue. No-op if not queued. */
	static APPLICATIONCORE_API void CancelDeferredHideWindow(struct SDL_Window* Window);

	// Linux specific
	/** Informs ApplicationCore that it needs to create Vulkan-compatible windows (mutually exclusive with OpenGL) */
	static APPLICATIONCORE_API void UsingVulkan();
	/** Informs ApplicationCore that it needs to create OpenGL-compatible windows (mutually exclusive with Vulkan) */
	static APPLICATIONCORE_API void UsingOpenGL();
};

typedef FLinuxPlatformApplicationMisc FPlatformApplicationMisc;
