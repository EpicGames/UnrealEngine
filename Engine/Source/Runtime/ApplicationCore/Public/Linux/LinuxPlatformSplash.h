// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	LinuxSplash.h: Linux platform splash screen.
==============================================================================================*/

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformSplash.h"

/**
 * Linux splash implementation
 */
struct FLinuxPlatformSplash : public FGenericPlatformSplash
{
	/**
	 * Show the splash screen
	 */
	static APPLICATIONCORE_API void Show();
	/**
	 * Hide the splash screen
	 */
	static APPLICATIONCORE_API void Hide();

	/**
	 * Sets the text displayed on the splash screen (for startup/loading progress)
	 *
	 * @param	InType		Type of text to change
	 * @param	InText		Text to display
	 */
	using FGenericPlatformSplash::SetSplashText;
	static APPLICATIONCORE_API void SetSplashText(const SplashTextType::Type InType, const FText& InText);
};

typedef FLinuxPlatformSplash FPlatformSplash;
