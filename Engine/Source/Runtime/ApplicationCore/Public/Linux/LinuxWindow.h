// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"
#include "GenericPlatform/GenericWindow.h"

#include <SDL3/SDL.h>

class FLinuxApplication;
struct FGenericWindowDefinition;

DECLARE_LOG_CATEGORY_EXTERN( LogLinuxWindow, Log, All );
DECLARE_LOG_CATEGORY_EXTERN( LogLinuxWindowType, Log, All );
DECLARE_LOG_CATEGORY_EXTERN( LogLinuxWindowEvent, Log, All );

typedef SDL_Window*		SDL_HWindow;

enum class EWindowActivationPolicy;

/**
 * A platform specific implementation of FNativeWindow.
 * Native Windows provide platform-specific backing for and are always owned by an SWindow.
 */
class FLinuxWindow : public FGenericWindow
{
public:
	APPLICATIONCORE_API ~FLinuxWindow();

	/** Create a new SDLWindow.
	 *
	 * @param SlateWindows		List of all top level Slate windows.  This function will add the owner window to this list.
	 * @param OwnerWindow		The SlateWindow for which we are crating a backing Win32Window
	 * @param InHInstance		Win32 application instance handle
	 * @param InParent			Parent Win32 window; usually NULL.
	 * @param bShowImmediately	True to show this window as soon as its initialized
	 */
	static APPLICATIONCORE_API TSharedRef< FLinuxWindow > Make();

	APPLICATIONCORE_API SDL_HWindow GetHWnd() const;

	APPLICATIONCORE_API void Initialize( class FLinuxApplication* const Application, const TSharedRef< FGenericWindowDefinition >& InDefinition, const TSharedPtr< FLinuxWindow >& InParent, const bool bShowImmediately );

	/** TODO: describe */
	APPLICATIONCORE_API bool IsRegularWindow() const;

	/** TODO: describe */
	APPLICATIONCORE_API bool IsPopupMenuWindow() const;

	/** TODO: describe */
	APPLICATIONCORE_API bool IsTooltipWindow() const;

	/** TODO: describe */
	APPLICATIONCORE_API bool IsNotificationWindow() const;

	/** TODO: describe */
	APPLICATIONCORE_API bool IsTopLevelWindow() const;

	/** @return true if this is a modal window */
	APPLICATIONCORE_API bool IsModalWindow() const;

	/** TODO: describe */
	APPLICATIONCORE_API bool IsDialogWindow() const;

	/** TODO: describe */
	APPLICATIONCORE_API bool IsDragAndDropWindow() const;

	/** TODO: describe */
	APPLICATIONCORE_API bool IsUtilityWindow() const;

	/** @return the window activation policy used when showing the window */
	APPLICATIONCORE_API EWindowActivationPolicy GetActivationPolicy() const;

	/** TODO: describe */
	APPLICATIONCORE_API bool IsFocusWhenFirstShown() const;

	/** TODO: describe */
	APPLICATIONCORE_API const TSharedPtr< FLinuxWindow >& GetParent() const;

	/** Internal ID using for debugging */
	APPLICATIONCORE_API uint32 GetID() const;

	/** Debugging function - dumps window info to log */
	APPLICATIONCORE_API void LogInfo();

	APPLICATIONCORE_API bool IsNativeMoving() const;
	APPLICATIONCORE_API bool BeginNativeMove();
	APPLICATIONCORE_API void AfterNativeMove();
	APPLICATIONCORE_API void EndNativeMove();

	enum NativeResizeDirection 
	{
		ResizeSouthWest = 0,
		ResizeSouth,
		ResizeSouthEast,
		ResizeEast,
		ResizeNorthEast,
		ResizeNorth,
		ResizeNorthWest,
		ResizeWest,

		InvalidDirection
	};

	APPLICATIONCORE_API bool IsNativeResizing() const;
	APPLICATIONCORE_API bool BeginNativeResize( NativeResizeDirection Direction );
	APPLICATIONCORE_API void AfterNativeResize();
	APPLICATIONCORE_API void EndNativeResize();

public:
	APPLICATIONCORE_API virtual void ReshapeWindow( int32 X, int32 Y, int32 Width, int32 Height ) override;

	virtual void* GetOSWindowHandle() const  override { return HWnd; }

	//	not finished
	APPLICATIONCORE_API virtual bool GetFullScreenInfo( int32& X, int32& Y, int32& Width, int32& Height ) const override;

	APPLICATIONCORE_API virtual void MoveWindowTo ( int32 X, int32 Y ) override;

	APPLICATIONCORE_API virtual void BringToFront( bool bForce = false ) override;

	APPLICATIONCORE_API virtual void Destroy() override;

	APPLICATIONCORE_API virtual void Minimize() override;

	APPLICATIONCORE_API virtual void Maximize() override;

	APPLICATIONCORE_API virtual void Restore() override;

	APPLICATIONCORE_API virtual void Show() override;

	APPLICATIONCORE_API virtual void Hide() override;

	APPLICATIONCORE_API virtual void SetWindowMode( EWindowMode::Type NewWindowMode ) override;

	virtual EWindowMode::Type GetWindowMode() const override { return WindowMode; } 

	APPLICATIONCORE_API virtual bool IsMaximized() const override;

	APPLICATIONCORE_API virtual bool IsVisible() const override;

	APPLICATIONCORE_API virtual bool IsMinimized() const override;

	APPLICATIONCORE_API virtual bool GetRestoredDimensions(int32& X, int32& Y, int32& Width, int32& Height) override;

	APPLICATIONCORE_API virtual void SetWindowFocus() override;

	APPLICATIONCORE_API virtual void SetOpacity( const float InOpacity ) override;

	APPLICATIONCORE_API virtual void Enable( bool bEnable ) override;

	APPLICATIONCORE_API virtual bool IsPointInWindow( int32 X, int32 Y ) const override;

	APPLICATIONCORE_API virtual int32 GetWindowBorderSize() const override;

	APPLICATIONCORE_API virtual bool IsForegroundWindow() const override;
	/**
	 * Sets the window text - usually the title but can also be text content for things like controls
	 *
	 * @param Text	The window's title or content text
	 */
	APPLICATIONCORE_API virtual void SetText(const TCHAR* const Text) override;

	/** @return	Gives the native window a chance to adjust our stored window size before we cache it off */
	APPLICATIONCORE_API virtual void AdjustCachedSize( FVector2D& Size ) const override;

	virtual float GetDPIScaleFactor() const override
	{
		return DPIScaleFactor;
	}

	virtual void SetDPIScaleFactor(float Value) override
	{
		DPIScaleFactor = Value;
	}

	/**
	 * Returns window border sizes (frame extents).
	 * 
	 * @param BorderWidth - width of a vertical border (left side)
	 * @param BorderHeight - height of a horizontal border (from the top)
	 */
	APPLICATIONCORE_API void GetNativeBordersSize(int32& OutLeftBorderWidth, int32& OutTopBorderHeight) const;

	/**
	 *  Caches window properties that are too expensive to query all the time.
	 *  (e.g. border sizes)
	 */
	APPLICATIONCORE_API void CacheNativeProperties();

private:

	/**
	 * Protect the constructor; only TSharedRefs of this class can be made.
	 */
	APPLICATIONCORE_API FLinuxWindow();

private:

	FLinuxApplication* OwningApplication;

	/** linux window handle */
	SDL_HWindow HWnd;

	/** Store the window region size for querying whether a point lies within the window */
	int32 RegionWidth;
	int32 RegionHeight;
	
	/** The mode that the window is in (windowed, fullscreen, windowedfullscreen ) */
	EWindowMode::Type WindowMode;

	RECT PreFullscreenWindowRect;

	/** Virtual width and height of the window.  This is only different than the actual width and height for
	    windows which we're trying to optimize because their size changes frequently.  We'll create a larger
		window and have Windows draw it "cropped" so that it appears smaller, rather than actually resizing
		it and incurring a GPU buffer resize performance hit */
	int32 VirtualWidth;
	int32 VirtualHeight;

	/** TODO: describe */
	bool bIsVisible;

	/** TODO: describe */
	bool bWasFullscreen;

	/** TODO: describe */
	bool bIsPopupWindow;

	/** TODO: describe */
	bool bIsTooltipWindow;

	/** TODO: describe */
	bool bIsConsoleWindow;

	/** TODO: describe */
	bool bIsDialogWindow;

	/** TODO: describe */
	bool bIsNotificationWindow;

	/** TODO: describe */
	bool bIsTopLevelWindow;

	/** TODO: describe */
	bool bIsDragAndDropWindow;

	/** TODO: describe */
	bool bIsUtilityWindow;

	/** Whether the is inside this window */
	bool bIsPointerInsideWindow;

	/** SDL ID (for debugging purposes) */
	uint32 WindowSDLID;

	/** Parent window as given by Slate */
	TSharedPtr< FLinuxWindow > ParentWindow;
	static APPLICATIONCORE_API SDL_HitTestResult HitTest( SDL_Window *SDLwin, const SDL_Point *point, void *data );

	/**
	 * Converts screen-absolute coordinates to parent-relative coordinates for popup menu
	 * and tooltip windows. Walks the parent chain to accumulate offsets. For windows that
	 * are not parent-relative, X and Y are left unmodified.
	 *
	 * @param ThisWindow	The window whose position is being converted.
	 * @param X				In/out X coordinate to convert.
	 * @param Y				In/out Y coordinate to convert.
	 */
	static void ConvertToParentRelativePosition(SDL_Window* ThisWindow, int32& X, int32& Y);

	/**
	 * Gets the offset from screen-absolute to parent-relative coordinates during window
	 * creation. If Props indicates the window is not parent-relative, X and Y are set to 0.
	 * Use ConvertToParentRelativePosition() if the window already exists.
	 *
	 * @param Props			SDL properties for the window being created.
	 * @param StartParent	The parent window for the window being created.
	 * @param X				Out: X offset to apply.
	 * @param Y				Out: Y offset to apply.
	 */
	static void GetParentRelativeOffset(SDL_PropertiesID Props, SDL_Window* StartParent, int32& X, int32& Y);

	/** Cached width of the left border */
	int32 LeftBorderWidth;

	/** Cached height of the top border */
	int32 TopBorderHeight;

	/** Whether native properties cache is valid */
	bool bValidNativePropertiesCache;

	/**
	 * Ratio of pixels to SlateUnits in this window.
	 * E.g. DPIScale of 2.0 means there is a 2x2 pixel square for every 1x1 SlateUnit.
	 */
	float DPIScaleFactor;
};
