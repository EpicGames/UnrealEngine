// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if WITH_ACCESSIBILITY
#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"
#include "Mac/MacSystemIncludes.h"
#import <AppKit/AppKit.h>

/**
 * A custom NSView used to support accessibility.
 * This should be the base class for any custom views that wish to support accessibility
 */
@class FMacAccessibilityElement;

@interface FCocoaAccessibilityView : NSView
/**The FMacAccessibilityElement* that corresponds to this views backing window */
@property (nonatomic, assign) FMacAccessibilityElement* AccessibleWindow;

/**
 * Sets thee AccessibleWindow property. Used to create the accessibility hierarchy.
 * Setting up of the accessibility hierarchy allows VoiceOver to interact with thee AccessibleWindow and its contained elements.
 */
-(void) SetAccessibilityWindowAsAccessibilityChild:(AccessibleWidgetId) InWindowId;

/**
 * Clears the AccessibleWindow property and tears down the accessibility hierarchy for this Window.
 * This makes VOiceOVer unable to interact with the underlying elements in AccessibleWindow.
 */
-(void) RemoveAccessibilityWindow;

@end

#endif
