// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_ACCESSIBILITY

#include "CoreMinimal.h"
#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"
#include "Misc/Variant.h"
#include "Mac/MacSystemIncludes.h"

#import <AppKit/AppKit.h>


@interface FMacAccessibilityElement : NSAccessibilityElement
{
@private
	/** An array that holds all the accessibility children of this array. Is filled when the accessibilityChildren property is requested. */
	NSMutableArray* _AccessibilityChildren;
	/** The Accessibility Role of the element. E.g NSAccessibilityButtonRole, NSAccessibilityTextFieldRole etc */
	NSAccessibilityRole _AccessibilityRole;
}
- (id)initWithId:(AccessibleWidgetId)InId;

/** Check if LastCachedStringTime was updated recently. */
- (bool)ShouldCacheStrings;
/** Updates the label, help and value of the element */
- (void)CacheStrings;

/** Exposes this element to VoiceOver's accessibility hierarchy and allows the element to be navigated to from VoiceOver */
- (void)ExposeToVoiceOver:(bool)bShouldExpose;
/** The identifier used to access this widget through the accessible API. */
@property (nonatomic, assign) AccessibleWidgetId Id;
/** The identifier used to access this widget's parent through the accessible API. */
@property (nonatomic, assign) AccessibleWidgetId ParentId;
/** The identifier used to access this widget's owning window through the accessible API. */
@property (nonatomic, assign) AccessibleWidgetId OwningWindowId;;

/** A list of identifiers for all current children of this container. */
@property (nonatomic, assign) TArray<AccessibleWidgetId> ChildIds;
/** The bounding rect of the container. Note this is NOT a valid AABB in Cocoa screen space. See accessibilityFrame implementation */
@property (nonatomic) FBox2D Bounds;
/** A cached version of the name of the widget. */
@property (nonatomic) FString Label;
/** A cached version of the help text of the widget. */
@property (nonatomic) FString Help;
/** A cached version of the value of property widgets. */
@property (nonatomic) FVariant Value;
/** Timestamp for when Label, Help, and Value were last cached. */
@property (nonatomic, assign) double LastCachedStringTime;

@end

#endif
