// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_ACCESSIBILITY

#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"
#include "Mac/MacSystemIncludes.h"

@class FMacAccessibilityElement;

/**
 * This class is a singleton and should be accessed through [FMacAccessibilityManager AccessibilityManager].
 * Contains a Cache that stores a list of accessible elements that map to AccessibleWidgetIds for lookup.
 * The cache is also responsible for polling attributes from the underlying
 * IAccessibleWidgets that are too expensive to be done when requested by OSX due to
 * needing to be accessed from a different thread.
 *
 * The Accessibility Manager is also used to monitor changes in Voiceover status and call the appropriate functions
 * in FMacApplication using Key Value Observation (KVO).
  */
@interface FMacAccessibilityManager : NSObject
{
@private
	/** AccessibleWidgetId(String)->FMacAccessibilityElement map for all created accessibility elements. */
	NSMutableDictionary* Cache;
}

/** Pointer to the Mac Application that initialized this Singleton*/
@property (nonatomic) class FMacApplication* MacApplication;
/** Retrieve a cached element, or create one if it doesn't exist yet. */
- (FMacAccessibilityElement*)GetAccessibilityElement:(AccessibleWidgetId)Id;
/** Returns true if the Cache contains the Id. Does not create one if it doesn't exist. */
- (bool)AccessibilityElementExists:(AccessibleWidgetId)Id;
/** Returns the number of accessibility elements currently in the cache */
- (int32)GetAccessibilityCacheSize;
/** Returns true if the cache is empty */
- (bool)IsAccessibilityCacheEmpty;
/** Removes an entry from the Cache. */
- (void)RemoveAccessibilityElement:(AccessibleWidgetId)Id;
- (void)RemoveAccessibilitySubtree:(AccessibleWidgetId)RootId;
/** Completely empties the cache. */
- (void)Clear;
/** Loop over all cached elements and update any properties necessary on the Game thread. */
- (void)UpdateAllCachedProperties;
/** Used to clean up memory and unregister from KVO. This should ONLY be called from ~FMacapplication() */
- (void)TearDown;

/** Singleton accessor */
+ (FMacAccessibilityManager*)AccessibilityManager;

#if !UE_BUILD_SHIPPING
- (void)DumpAccessibilityStats;

- (void)PrintCache;
- (void)PrintAccessibilityElement:(FMacAccessibilityElement*) ELement;
- (void)PrintAccessibilityElementById:(AccessibleWidgetId) Id;
- (id)GetAccessibilityParent:(FMacAccessibilityElement*) Element;
- (FMacAccessibilityElement*)GetAccessibilityRoot;
- (bool)isElementOrphaned:(FMacAccessibilityElement*) Element;

#endif

@end

#endif
