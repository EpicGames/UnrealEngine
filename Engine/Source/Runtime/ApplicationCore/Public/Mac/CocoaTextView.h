// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Mac/MacSystemIncludes.h"
#include "GenericPlatform/ITextInputMethodSystem.h"
#if WITH_ACCESSIBILITY
#include "Mac/Accessibility/CocoaAccessibilityView.h"
#endif

#ifdef __OBJC__
//INherit from FCocoaAccessibilityView to allow descendants (FMetalView in particular) to be visible to Voiceover
#if WITH_ACCESSIBILITY
OBJC_EXPORT @interface FCocoaTextView : FCocoaAccessibilityView <NSTextInputClient>
#else
OBJC_EXPORT @interface FCocoaTextView : NSView <NSTextInputClient>
#endif
{
	TSharedPtr<ITextInputMethodContext> IMMContext;
	NSRange markedRange;
	bool reallyHandledEvent;
}

- (void)activateInputMethod:(const TSharedRef<ITextInputMethodContext>&)InContext;
- (void)deactivateInputMethod;
- (bool)isActiveInputMethod:(const TSharedRef<ITextInputMethodContext>&)InContext;

- (bool)imkKeyDown:(NSEvent*)Event;

//@protocol NSTextInputClient
//@required
/* The receiver inserts aString replacing the content specified by replacementRange. aString can be either an NSString or NSAttributedString instance.
 */
- (void)insertText:(id)aString replacementRange:(NSRange)replacementRange;

/* The receiver invokes the action specified by aSelector.
 */
- (void)doCommandBySelector:(SEL)aSelector;

/* The receiver inserts aString replacing the content specified by replacementRange. aString can be either an NSString or NSAttributedString instance. selectedRange specifies the selection inside the string being inserted; hence, the location is relative to the beginning of aString. When aString is an NSString, the receiver is expected to render the marked text with distinguishing appearance (i.e. NSTextView renders with -markedTextAttributes).
 */
- (void)setMarkedText:(id)aString selectedRange:(NSRange)selectedRange replacementRange:(NSRange)replacementRange;

/* The receiver unmarks the marked text. If no marked text, the invocation of this method has no effect.
 */
- (void)unmarkText;

/* Returns the selection range. The valid location is from 0 to the document length.
 */
- (NSRange)selectedRange;

/* Returns the marked range. Returns {NSNotFound, 0} if no marked range.
 */
- (NSRange)markedRange;

/* Returns whether or not the receiver has marked text.
 */
- (BOOL)hasMarkedText;

/* Returns attributed string specified by aRange. It may return nil. If non-nil return value and actualRange is non-NULL, it contains the actual range for the return value. The range can be adjusted from various reasons (i.e. adjust to grapheme cluster boundary, performance optimization, etc).
 */
- (NSAttributedString *)attributedSubstringForProposedRange:(NSRange)aRange actualRange:(NSRangePointer)actualRange;

/* Returns an array of attribute names recognized by the receiver.
 */
- (NSArray*)validAttributesForMarkedText;

/* Returns the first logical rectangular area for aRange. The return value is in the screen coordinate. The size value can be negative if the text flows to the left. If non-NULL, actuallRange contains the character range corresponding to the returned area.
 */
- (NSRect)firstRectForCharacterRange:(NSRange)aRange actualRange:(NSRangePointer)actualRange;

/* Returns the index for character that is nearest to aPoint. aPoint is in the screen coordinate system.
 */
- (NSUInteger)characterIndexForPoint:(NSPoint)aPoint;

//@optional
/* Returns an attributed string representing the receiver's document content. An NSTextInputClient can implement this interface if can be done efficiently. The caller of this interface can random access arbitrary portions of the receiver's content more efficiently.
 */
//- (NSAttributedString *)attributedString;

/* Returns the fraction of distance for aPoint from the left side of the character. This allows caller to perform precise selection handling.
 */
//- (CGFloat)fractionOfDistanceThroughGlyphForPoint:(NSPoint)aPoint;

/* Returns the baseline position relative to the origin of rectangle returned by -firstRectForCharacterRange:actualRange:. This information allows the caller to access finer-grained character position inside the NSTextInputClient document.
 */
//- (CGFloat)baselineDeltaForCharacterAtIndex:(NSUInteger)anIndex;

/* Returns the window level of the receiver. An NSTextInputClient can implement this interface to specify its window level if it is higher than NSFloatingWindowLevel.
 */
- (NSInteger)windowLevel;

/* Returns if the marked text is in vertical layout.
 */
//- (BOOL)drawsVerticallyForCharacterAtIndex:(NSUInteger)charIndex NS_AVAILABLE_MAC(10_6);

@end

#else // __OBJC__

class FCocoaTextView;

#endif // __OBJC__
