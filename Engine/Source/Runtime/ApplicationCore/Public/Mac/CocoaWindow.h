// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericWindow.h"
#include "GenericPlatform/GenericWindowDefinition.h"
#include "Mac/MacSystemIncludes.h"

#if WITH_ACCESSIBILITY
#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"
#endif


/**
 * Custom window class used for input handling
 */
@interface FCocoaWindow : NSWindow <NSWindowDelegate, NSDraggingDestination>
{
	EWindowMode::Type WindowMode;
	bool bRenderInitialized;
	bool bIsBeingOrderedFront;
	CGFloat Opacity;
@public
	bool bAcceptsInput;
	bool bZoomed;
	bool bIsOnActiveSpace;
	bool bIsBeingResized;
	bool bMinimized;
	bool bKeyWindow;
	NSScreen* CachedScreen;
	CGFloat CachedBackingScaleFactor;
}

@property (assign) EWindowMode::Type TargetWindowMode;
@property (assign,readonly,getter=AllowMainWindow) BOOL AllowMainWindow;
@property (assign) EWindowType Type;

#if WITH_ACCESSIBILITY
/** Adds the accessible element representing this window to the content view and exposes the window to accessibility interactions */
- (void)UpdateAccessibilityView:(AccessibleWidgetId) InAccessibilityWindowId;

/** removes the accessible element representing this window from the content view and prevents accessibility interactions being performed on the window */
- (void)ClearAccessibilityView;
#endif
/** Get the frame filled by a child OpenGL view, which may cover the window or fill the content view depending upon the window style.
 @return The NSRect for a child OpenGL view. */
- (NSRect)openGLFrame;

/** Get the view used for OpenGL rendering. @return The OpenGL view for rendering. */
- (NSView*)openGLView;

/** Lets window know if its owner (SWindow) accepts input */
- (void)setAcceptsInput:(bool)InAcceptsInput;

/** Set the initial window mode. */
- (void)setWindowMode:(EWindowMode::Type)WindowMode;

/**	@return The current mode for this Cocoa window. */
- (EWindowMode::Type)windowMode;

/** Order window to the front. */
- (void)orderFrontAndMakeMain:(bool)bMain andKey:(bool)bKey;

- (void)startRendering;
- (bool)isRenderInitialized;

@end

extern NSString* NSDraggingExited;
extern NSString* NSDraggingUpdated;
extern NSString* NSPrepareForDragOperation;
extern NSString* NSPerformDragOperation;
