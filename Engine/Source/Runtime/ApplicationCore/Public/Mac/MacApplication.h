// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Apple/ScopeAutoreleasePool.h"
#include "CoreMinimal.h"
#include "GenericPlatform/GenericApplication.h"
#include "Mac/MacWindow.h"
#include "Mac/MacTextInputMethodSystem.h"
#include "Misc/CoreDelegates.h"
#include "GenericPlatform/IInputInterface.h"
#include <atomic>

class FAppleMouseController;
class IModularFeature;

struct FDeferredMacEvent
{
	FDeferredMacEvent()
	:	Event(nullptr)
	,	Window(nullptr)
	,	Type(0)
	,	LocationInWindow(FVector2D::ZeroVector)
	,	ModifierFlags(0)
	,	Timestamp(0.0)
	,	WindowNumber(0)
	,	Context(nullptr)
	,	Delta(FVector2D::ZeroVector)
	,	ScrollingDelta(FVector2D::ZeroVector)
	,	ButtonNumber(0)
	,	ClickCount(0)
	,	Phase(NSEventPhaseNone)
	,	MomentumPhase(NSEventPhaseNone)
	,	IsDirectionInvertedFromDevice(false)
	,	Character(0)
	,	CharCode(0)
	,	IsRepeat(false)
	,	IsPrintable(false)
	,	KeyCode(0)
	,	NotificationName(nullptr)
	,	DraggingPasteboard(nullptr)
	{
	}

	FDeferredMacEvent(const FDeferredMacEvent& Other)
	:	Event(Other.Event ? [Other.Event retain] : nullptr)
	,	Window(Other.Window)
	,	Type(Other.Type)
	,	LocationInWindow(Other.LocationInWindow)
	,	ModifierFlags(Other.ModifierFlags)
	,	Timestamp(Other.Timestamp)
	,	WindowNumber(Other.WindowNumber)
	,	Context(Other.Context ? [Other.Context retain] : nullptr)
	,	Delta(Other.Delta)
	,	ScrollingDelta(Other.ScrollingDelta)
	,	ButtonNumber(Other.ButtonNumber)
	,	ClickCount(Other.ClickCount)
	,	Phase(Other.Phase)
	,	MomentumPhase(Other.MomentumPhase)
	,	IsDirectionInvertedFromDevice(Other.IsDirectionInvertedFromDevice)
	,	Character(Other.Character)
	,	CharCode(Other.CharCode)
	,	IsRepeat(Other.IsRepeat)
	,	IsPrintable(Other.IsPrintable)
	,	KeyCode(Other.KeyCode)
	,	NotificationName(Other.NotificationName ? [Other.NotificationName retain] : nullptr)
	,	DraggingPasteboard(Other.DraggingPasteboard ? [Other.DraggingPasteboard retain] : nullptr)
	{
	}

	~FDeferredMacEvent()
	{
		SCOPED_AUTORELEASE_POOL;
		if (Event)
		{
			[Event release];
		}
		if (Context)
		{
			[Context release];
		}
		if (NotificationName)
		{
			[NotificationName release];
		}
		if (DraggingPasteboard)
		{
			[DraggingPasteboard release];
		}
	}

	// Using NSEvent on the game thread is unsafe, so we copy of all its properties and use them when processing the event.
	// However, in some cases we need the original NSEvent (highlighting menus, resending unhandled key events), so we store it as well.
	NSEvent* Event;

	FCocoaWindow* Window;

	int32 Type;
	FVector2D LocationInWindow;
	uint32 ModifierFlags;
	NSTimeInterval Timestamp;
	int32 WindowNumber;
	NSGraphicsContext* Context;
	FVector2D Delta;
	FVector2D ScrollingDelta;
	int32 ButtonNumber;
	int32 ClickCount;
	NSEventPhase Phase;
	NSEventPhase MomentumPhase;
	bool IsDirectionInvertedFromDevice;
	TCHAR Character;
	TCHAR CharCode;
	bool IsRepeat;
	bool IsPrintable;
	uint32 KeyCode;

	NSString* NotificationName;
	NSPasteboard* DraggingPasteboard;
};

struct FMacScreen
{
	NSScreen* Screen;
	NSRect Frame;
	NSRect VisibleFrame;
	NSRect FramePixels;
	NSRect VisibleFramePixels;
	NSEdgeInsets SafeAreaInsets;

	FMacScreen(NSScreen* InScreen) : Screen([InScreen retain]), Frame(InScreen.frame), VisibleFrame(InScreen.visibleFrame),
									 FramePixels(InScreen.frame), VisibleFramePixels(InScreen.visibleFrame),
									 SafeAreaInsets(InScreen.safeAreaInsets) {}
	~FMacScreen() { [Screen release]; }
};
typedef TSharedRef<FMacScreen, ESPMode::ThreadSafe> FMacScreenRef;

/**
 * Mac-specific application implementation.
 */
class APPLICATIONCORE_API FMacApplication 
	: public GenericApplication
	, public IInputInterface
{
public:

	static FMacApplication* CreateMacApplication();

public:	

	~FMacApplication();

public:

	virtual void SetMessageHandler(const TSharedRef<class FGenericApplicationMessageHandler>& InMessageHandler) override;
#if WITH_ACCESSIBILITY
	virtual void SetAccessibleMessageHandler(const TSharedRef<FGenericAccessibleMessageHandler>& InAccessibleMessageHandler) override;
	
	/** Called when Voiceover is enabled. FMacAccessibilityManager should be the only class to call this */
	void OnVoiceoverEnabled();
	
	/** Called when Voiceover is disabled. FMacAccessibilityManager should be the only class to call this */
	void OnVoiceoverDisabled();
#endif


	virtual void PollGameDeviceState(const float TimeDelta) override;

	virtual void PumpMessages(const float TimeDelta) override;

	virtual void ProcessDeferredEvents(const float TimeDelta) override;

	void DismissAllMenusOnGameThread();

	virtual TSharedRef<FGenericWindow> MakeWindow() override;

	virtual void InitializeWindow(const TSharedRef<FGenericWindow>& Window, const TSharedRef<FGenericWindowDefinition>& InDefinition, const TSharedPtr<FGenericWindow>& InParent, const bool bShowImmediately) override;

	virtual FModifierKeysState GetModifierKeys() const override;

	virtual bool IsCursorDirectlyOverSlateWindow() const override;

	virtual TSharedPtr<FGenericWindow> GetWindowUnderCursor() override;

	virtual void SetHighPrecisionMouseMode(const bool Enable, const TSharedPtr<FGenericWindow>& InWindow) override;

	virtual bool IsUsingHighPrecisionMouseMode() const override { return bUsingHighPrecisionMouseInput; }

	virtual bool IsUsingTrackpad() const override { return bUsingTrackpad; }

	virtual bool IsGamepadAttached() const override;

	virtual FPlatformRect GetWorkArea(const FPlatformRect& CurrentWindow) const override;

	virtual EWindowTitleAlignment::Type GetWindowTitleAlignment() const override { return EWindowTitleAlignment::Center; }

	virtual EWindowTransparency GetWindowTransparencySupport() const override { return EWindowTransparency::PerWindow; }

	virtual ITextInputMethodSystem *GetTextInputMethodSystem() override { return TextInputMethodSystem.Get(); }

#if WITH_EDITOR
	virtual void SendAnalytics(IAnalyticsProvider* Provider) override;

	void StartScopedModalEvent(const FCoreDelegates::FModalWindowContext& Context);

	void EndScopedModalEvent(const FCoreDelegates::FModalWindowContext& Context);
#endif

public:
	void ResendLastEvent();

	void CloseWindow(TSharedRef<FMacWindow> Window);

	void DeferEvent(NSObject* Object);

	bool IsProcessingDeferredEvents() const { return bIsProcessingDeferredEvents; }

	TSharedPtr<FMacWindow> FindWindowByNSWindow(FCocoaWindow* WindowHandle);
	
	void OnWindowWillResize(TSharedRef<FMacWindow> Window);

	/** Queues a window for text layout invalidation when safe */
	void InvalidateTextLayout(FCocoaWindow* Window);

	void ResetModifierKeys() { ModifierKeysFlags = 0; }

	/** Restore mouse input state after macOS state transitions (sleep, minimize, app switch)
	 *  that may silently reset CGAssociate or invalidate GCMouse handlers. */
	void ReassociateMouseInput();

	bool IsWorkspaceSessionActive() const { return bIsWorkspaceSessionActive; }

	void SystemModalMode(bool const bInSystemModalMode) { bSystemModalMode = bInSystemModalMode; }

	const TArray<TSharedRef<FMacWindow>>& GetAllWindows() const { return Windows; }

	FCriticalSection& GetWindowsArrayMutex() { return WindowsMutex; }

	void OnCursorLock();

	void IgnoreMouseMoveDelta() { FPlatformAtomics::InterlockedExchange(&bIgnoreMouseMoveDelta, 1); }

	void SetIsRightClickEmulationEnabled(bool bEnabled) { bIsRightClickEmulationEnabled = bEnabled; }

	void OnWindowDidResize(TSharedRef<FMacWindow> Window, bool bRestoreMouseCursorLocking = false);

	void OnWindowChangedScreen(TSharedRef<FMacWindow> Window);

	void OnWindowOrderedFront(TSharedRef<FMacWindow> Window);

	void OnWindowActivationChanged(const TSharedRef<FMacWindow>& Window, const EWindowActivation ActivationType);

	static void OnDisplayReconfiguration(CGDirectDisplayID Display, CGDisplayChangeSummaryFlags Flags, void* UserInfo);
	
	inline bool IsDisplayReconfiguring() const { return bDisplayReconfiguring; }

public:
    virtual IInputInterface* GetInputInterface() override { return this; }
    
	// IInputInterface overrides

	virtual void SetForceFeedbackChannelValue (int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) override;
	virtual void SetForceFeedbackChannelValues(int32 ControllerId, const FForceFeedbackValues &Values) override;
	virtual void SetHapticFeedbackValues(int32 ControllerId, int32 Hand, const FHapticFeedbackValues& Values) override;
	virtual void SetLightColor(int32 ControllerId, FColor Color) override;
	virtual void ResetLightColor(int32 ControllerId) override;
	virtual void SetDeviceProperty(int32 ControllerId, const FInputDeviceProperty* Property) override;
public:

	static void UpdateScreensArray();

	static FMacScreenRef FindScreenBySlatePosition(double X, double Y);

	static FMacScreenRef FindScreenByCocoaPosition(double X, double Y);

	static FVector2D ConvertSlatePositionToCocoa(double X, double Y);

	static FVector2D ConvertCocoaPositionToSlate(double X, double Y);

	static CGPoint ConvertSlatePositionToCGPoint(double X, double Y);

	static FVector2D CalculateScreenOrigin(NSScreen* Screen);

	static float GetPrimaryScreenBackingScaleFactor();

	typedef void (*MenuBarShutdownFuncPtr)();
	static MenuBarShutdownFuncPtr MenuBarShutdownFunc;
	
	static unichar TranslateKeyCodeToUniCode(uint32 KeyCode, uint32 Modifier);

private:

	static NSEvent* HandleNSEvent(NSEvent* Event);
#if WITH_EDITOR
	static int32 MTContactCallback(void* Device, void* Data, int32 NumFingers, double TimeStamp, int32 Frame);
#endif

	FMacApplication();

	void ProcessEvent(const FDeferredMacEvent& Event);
	void ResendEvent(NSEvent* Event);

	void ProcessMouseMovedEvent(const FDeferredMacEvent& Event, TSharedPtr<FMacWindow> EventWindow);
	void ProcessMouseDownEvent(const FDeferredMacEvent& Event, TSharedPtr<FMacWindow> EventWindow);
	void ProcessMouseUpEvent(const FDeferredMacEvent& Event, TSharedPtr<FMacWindow> EventWindow);
	void ProcessScrollWheelEvent(const FDeferredMacEvent& Event, TSharedPtr<FMacWindow> EventWindow);
	void ProcessGestureEvent(const FDeferredMacEvent& Event);
	void ProcessKeyDownEvent(const FDeferredMacEvent& Event, TSharedPtr<FMacWindow> EventWindow);
	void ProcessKeyUpEvent(const FDeferredMacEvent& Event);

	void OnWindowDidMove(TSharedRef<FMacWindow> Window);
	bool OnWindowDestroyed(TSharedRef<FMacWindow> DestroyedWindow);
	void OnWindowActivated(TSharedRef<FMacWindow> Window);

	void OnApplicationDidBecomeActive();
	void OnApplicationWillResignActive();
	void OnWindowsReordered();
	void OnActiveSpaceDidChange();
	
	void CacheKeyboardInputSource();

	void ConditionallyUpdateModifierKeys(const FDeferredMacEvent& Event);
	void HandleModifierChange(NSUInteger NewModifierFlags, NSUInteger FlagsShift, NSUInteger UEShift, EMacModifierKeys TranslatedCode);

	FCocoaWindow* FindEventWindow(NSEvent* CocoaEvent) const;
	FCocoaWindow* FindSlateWindowUnderCursor() const;
	EWindowZone::Type GetCurrentWindowZone(const TSharedRef<FMacWindow>& Window) const;
	bool IsEdgeZone(EWindowZone::Type Zone) const;
	bool IsPrintableKey(uint32 Character) const;
	TCHAR ConvertChar(TCHAR Character) const;
	TCHAR TranslateCharCode(TCHAR CharCode, uint32 KeyCode) const;

	void CloseQueuedWindows();

	/** Invalidates all queued windows requiring text layout changes */
	void InvalidateTextLayouts();

#if WITH_EDITOR
	void RecordUsage(EGestureEvent Gesture);
#else
	void RecordUsage(EGestureEvent Gesture) { }
#endif

#if WITH_ACCESSIBILITY
	void OnAccessibleEventRaised(const FAccessibleEventArgs& Args);
	#endif

	void OnInputDeviceModuleRegistered(const FName& Type, IModularFeature* ModularFeature);

private:

	bool bUsingHighPrecisionMouseInput;

	bool bUsingTrackpad;

	EMouseButtons::Type LastPressedMouseButton;

	FCriticalSection EventsMutex;
	TArray<FDeferredMacEvent> DeferredEvents;

	FCriticalSection WindowsMutex;
	TArray<TSharedRef<FMacWindow>> Windows;

	bool bIsProcessingDeferredEvents;

	struct FSavedWindowOrderInfo
	{
		int32 WindowNumber;
		int32 Level;
		FSavedWindowOrderInfo(int32 InWindowNumber, int32 InLevel) : WindowNumber(InWindowNumber), Level(InLevel) {}
	};
	TArray<FSavedWindowOrderInfo> SavedWindowsOrder;
	
	TSharedRef<class FMacControllerInterface> HIDInput;

	TSharedPtr<FAppleMouseController> MouseController;
	
	void HandleMouseControllerMovement(FIntVector2 Delta);

	/** List of input devices implemented in external modules. */
	TArray<TSharedPtr<class IInputDevice>> ExternalInputDevices;
	bool bHasLoadedInputPlugins;

	FCocoaWindow* DraggedWindow;
	FCocoaWindow* WindowUnderCursor;

	TSharedPtr<FMacWindow> ActiveWindow;
    uint32 SystemResolutionX;
    uint32 SystemResolutionY;

	bool bSystemModalMode;

	/** The current set of modifier keys that are pressed. This is used to detect differences between left and right modifier keys on key up events*/
	uint32 ModifierKeysFlags;

	/** The current set of Cocoa modifier flags, used to detect when Mission Control has been invoked & returned so that we can synthesis the modifier events it steals */
	NSUInteger CurrentModifierFlags;

	bool bIsRightClickEmulationEnabled;
	bool bEmulatingRightClick;

	volatile int32 bIgnoreMouseMoveDelta;
	std::atomic<bool> bMouseInputNeedsReassociation;

	FCriticalSection WindowsToCloseMutex;
	TArray<FCocoaWindow*> CocoaWindowsToClose;
	TArray<TSharedRef<FMacWindow>> SlateWindowsToClose;

	TArray<FCocoaWindow*> WindowsRequiringTextInvalidation;

	TSharedPtr<FMacTextInputMethodSystem> TextInputMethodSystem;

	bool bIsWorkspaceSessionActive;

	/** Notification center observers */
	id AppActivationObserver;
	id AppDeactivationObserver;
	id WorkspaceActivationObserver;
	id WorkspaceDeactivationObserver;
	id WorkspaceActiveSpaceChangeObserver;
	id ScreensDidWakeObserver;
	id SystemDidWakeObserver;

	id EventMonitor;
	id MouseMovedEventMonitor;

#if WITH_EDITOR
	/** Holds the last gesture used to try and capture unique uses for gestures. */
	EGestureEvent LastGestureUsed;

	/** Stores the number of times a gesture has been used for analytics */
	int32 GestureUsage[(int32)EGestureEvent::Count];
#endif

	NSData* KeyBoardLayoutData;

#if WITH_ACCESSIBILITY
	/** Timer used to refresh the accessibility cache */
	NSTimer* AccessibilityCacheTimer;
	
	NSTimer* AccessibilityAnnouncementDelayTimer;
#endif
	
	NSEvent* LastNativeEvent;
	
	bool bDisplayReconfiguring;

	friend class FMacWindow;
};

APPLICATIONCORE_API extern FMacApplication* MacApplication;
