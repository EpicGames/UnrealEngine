// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/OutputDeviceConsole.h"
#include "Mac/MacSystemIncludes.h"

@interface FMacConsoleWindow : NSWindow<NSWindowDelegate>
@end

/**
 * Mac implementation of console log window, utilizing the Win32 console API
 */
class APPLICATIONCORE_API FMacConsoleOutputDevice : public FOutputDeviceConsole
{
private:
	/** Handle to the console log window */
	FMacConsoleWindow* ConsoleHandle;

	/** Handle to window's content view */
	NSTextView* TextView;

	/** Handle to window's scroll view */
	NSScrollView* ScrollView;

	/** Handle text color of TextView */
	NSDictionary* TextViewTextColor;

	/** Critical section for Serialize() */
	FCriticalSection CriticalSection;

	/** Number of outstanding dispatched Cocoa tasks */
	uint64 OutstandingTasks;

	/** Maximum allowed log verbosity */
	ELogVerbosity::Type AllowedLogVerbosity = ELogVerbosity::All;

	/**
	 * Saves the console window's position and size to the game .ini
	 */
	void SaveToINI();

	/**
	 * Creates console window
	 */
	void CreateConsole();

	/**
	 * Destroys console window
	 */
	void DestroyConsole();

	/**
	 *
	 */
	void SetDefaultTextColor();

public:

	/** 
	 * Constructor, setting console control handler.
	 */
	FMacConsoleOutputDevice();
	~FMacConsoleOutputDevice();

	/**
	 * Shows or hides the console window. 
	 *
	 * @param ShowWindow	Whether to show (true) or hide (false) the console window.
	 */
	virtual void Show( bool ShowWindow );

	/** 
	 * Returns whether console is currently shown or not
	 *
	 * @return true if console is shown, false otherwise
	 */
	virtual bool IsShown();

	virtual bool IsAttached() { return false; }

	/**
	 * Displays text on the console and scrolls if necessary.
	 *
	 * @param Data	Text to display
	 * @param Event	Event type, used for filtering/ suppression
	 */
	void Serialize( const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category );
};
