// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/ICursor.h"
#include "Math/IntRect.h"
#include "Math/IntVector.h"
#include "Mac/MacSystemIncludes.h"

@class FCocoaWindow;

class FMacCursor : public ICursor
{
public:

	FMacCursor();

	virtual ~FMacCursor();

	virtual void* CreateCursorFromFile(const FString& InPathToCursorWithoutExtension, FVector2D HotSpot) override;

	virtual bool IsCreateCursorFromRGBABufferSupported() const override { return true; }

	virtual void* CreateCursorFromRGBABuffer(const FColor* Pixels, int32 Width, int32 Height, FVector2D InHotSpot) override;

	virtual FVector2D GetPosition() const override;

	virtual void SetPosition(const int32 X, const int32 Y) override;

	virtual void SetType(const EMouseCursor::Type InNewCursor) override;

	virtual EMouseCursor::Type GetType() const override { return CurrentType; }

	virtual void GetSize(int32& Width, int32& Height) const override;

	virtual void Show(bool bShow) override;

	virtual void Lock(const RECT* const Bounds) override;

	virtual void SetTypeShape(EMouseCursor::Type InCursorType, void* CursorHandle) override;

public:

	bool UpdateCursorClipping(FIntVector2& CursorPosition);

	void WarpCursor(const int32 X, const int32 Y);

	FIntVector2 GetIntPosition() const;

	FIntVector2 GetMouseWarpDelta();

	void SetHighPrecisionMouseMode(const bool bEnable);

	void UpdateCurrentPosition(const FIntVector2& Position);

	void UpdateVisibility();

	bool IsLocked() const { return CursorClipRect.Area() > 0; }

	void SetShouldIgnoreLocking(bool bIgnore) { bShouldIgnoreLocking = bIgnore; }

private:

	EMouseCursor::Type CurrentType;

	NSCursor* CursorHandles[EMouseCursor::TotalCursorCount];
	NSCursor* CursorOverrideHandles[EMouseCursor::TotalCursorCount];

	FIntRect CursorClipRect;

	bool bIsVisible;
	bool bUseHighPrecisionMode;
	NSCursor* CurrentCursor;
	int32 CursorTypeOverride;

	FIntVector2 CurrentPosition;
	FIntVector2 MouseWarpDelta;
	bool bIsPositionInitialised;
	bool bShouldIgnoreLocking;

	io_object_t HIDInterface;
	double SavedAcceleration;
};
