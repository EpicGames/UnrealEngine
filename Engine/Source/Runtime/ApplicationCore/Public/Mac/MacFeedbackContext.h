// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/FeedbackContext.h"
#include "Misc/OutputDeviceConsole.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/OutputDeviceRedirector.h"

/**
 * Feedback context implementation for Mac.
 */
class FMacFeedbackContext : public FFeedbackContext
{
	/** Context information for warning and error messages */
	FContextSupplier* Context = nullptr;

public:
	bool YesNof(const FText& Question) override
	{
		return FPlatformMisc::MessageBoxExt(EAppMsgType::YesNo, *Question.ToString(), TEXT("")) == EAppReturnType::Yes;
	}

	FContextSupplier* GetContext() const override { return Context; }
	void SetContext(FContextSupplier* InContext) override { Context = InContext; }
};
