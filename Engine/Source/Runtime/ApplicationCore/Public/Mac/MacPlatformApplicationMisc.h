// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformApplicationMisc.h"

#ifdef __OBJC__
@protocol NSObject;
#endif

typedef void (*MacApplicationExternalCb)(void);
typedef void (*MacApplicationExternalCbOneBool)(bool);

struct APPLICATIONCORE_API FMacPlatformApplicationMisc : public FGenericPlatformApplicationMisc
{
	static void PreInit();
	static void PostInit();
	static void TearDown();
	static void LoadPreInitModules();
	static class FOutputDeviceConsole* CreateConsoleOutputDevice();
	static class FOutputDeviceError* GetErrorOutputDevice();
	static class FFeedbackContext* GetFeedbackContext();
	static class GenericApplication* CreateApplication();
	static void RequestMinimize();
	static bool IsThisApplicationForeground();
	static bool IsScreensaverEnabled();
	static bool ControlScreensaver(EScreenSaverAction Action);
	static void ActivateApplication();
    static void UpdateApplicationMenu();
    static void LanguageChanged();
	static void UpdateWindowMenu();
    static void UpdateCocoaWindows();
	static struct FLinearColor GetScreenPixelColor(const FVector2D& InScreenPos, float InGamma = 1.0f);
	static float GetDPIScaleFactorAtPoint(float X, float Y);
	static bool IsHighDPIModeEnabled() { return bIsHighResolutionCapable && IsHighDPIAwarenessEnabled(); }
	static void PumpMessages(bool bFromMainLoop);
	static void ClipboardCopy(const TCHAR* Str);
	static void ClipboardPaste(class FString& Dest);
	
	static void* GetActiveWindowPtr();
	
	// MAC ONLY
	static void ProcessAsyncTasks();
	static bool IsMainThread();

	static MacApplicationExternalCb UpdateCachedMacMenuStateCb;
	static MacApplicationExternalCb PostInitMacMenuStartupCb;
	static MacApplicationExternalCbOneBool UpdateApplicationMenuCb;
	static MacApplicationExternalCbOneBool UpdateWindowMenuCb;
	static MacApplicationExternalCb LanguageChangedCb;

	static bool bChachedMacMenuStateNeedsUpdate;
	
	static bool bLanguageChanged;
    
    static bool bMacApplicationModalMode;

	static bool bIsHighResolutionCapable;

	static id<NSObject> CommandletActivity;

private:
	static bool bDisplaySleepEnabled;
};

typedef FMacPlatformApplicationMisc FPlatformApplicationMisc;
