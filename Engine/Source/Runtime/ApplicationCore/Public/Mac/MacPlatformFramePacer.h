// Copyright Epic Games, Inc. All Rights Reserved.


/*=============================================================================================
	MacPlatformFramePacer.h: Apple Mac platform frame pacer classes.
==============================================================================================*/


#pragma once
#include "GenericPlatform/GenericPlatformFramePacer.h"

struct FMacFramePacer;

typedef void (^FMacFramePacerHandler)(uint32 CGDirectDisplayID, double OutputSeconds, double OutputDuration);

/**
 * Mac implementation of FGenericPlatformRHIFramePacer
 **/
struct APPLICATIONCORE_API FMacPlatformRHIFramePacer : public FGenericPlatformRHIFramePacer
{
    // FGenericPlatformRHIFramePacer interface
    static bool IsEnabled();
	static void InitWithEvent(class FEvent* TriggeredEvent);
	static void AddHandler(FMacFramePacerHandler Handler);
	static void AddEvent(uint32 CGDirectDisplayID, class FEvent* TriggeredEvent);
	static void RemoveHandler(FMacFramePacerHandler Handler);
    static void Destroy();
    
    /** Access to the Mac Frame Pacer: CVDisplayLink */
    static FMacFramePacer* FramePacer;
};


typedef FMacPlatformRHIFramePacer FPlatformRHIFramePacer;
typedef FMacFramePacerHandler FPlatformRHIFramePacerHandler;
