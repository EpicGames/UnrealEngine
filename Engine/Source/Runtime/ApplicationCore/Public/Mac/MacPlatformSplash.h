// Copyright Epic Games, Inc. All Rights Reserved.
/*=============================================================================================
	MacPlatformSplash.h: Mac platform splash screen...
==============================================================================================*/

#pragma once
#include "GenericPlatform/GenericPlatformSplash.h"

/**
* Mac splash implementation 
**/
struct APPLICATIONCORE_API FMacPlatformSplash : public FGenericPlatformSplash
{
	/**
	* Show the splash screen
	*/
	static void Show();
	/**
	* Hide the splash screen
	*/
	static void Hide();

	/**
	 * Sets the text displayed on the splash screen (for startup/loading progress)
	 *
	 * @param	InType		Type of text to change
	 * @param	InText		Text to display
	 */
	using FGenericPlatformSplash::SetSplashText;
	static void SetSplashText( const SplashTextType::Type InType, const FText& InText );

	/**
	 * Return whether the splash screen is being shown or not
	 */
	static bool IsShown();
};

typedef FMacPlatformSplash FPlatformSplash;

