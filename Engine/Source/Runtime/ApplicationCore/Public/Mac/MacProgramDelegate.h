// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Mac/MacSystemIncludes.h"

using ProgramMainType=int32(*)(const TCHAR *);
using ProgramExitType=void(*)();

@interface MacProgramDelegate : NSObject<NSApplicationDelegate, NSFileManagerDelegate>
{
	// the main function for the program (shared with other platforms)
	ProgramMainType ProgramMain;
	
	// the function to run at exit time, on the game thread (very probagly FEngineLoop::Exit)
	ProgramExitType ProgramExit;
	
	// the commandline made from argv, but a subclass could modify it
	FString SavedCommandLine;
}



-(id)initWithProgramMain:(ProgramMainType)programMain programExit:(ProgramExitType)programExit;

+(int)mainWithArgc:(int)argc argv:(char*[])argv programMain:(ProgramMainType)programMain programExit:(ProgramExitType)programExit;
+(int)mainWithArgc:(int)argc argv:(char*[])argv existingDelegate:(MacProgramDelegate*)delegate;

@end
