// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/ITextInputMethodSystem.h"

class FMacTextInputMethodSystem : public ITextInputMethodSystem
{
public:
	virtual ~FMacTextInputMethodSystem() {}
	bool Initialize();
	void Terminate();
	
	// ITextInputMethodSystem Interface Begin
	virtual void ApplyDefaults(const TSharedRef<FGenericWindow>& InWindow) override;
	virtual TSharedPtr<ITextInputMethodChangeNotifier> RegisterContext(const TSharedRef<ITextInputMethodContext>& Context) override;
	virtual void UnregisterContext(const TSharedRef<ITextInputMethodContext>& Context) override;
	virtual void ActivateContext(const TSharedRef<ITextInputMethodContext>& Context) override;
	virtual void DeactivateContext(const TSharedRef<ITextInputMethodContext>& Context) override;
	virtual bool IsActiveContext(const TSharedRef<ITextInputMethodContext>& Context) const override;
	// ITextInputMethodSystem Interface End
	
private:
	TMap< TWeakPtr<ITextInputMethodContext>, TWeakPtr<ITextInputMethodChangeNotifier> > ContextMap;
};

