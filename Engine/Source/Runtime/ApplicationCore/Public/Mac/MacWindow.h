// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericWindow.h"
#include "Templates/SharedPointer.h"
#include "Mac/CocoaWindow.h"

/**
 * A platform specific implementation of FGenericWindow.
 * Native Windows provide platform-specific backing for and are always owned by an SWindow.
 */
class APPLICATIONCORE_API FMacWindow : public FGenericWindow, public TSharedFromThis<FMacWindow>
{
public:
	~FMacWindow();

	static TSharedRef< FMacWindow > Make();

	FCocoaWindow* GetWindowHandle() const;

	void Initialize( class FMacApplication* const Application, const TSharedRef< FGenericWindowDefinition >& InDefinition, const TSharedPtr< FMacWindow >& InParent, const bool bShowImmediately );
	
	void OnDisplayReconfiguration(CGDirectDisplayID Display, CGDisplayChangeSummaryFlags Flags);

	void OnWindowDidChangeScreen();

public:

	virtual void ReshapeWindow( int32 X, int32 Y, int32 Width, int32 Height ) override;

	virtual bool GetFullScreenInfo( int32& X, int32& Y, int32& Width, int32& Height ) const override;

	virtual void MoveWindowTo ( int32 X, int32 Y ) override;

	virtual void BringToFront( bool bForce = false ) override;

	virtual void Destroy() override;

	virtual void Minimize() override;

	virtual void Maximize() override;

	virtual void Restore() override;

	virtual void Show() override;

	virtual void Hide() override;

	virtual void SetWindowMode( EWindowMode::Type NewWindowMode ) override;

	virtual EWindowMode::Type GetWindowMode() const override;

	virtual bool IsMaximized() const override;

	virtual bool IsMinimized() const override;

	virtual bool IsVisible() const override;

	virtual bool GetRestoredDimensions(int32& X, int32& Y, int32& Width, int32& Height) override;

	virtual void SetWindowFocus() override;

	virtual void SetOpacity( const float InOpacity ) override;

	virtual bool IsPointInWindow( int32 X, int32 Y ) const override;

	virtual int32 GetWindowBorderSize() const override;

	virtual void* GetOSWindowHandle() const  override { return WindowHandle; }

	virtual bool IsForegroundWindow() const override;

	virtual void SetText(const TCHAR* const Text) override;

	virtual float GetDPIScaleFactor() const override;

	virtual void SetNativeWindowButtonsVisibility(bool bVisible);

	bool IsRegularWindow() const;

	int32 PositionX;
	int32 PositionY;


private:

	/**
	 * Protect the constructor; only TSharedRefs of this class can be made.
	 */
	FMacWindow();

	const CGDirectDisplayID GetDisplayID() const { return DisplayID; }

	void ApplySizeAndModeChanges(int32 X, int32 Y, int32 Width, int32 Height, EWindowMode::Type WindowMode);
	void UpdateFullScreenState(bool bToggleFullScreen);
	void WaitForFullScreenTransition();

private:

	FMacApplication* OwningApplication;

	/** Mac window handle */
	FCocoaWindow* WindowHandle;

	CGDirectDisplayID DisplayID;
    
    float CachedOpacity;

	struct FWindowedModeSavedState
	{
		CGDirectDisplayID CapturedDisplayID;
		CGDisplayModeRef DesktopDisplayMode;
		int32 WindowLevel;

		FWindowedModeSavedState() : CapturedDisplayID(kCGNullDirectDisplay), DesktopDisplayMode(nullptr), WindowLevel(NSNormalWindowLevel) {}
	} WindowedModeSavedState;

	bool bIsVisible : 1;
	bool bIsClosed : 1;

	/** Whether the window is yet to have its first Show() call. This is set false after first Show(). */
	bool bIsFirstTimeVisible : 1;
    
    bool bIsMainEditorWindow : 1;
};
