// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Containers/Map.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "Misc/CoreMisc.h"
#include "Misc/CoreDelegates.h"
#include "GenericPlatform/GenericWindow.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "GenericPlatform/GenericWindowDefinition.h"
#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/IInputInterface.h"
#include "Null/NullWindow.h"
#include "Null/NullCursor.h"

class IInputDevice;
class IModularFeature;

struct FNullPlatformDisplayMetrics : public FDisplayMetrics
{
	APPLICATIONCORE_API static void RebuildDisplayMetrics(struct FDisplayMetrics& OutDisplayMetrics);
};

/**
 * An implementation of GenericApplication specifically for use when rendering off screen.
 * This application has no platform backing so instead keeps track of its associated NullWindows itself.
 */
class FNullApplication : public GenericApplication, public FSelfRegisteringExec, public IInputInterface
{
public:
	static APPLICATIONCORE_API FNullApplication* CreateNullApplication();

	static APPLICATIONCORE_API void MoveWindowTo(FGenericWindow* Window, const int32 X, const int32 Y);

	static APPLICATIONCORE_API void OnSizeChanged(FGenericWindow* Window, const int32 Width, const int32 Height);

	static APPLICATIONCORE_API void GetFullscreenInfo(int32& X, int32& Y, int32& Width, int32& Height);

	static APPLICATIONCORE_API void ShowWindow(FGenericWindow* Window);

	static APPLICATIONCORE_API void HideWindow(FGenericWindow* Window);

	static APPLICATIONCORE_API void DestroyWindow(FGenericWindow* Window);

public:
	APPLICATIONCORE_API virtual ~FNullApplication();

	APPLICATIONCORE_API virtual void DestroyApplication() override;

	// FSelfRegisteringExec
	APPLICATIONCORE_API virtual bool Exec_Dev(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;

	APPLICATIONCORE_API TSharedPtr<FNullWindow> FindWindowByPtr(FGenericWindow* WindowToFind);

	APPLICATIONCORE_API void ActivateWindow(const TSharedPtr<FNullWindow>& Window);

public:
	APPLICATIONCORE_API virtual void SetMessageHandler(const TSharedRef<class FGenericApplicationMessageHandler>& InMessageHandler) override;

	APPLICATIONCORE_API virtual void PollGameDeviceState(const float TimeDelta) override;

	APPLICATIONCORE_API virtual void PumpMessages(const float TimeDelta) override;

	APPLICATIONCORE_API virtual void ProcessDeferredEvents(const float TimeDelta) override;

	APPLICATIONCORE_API virtual TSharedRef<FGenericWindow> MakeWindow() override;

	APPLICATIONCORE_API virtual void InitializeWindow(const TSharedRef<FGenericWindow>& Window, const TSharedRef<FGenericWindowDefinition>& InDefinition, const TSharedPtr<FGenericWindow>& InParent, const bool bShowImmediately) override;

	APPLICATIONCORE_API void DestroyWindow(TSharedRef<FNullWindow> WindowToRemove);

	APPLICATIONCORE_API virtual void SetCapture(const TSharedPtr<FGenericWindow>& InWindow) override;

	APPLICATIONCORE_API virtual void* GetCapture(void) const override;

	APPLICATIONCORE_API virtual void SetHighPrecisionMouseMode(const bool Enable, const TSharedPtr<FGenericWindow>& InWindow) override;

	virtual bool IsUsingHighPrecisionMouseMode() const override { return bUsingHighPrecisionMouseInput; }

	APPLICATIONCORE_API virtual bool IsGamepadAttached() const override;

	APPLICATIONCORE_API virtual FModifierKeysState GetModifierKeys() const override;

	APPLICATIONCORE_API virtual FPlatformRect GetWorkArea(const FPlatformRect& CurrentWindow) const override;

	APPLICATIONCORE_API void SetWorkArea(const FPlatformRect& NewWorkArea);

	virtual EWindowTransparency GetWindowTransparencySupport() const override
	{
		return EWindowTransparency::PerWindow;
	}

	APPLICATIONCORE_API virtual bool IsCursorDirectlyOverSlateWindow() const override;

	APPLICATIONCORE_API virtual TSharedPtr<FGenericWindow> GetWindowUnderCursor() override;

	virtual bool IsMouseAttached() const override { return true; }

private:
	APPLICATIONCORE_API FNullApplication();

	/** Handles "Cursor" exec commands" */
	APPLICATIONCORE_API bool HandleCursorCommand(const TCHAR* Cmd, FOutputDevice& Ar);

	/** Handles "Window" exec commands" */
	APPLICATIONCORE_API bool HandleWindowCommand(const TCHAR* Cmd, FOutputDevice& Ar);

	/** Handles parsing the work area resolution from the command line */
	APPLICATIONCORE_API bool ParseResolution(const TCHAR* InResolution, uint32& OutX, uint32& OutY);

	void OnInputDeviceModuleRegistered(const FName& Type, IModularFeature* ModularFeature);

public:
	virtual IInputInterface* GetInputInterface() override
	{
		return this;
	}

	// IInputInterface overrides
	APPLICATIONCORE_API virtual void SetForceFeedbackChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) override;
	APPLICATIONCORE_API virtual void SetForceFeedbackChannelValues(int32 ControllerId, const FForceFeedbackValues& Values) override;
	APPLICATIONCORE_API virtual void SetHapticFeedbackValues(int32 ControllerId, int32 Hand, const FHapticFeedbackValues& Values) override;
	virtual void SetLightColor(int32 ControllerId, FColor Color) override {}
	virtual void ResetLightColor(int32 ControllerId) override {}

private:
	TArray<TSharedRef<FNullWindow>> Windows;

	/** List of input devices implemented in external modules. */
	TArray<TSharedPtr<class IInputDevice>> ExternalInputDevices;
	bool bHasLoadedInputPlugins;

	/** Using high precision mouse input */
	bool bUsingHighPrecisionMouseInput;

	/** Window that we think has been activated last. */
	TSharedPtr<FNullWindow> CurrentlyActiveWindow;

	/** Window that we think has been previously active. */
	TSharedPtr<FNullWindow> PreviousActiveWindow;

	/** The virtual work area*/
	FPlatformRect WorkArea;
};

extern FNullApplication* NullApplication;
