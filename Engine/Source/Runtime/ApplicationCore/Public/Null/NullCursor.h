// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Vector2D.h"
#include "GenericPlatform/ICursor.h"

/**
 * An implementation of an ICursor specifically for use when rendering off screen. 
 * This cursor has no platform backing so instead keeps track of its position and other properties itself.
 */
class FNullCursor : public ICursor
{
public:
	FNullCursor();

	virtual ~FNullCursor();

	/** The position of the cursor */
	virtual FVector2D GetPosition() const override;

	/** Sets the position of the cursor */
	virtual void SetPosition(const int32 X, const int32 Y) override;

	/** Sets the cursor */
	virtual void SetType(const EMouseCursor::Type InNewCursor) override;

	/** Gets the current type of the cursor */
	virtual EMouseCursor::Type GetType() const override;

	/** Gets the size of the cursor */
	virtual void GetSize(int32& Width, int32& Height) const override;

	/**
	 * Shows or hides the cursor
	 *
	 * @param bShow	true to show the mouse cursor, false to hide it
	 */
	virtual void Show(bool bShow) override;

	/**
	 * Locks the cursor to the passed in bounds
	 *
	 * @param Bounds	The bounds to lock the cursor to.  Pass NULL to unlock.
	 */
	virtual void Lock(const RECT* const Bounds) override;

	/**
	 * Allows overriding the shape of a particular cursor.
	 */
	virtual void SetTypeShape(EMouseCursor::Type InCursorType, void* CursorHandle) override;

private:
	/** Cached global X position */
	int32 CachedGlobalXPosition;

	/** Cached global Y position */
	int32 CachedGlobalYPosition;
};
