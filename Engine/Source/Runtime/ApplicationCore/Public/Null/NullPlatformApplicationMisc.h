// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformApplicationMisc.h"

struct FNullPlatformApplicationMisc : public FGenericPlatformApplicationMisc
{
	static APPLICATIONCORE_API class GenericApplication* CreateApplication();

	static APPLICATIONCORE_API bool IsUsingNullApplication();
};
