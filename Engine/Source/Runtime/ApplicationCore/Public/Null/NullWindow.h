// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"
#include "GenericPlatform/GenericWindow.h"
#include "Math/IntPoint.h"

class FNullApplication;
struct FGenericWindowDefinition;

DECLARE_LOG_CATEGORY_EXTERN(LogNullWindow, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogNullWindowType, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogNullWindowEvent, Log, All);

/**
 * An implementation of FGenericWindow specifically for use when rendering off screen.
 * This window has no platform backing so instead keeps track of its position and other properties itself.
 */
class FNullWindow : public FGenericWindow
{
public:
	static APPLICATIONCORE_API TSharedRef<FNullWindow> Make();

	APPLICATIONCORE_API virtual ~FNullWindow();

	APPLICATIONCORE_API void Initialize(class FNullApplication* const Application, const TSharedRef<FGenericWindowDefinition>& InDefinition, const TSharedPtr<FNullWindow>& InParent, const bool bShowImmediately);

	/** Native windows should implement ReshapeWindow by changing the platform-specific window to be located at (X,Y) and be the dimensions Width x Height. */
	APPLICATIONCORE_API virtual void ReshapeWindow(int32 X, int32 Y, int32 Width, int32 Height) override;

	/** Returns the rectangle of the screen the window is associated with */
	APPLICATIONCORE_API virtual bool GetFullScreenInfo(int32& X, int32& Y, int32& Width, int32& Height) const override;

	/** Native windows should implement MoveWindowTo by relocating the platform-specific window to (X,Y). */
	APPLICATIONCORE_API virtual void MoveWindowTo(int32 X, int32 Y) override;

	/** Native windows should implement BringToFront by making this window the top-most window (i.e. focused). */
	APPLICATIONCORE_API virtual void BringToFront(bool bForce = false) override;

	/** @hack Force a window to front even if a different application is in front. */
	APPLICATIONCORE_API virtual void HACK_ForceToFront() override;

	/** Native windows should implement this function by asking the OS to destroy OS-specific resource associated with the window (e.g. Win32 window handle) */
	APPLICATIONCORE_API virtual void Destroy() override;

	/** Native window should implement this function by performing the equivalent of the Win32 minimize-to-taskbar operation */
	APPLICATIONCORE_API virtual void Minimize() override;

	/** Native window should implement this function by performing the equivalent of the Win32 maximize operation */
	APPLICATIONCORE_API virtual void Maximize() override;

	/** Native window should implement this function by performing the equivalent of the Win32 restore operation */
	APPLICATIONCORE_API virtual void Restore() override;

	/** Native window should make itself visible */
	APPLICATIONCORE_API virtual void Show() override;

	/** Native window should hide itself */
	APPLICATIONCORE_API virtual void Hide() override;

	/** Toggle native window between fullscreen and normal mode */
	APPLICATIONCORE_API virtual void SetWindowMode(EWindowMode::Type InNewWindowMode) override;

	/** @return true if the native window is currently in fullscreen mode, false otherwise */
	APPLICATIONCORE_API virtual EWindowMode::Type GetWindowMode() const override;

	/** @return true if the native window is maximized, false otherwise */
	APPLICATIONCORE_API virtual bool IsMaximized() const override;

	/** @return true if the native window is minimized, false otherwise */
	APPLICATIONCORE_API virtual bool IsMinimized() const override;

	/** @return true if the native window is visible, false otherwise */
	APPLICATIONCORE_API virtual bool IsVisible() const override;

	/**
	 * Populates the size and location of the window when it is restored.
	 * If the function fails, false is returned and X,Y,Width,Height will be undefined.
	 *
	 * @return true when the size and location and successfully retrieved; false otherwise.
	 */
	APPLICATIONCORE_API virtual bool GetRestoredDimensions(int32& X, int32& Y, int32& Width, int32& Height) override;

	/**
	 * Native windows should implement SetWindowFocus to let the OS know that a window has taken focus.
	 * Slate handles focus on a per widget basis internally but the OS still needs to know what window has focus for proper message routing
	 */
	APPLICATIONCORE_API virtual void SetWindowFocus() override;

	/**
	 * Sets the opacity of this window
	 *
	 * @param	InOpacity	The new window opacity represented as a floating point scalar
	 */
	APPLICATIONCORE_API virtual void SetOpacity(const float InOpacity) override;

	/**
	 * Enables or disables the window.  If disabled the window receives no input
	 *
	 * @param bEnable	true to enable the window, false to disable it.
	 */
	APPLICATIONCORE_API virtual void Enable(bool bEnable) override;

	/** @return true if native window exists underneath the coordinates */
	APPLICATIONCORE_API virtual bool IsPointInWindow(int32 X, int32 Y) const override;

	/** Gets OS specific window border size. This is necessary because Win32 does not give control over this size. */
	APPLICATIONCORE_API virtual int32 GetWindowBorderSize() const override;

	/** Gets OS specific window title bar size */
	APPLICATIONCORE_API virtual int32 GetWindowTitleBarSize() const override;

	/** Gets the OS Window handle in the form of a void pointer for other API's */
	APPLICATIONCORE_API virtual void* GetOSWindowHandle() const override;

	/** @return true if the window is in the foreground */
	APPLICATIONCORE_API virtual bool IsForegroundWindow() const override;

	/** @return true if the window is in the foreground */
	APPLICATIONCORE_API virtual bool IsFullscreenSupported() const override;

	/**
	 * Sets the window text - usually the title but can also be text content for things like controls
	 *
	 * @param Text	The window's title or content text
	 */
	APPLICATIONCORE_API virtual void SetText(const TCHAR* const Text) override;

	/** @return	The definition describing properties of the window */
	APPLICATIONCORE_API virtual const FGenericWindowDefinition& GetDefinition() const override;

	/** @return	Returns true if the window definition is valid */
	APPLICATIONCORE_API virtual bool IsDefinitionValid() const override;

	/** @return	Gives the native window a chance to adjust our stored window size before we cache it off */
	APPLICATIONCORE_API virtual void AdjustCachedSize(FVector2D& Size) const override;

	virtual float GetDPIScaleFactor() const override
	{
		return DPIScaleFactor;
	}

	virtual void SetDPIScaleFactor(float Value) override
	{
		DPIScaleFactor = Value;
	}

	/** determines whether or not this window does its own DPI management */
	APPLICATIONCORE_API virtual bool IsManualManageDPIChanges() const override;

	/** call with a true argument if this window need to do its custom size management in response to DPI variations */
	APPLICATIONCORE_API virtual void SetManualManageDPIChanges(const bool bAutoHandle) override;

	/**
	 * Attempts to draw the user's attention to this window in whatever way is
	 * appropriate for the platform if this window is not the current active
	 * window.
	 *
	 * @param Parameters The parameters for drawing attention. Depending on
	 *        the platform, not all parameters may be supported.
	 */
	APPLICATIONCORE_API virtual void DrawAttention(const FWindowDrawAttentionParameters& Parameters) override;

	/** Shows or hides native window buttons on platforms that use them */
	APPLICATIONCORE_API virtual void SetNativeWindowButtonsVisibility(bool bVisible);

	/** Always returns true, since null windows won't present frames to a display. */
	APPLICATIONCORE_API virtual bool IsRenderingOffScreen() const override;

private:
	/**
	 * Protect the constructor; only TSharedRefs of this class can be made.
	 */
	APPLICATIONCORE_API FNullWindow();

	FNullApplication* OwningApplication;

	float DPIScaleFactor;

	FIntPoint ScreenPosition;

	FIntPoint SizeInScreen;

	/** Manually store window visibility as OS doesn't do that for us in nullplatform */
	bool bIsVisible;

	bool bIsInitialized;
};
