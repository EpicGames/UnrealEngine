// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/App.h"
#include "Misc/OutputDeviceConsole.h"
#include "Misc/FeedbackContext.h"
#include <syslog.h>

/**
 * Feedback context implementation for Unix.
 */
class FUnixFeedbackContext : public FFeedbackContext
{
	/** Context information for warning and error messages */
	FContextSupplier*	Context;

public:
	// Variables.
	int32					SlowTaskCount;

	// Constructor.
	FUnixFeedbackContext()
	: FFeedbackContext()
	, Context( NULL )
	, SlowTaskCount( 0 )
	{}

	void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override;
	void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time) override;
	void SerializeRecord(const UE::FLogRecord& Record) override;

	/** Ask the user a binary question, returning their answer */
	bool YesNof(const FText& Question) override;

	void BeginSlowTask(const FText& Task, bool ShowProgressDialog, bool bShowCancelButton=false)
	{
		GIsSlowTask = ++SlowTaskCount>0;
	}

	void EndSlowTask()
	{
		check(SlowTaskCount>0);
		GIsSlowTask = --SlowTaskCount>0;
	}

	bool StatusUpdate(int32 Numerator, int32 Denominator, const FText& StatusText)
	{
		return true;
	}

	FContextSupplier* GetContext() const override
	{
		return Context;
	}

	void SetContext(FContextSupplier* InSupplier) override
	{
		Context = InSupplier;
	}

private:
	void LogErrorToSysLog(const TCHAR* V, const FName& Category, double Time) const;
	void LogErrorRecordToSysLog(const UE::FLogRecord& Record) const;
};
