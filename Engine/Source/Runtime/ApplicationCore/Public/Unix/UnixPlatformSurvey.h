// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	UnixPlatformSurvey.h: Unix platform hardware-survey classes
==============================================================================================*/

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformSurvey.h"

/**
* Unix implementation of FGenericPlatformSurvey
**/
struct FUnixPlatformSurvey : public FGenericPlatformSurvey
{
	/** Start, or check on, the hardware survey */
	APPLICATIONCORE_API static bool GetSurveyResults(FHardwareSurveyResults& OutResults, bool bWait = false);

private:

	/**
	 * Safely write strings into the fixed length TCHAR buffers of a FHardwareSurveyResults member
	 */
	static void WriteFStringToResults(TCHAR* OutBuffer, const FString& InString);
};

typedef FUnixPlatformSurvey FPlatformSurvey;
