// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_ACCESSIBILITY && UE_WINDOWS_USING_UIA

#include "Windows/AllowWindowsPlatformTypes.h"
#include <Ole2.h>
#include "Windows/HideWindowsPlatformTypes.h"
#include <UIAutomation.h>

#include "Templates/SharedPointer.h"

class FWindowsUIAManager;
class IAccessibleWidget;

/**
 * Base class for all Windows UIA Providers to inherit from. Provides implementation of IUnknown's
 * reference counting functions as well as integration with the UIA Manager class.
 */
class FWindowsUIABaseProvider
{
public:
	/** Notify the Provider that the backing application no longer exists. */
	void OnUIAManagerDestroyed();

protected:
	FWindowsUIABaseProvider(FWindowsUIAManager& InManager, TSharedRef<IAccessibleWidget> InWidget);
	virtual ~FWindowsUIABaseProvider();

	/**
	 * Check whether this Provider can have operations called on it. This might return false if the
	 * application is being destroyed, or if the underlying accessible widget is no longer valid.
	 */
	bool IsValid() const;

	/** Add one to RefCount, and return the new RefCount */
	uint32 IncrementRef();
	/** Subtract one from RefCount, and return the new RefCount. If RefCount is 0, deletes the Provider and returns 0. */
	uint32 DecrementRef();

	/** A pointer to the UIA Manager which is guaranteed to be valid so long as the application is still running. */
	FWindowsUIAManager* UIAManager;
	/**
	 * All Providers must have a valid accessible widget, even if the accessible widget itself does not
	 * have any valid data backing it. In this case, the accessible widget will return some default values.
	 */
	TSharedRef<IAccessibleWidget> Widget;

private:
	/** Counter for the number of strong references to this object. Many of these will be from external applications. */
	uint32 RefCount;
};

#endif
