// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_ACCESSIBILITY && UE_WINDOWS_USING_UIA

#include "Windows/Accessibility/WindowsUIABaseProvider.h"
#include "Templates/SharedPointer.h"

class FWindowsUIAControlProvider;
class IAccessibleWidget;

/**
 * A TextRange Provider represents a start and end position within an ITextProvider. This can reference
 * a word, a paragraph, a selection, etc. Multiple text ranges can exist for a single piece of text.
 */
class FWindowsUIATextRangeProvider
	: public FWindowsUIABaseProvider
	, public ITextRangeProvider
{
public:
	FWindowsUIATextRangeProvider(FWindowsUIAManager& InManager, TSharedRef<IAccessibleWidget> InWidget, FTextRange InRange);

	// IUnknown
	HRESULT STDCALL QueryInterface(REFIID riid, void** ppInterface) override;
	ULONG STDCALL AddRef() override;
	ULONG STDCALL Release() override;
	// ~

	// ITextRangeProvider
	virtual HRESULT STDCALL Clone(ITextRangeProvider** pRetVal) override;
	virtual HRESULT STDCALL Compare(ITextRangeProvider* range, BOOL* pRetVal) override;
	virtual HRESULT STDCALL CompareEndpoints(TextPatternRangeEndpoint endpoint, ITextRangeProvider* targetRange, TextPatternRangeEndpoint targetEndpoint, int* pRetVal) override;
	virtual HRESULT STDCALL ExpandToEnclosingUnit(TextUnit unit) override;
	virtual HRESULT STDCALL FindAttribute(TEXTATTRIBUTEID attributeId, VARIANT val, BOOL backward, ITextRangeProvider** pRetVal) override;
	virtual HRESULT STDCALL FindText(BSTR text, BOOL backward, BOOL ignoreCase, ITextRangeProvider** pRetVal) override;
	virtual HRESULT STDCALL GetAttributeValue(TEXTATTRIBUTEID attributeId, VARIANT* pRetVal) override;
	virtual HRESULT STDCALL GetBoundingRectangles(SAFEARRAY** pRetVal) override;
	virtual HRESULT STDCALL GetEnclosingElement(IRawElementProviderSimple** pRetVal) override;
	virtual HRESULT STDCALL GetText(int maxLength, BSTR* pRetVal) override;
	virtual HRESULT STDCALL Move(TextUnit unit, int count, int* pRetVal) override;
	virtual HRESULT STDCALL MoveEndpointByUnit(TextPatternRangeEndpoint endpoint, TextUnit unit, int count, int* pRetVal) override;
	virtual HRESULT STDCALL MoveEndpointByRange(TextPatternRangeEndpoint endpoint, ITextRangeProvider* targetRange, TextPatternRangeEndpoint targetEndpoint) override;
	virtual HRESULT STDCALL Select() override;
	virtual HRESULT STDCALL AddToSelection() override;
	virtual HRESULT STDCALL RemoveFromSelection() override;
	virtual HRESULT STDCALL ScrollIntoView(BOOL alignToTop) override;
	virtual HRESULT STDCALL GetChildren(SAFEARRAY** pRetVal) override;
	// ~

protected:
	/**
	 * Gets the substring that this text range Provider represents.
	 *
	 * @return A substring of the underlying widget's text from TextRange's begin/end indices.
	 */
	FString TextFromTextRange();
	static FString TextFromTextRange(const FString& InString, const FTextRange& InRange);

	/** The range that this Provider represents in the underlying text. */
	FTextRange TextRange;

private:
	virtual ~FWindowsUIATextRangeProvider();
};

/**
 * The control Provider handles all control pattern-related functionality for the widget Provider. Control Providers should only
 * be generated through widget Providers in their GetPatternProvider() function. In doing this, a control Pattern is only guaranteed
 * to support functions that match the pattern Provider that it was generated for. This means that if GetPatternProvider() is called
 * for UIA_ValuePatternId, the control Provider is only guaranteed to work with functions that implement IValueProvider. It is up
 * to the caller to ensure that they are making valid function calls.
 */
class FWindowsUIAControlProvider
	: public FWindowsUIABaseProvider
	, public IInvokeProvider
	, public IRangeValueProvider
	, public ITextProvider
	, public IToggleProvider
	, public ITransformProvider
	, public IValueProvider
	, public IWindowProvider
	, public ISelectionProvider
	, public ISelectionItemProvider
{
public:
	FWindowsUIAControlProvider(FWindowsUIAManager& Manager, TSharedRef<IAccessibleWidget> InWidget);

	// IUnknown
	HRESULT STDCALL QueryInterface(REFIID riid, void** ppInterface) override;
	ULONG STDCALL AddRef() override;
	ULONG STDCALL Release() override;
	// ~

	// IInvokeProvider
	virtual HRESULT STDCALL Invoke() override;
	// ~

	// IRangeValueProvider
	virtual HRESULT STDCALL SetValue(double val) override;
	virtual HRESULT STDCALL get_Value(double* pRetVal) override;
	virtual HRESULT STDCALL get_IsReadOnly(BOOL* pRetVal) override;
	virtual HRESULT STDCALL get_Maximum(double* pRetVal) override;
	virtual HRESULT STDCALL get_Minimum(double* pRetVal) override;
	virtual HRESULT STDCALL get_LargeChange(double* pRetVal) override;
	virtual HRESULT STDCALL get_SmallChange(double* pRetVal) override;
	// ~

	// ITextProvider
	virtual HRESULT STDCALL get_DocumentRange(ITextRangeProvider** pRetVal) override;
	virtual HRESULT STDCALL get_SupportedTextSelection(SupportedTextSelection* pRetVal) override;
	virtual HRESULT STDCALL GetSelection(SAFEARRAY** pRetVal) override;
	virtual HRESULT STDCALL GetVisibleRanges(SAFEARRAY** pRetVal) override;
	virtual HRESULT STDCALL RangeFromChild(IRawElementProviderSimple* childElement, ITextRangeProvider** pRetVal) override;
	virtual HRESULT STDCALL RangeFromPoint(UiaPoint point, ITextRangeProvider** pRetVal) override;
	// ~

	// IToggleState
	virtual HRESULT STDCALL get_ToggleState(ToggleState* pRetVal) override;
	virtual HRESULT STDCALL Toggle() override;
	// ~

	// ITransformProvider
	virtual HRESULT STDCALL get_CanMove(BOOL *pRetVal) override;
	virtual HRESULT STDCALL get_CanResize(BOOL *pRetVal) override;
	virtual HRESULT STDCALL get_CanRotate(BOOL *pRetVal) override;
	virtual HRESULT STDCALL Move(double x, double y) override;
	virtual HRESULT STDCALL Resize(double width, double height) override;
	virtual HRESULT STDCALL Rotate(double degrees) override;
	// ~

	// IValueProvider
	virtual HRESULT STDCALL SetValue(LPCWSTR val) override;
	virtual HRESULT STDCALL get_Value(BSTR* pRetVal) override;
	//virtual HRESULT STDCALL get_IsReadOnly(BOOL* pRetVal) override; // Duplicate of IRangeValueProvider
	// ~

	// IWindowProvider
	virtual HRESULT STDCALL Close() override;
	virtual HRESULT STDCALL get_CanMaximize(BOOL* pRetVal) override;
	virtual HRESULT STDCALL get_CanMinimize(BOOL* pRetVal) override;
	virtual HRESULT STDCALL get_IsModal(BOOL* pRetVal) override;
	virtual HRESULT STDCALL get_IsTopmost(BOOL* pRetVal) override;
	virtual HRESULT STDCALL get_WindowInteractionState(WindowInteractionState* pRetVal) override;
	virtual HRESULT STDCALL get_WindowVisualState(WindowVisualState* pRetVal) override;
	virtual HRESULT STDCALL SetVisualState(WindowVisualState state) override;
	virtual HRESULT STDCALL WaitForInputIdle(int milliseconds, BOOL* pRetVal) override;
	// ~

	// ISelectionProvider
	//HRESULT STDMETHODCALLTYPE GetSelection(SAFEARRAY** pRetVal) override; // duplicate from ITextProvider. Implementation also there 
	virtual HRESULT STDMETHODCALLTYPE get_CanSelectMultiple(BOOL* pRetVal) override;
	virtual HRESULT STDMETHODCALLTYPE get_IsSelectionRequired(BOOL* pRetVal) override;
	// ~

	// ISelectionItemProvider
	virtual HRESULT STDMETHODCALLTYPE Select() override;
	virtual HRESULT STDMETHODCALLTYPE AddToSelection() override;
	virtual HRESULT STDMETHODCALLTYPE RemoveFromSelection() override;
	virtual HRESULT STDMETHODCALLTYPE get_IsSelected(BOOL* pRetVal) override;
	virtual HRESULT STDMETHODCALLTYPE get_SelectionContainer(IRawElementProviderSimple** pRetVal) override;
	// ~
private:
	virtual ~FWindowsUIAControlProvider();
};

#endif
