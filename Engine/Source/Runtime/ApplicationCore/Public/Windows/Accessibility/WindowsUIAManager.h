// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_ACCESSIBILITY && UE_WINDOWS_USING_UIA

#include "Templates/SharedPointer.h"
#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"

class FWindowsApplication;
class FWindowsWindow;
class FWindowsUIAControlProvider;
class FWindowsUIAWidgetProvider;
class FWindowsUIAWindowProvider;
class IAccessibleWidget;
class FVariant;

/**
 * Manager for Windows implementation of UE's accessibility API, utilizing Microsoft's UI Automation API. It provides a
 * central location for message passing to/from the WindowsApplication, AccessibleMessageHandler, and Windows UIA Providers.
 *
 * This class only accepts and returns references, so callers should ensure their objects are valid before using it.
 */
class FWindowsUIAManager
{
	/** Allow FWindowsUIABaseProvider to register and unregister itself from ProviderList automatically */
	friend class FWindowsUIABaseProvider;
public:
	FWindowsUIAManager(const FWindowsApplication& InApplication);
	~FWindowsUIAManager();

	/**
	 * Create a Windows UIA IRawElementProviderSimple from a given accessible widget. Providers are stored in a local cache,
	 * and this function will allocate a new Provider if one does not already exist for the given widget. Using this function
	 * will always increase the RefCount of the Provider by one.
	 *
	 * If InWidget->IsWindow() returns true, a FWindowsUIAWindowProvider will be created instead of a FWindowsUIAWidgetProvider.
	 *
	 * @param InWidget A non-null reference to an accessible widget
	 * @return A reference to the cached Provider
	 */
	FWindowsUIAWidgetProvider& GetWidgetProvider(TSharedRef<IAccessibleWidget> InWidget);

	/**
	 * Create a Windows UIA IRawElementProviderSimple from a given native window handle. Providers are stored in a local cache,
	 * and this function will allocate a new Provider if one does not already exist for the given widget. Using this function
	 * will always increase the RefCount of the Provider by one.
	 *
	 * If the Provider cache is empty upon calling this function, it will activate accessibility in the entire application.
	 *
	 * @param InWindow A non-null reference to a native window
	 * @return A reference to the cached Provider
	 */
	FWindowsUIAWindowProvider& GetWindowProvider(TSharedRef<FWindowsWindow> InWindow);

	/**
	 * Notify the Manager that the RefCount for a Provider has reached 0, which will cause it to be removed from the cache.
	 *
	 * If the Provider cache is empty after removing the Provider, accessibility will be disabled through the entire application.
	 *
	 * @param InWidget The accessible widget backing the Provider that was removed
	 */
	void OnWidgetProviderRemoved(TSharedRef<IAccessibleWidget> InWidget);

	/**
	 * Notify the Manager that the accessible message handler for the application has changed, in order to relink to the accessible event delegate.
	 */
	void OnAccessibleMessageHandlerChanged();

	uint32 GetCachedCurrentLocaleLCID() const { return CachedCurrentLocaleLCID; }

	/**
	* Runs the passed in function in the game thread. 
	* Blocks until the function completes execution in the game thread.

	* @param Function The function to run in the game thread 
	*/
	void RunInGameThreadBlocking(const TFunction<void()>& Function) const;

	static TMap<EAccessibleWidgetType, ULONG> WidgetTypeToWindowsTypeMap;
	static TMap<EAccessibleWidgetType, FText> WidgetTypeToTextMap;

#if !UE_BUILD_SHIPPING
	void DumpAccessibilityStats() const;
#endif

private:
	/** Callback function for processing events raised from the AccessibleMessageHandler */
	void OnEventRaised(const FAccessibleEventArgs& Args);

	/** Called when the first window widget provider is requested. */
	void OnAccessibilityEnabled(); 

	/** Called when the last widget provider is removed. */
	void OnAccessibilityDisabled();

	/** Updates the cached current locale LCID to match the current locale LCID in FInternationalization */
	void UpdateCachedCurrentLocaleLCID();

	/** Cache of all Providers with a RefCount of at least 1 that map to an accessible widget. */
	TMap<TSharedRef<IAccessibleWidget>, FWindowsUIAWidgetProvider*> CachedWidgetProviders;
	/**
	 * A set of all Providers with a RefCount of at least 1. This also includes non-widget Providers such as text ranges.
	 * When the application is closed, this is used to notify Providers held by external applications that they are invalid.
	 */
	TSet<FWindowsUIABaseProvider*> ProviderList;
	/** A reference back to the owning application that can be used to access the AccessibleMessageHandler */
	const FWindowsApplication& WindowsApplication;

	/**Handle associated with registering for culture change events from FInternationalization */
	FDelegateHandle OnCultureChangedHandle; 
	
	/**Current LCID of the current locale in FInternationalization. Used to ensure screen readers 
	* pronounce localized text in a native voice. 
	* For more info: https://docs.microsoft.com/en-us/windows/win32/intl/language-identifier-constants-and-strings
	*/
	uint32 CachedCurrentLocaleLCID;
};

#endif
