// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_ACCESSIBILITY && UE_WINDOWS_USING_UIA

#include "Windows/AllowWindowsPlatformTypes.h"
#include <Ole2.h>
#include "Windows/HideWindowsPlatformTypes.h"
#include <UIAutomation.h>

#include "Templates/SharedPointer.h"

class FVariant;
class IAccessibleWidget;

namespace WindowsUIAPropertyGetters
{
	VARIANT GetPropertyValueWindows(TSharedRef<IAccessibleWidget> AccessibleWidget, PROPERTYID WindowsPropertyId);
	FVariant GetPropertyValue(TSharedRef<IAccessibleWidget> AccessibleWidget, PROPERTYID WindowsPropertyId);

	/**
	 * Convert an FVariant to a Windows VARIANT.
	 * Only necessary conversions are implemented.
	 */
	VARIANT FVariantToWindowsVariant(const FVariant& Value);
}

#endif
