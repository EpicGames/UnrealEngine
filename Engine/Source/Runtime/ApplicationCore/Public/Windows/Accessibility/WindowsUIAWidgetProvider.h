// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_ACCESSIBILITY && UE_WINDOWS_USING_UIA

#include "Windows/Accessibility/WindowsUIABaseProvider.h"
#include "Misc/Variant.h"

class FWindowsUIAManager;
class IAccessibleWidget;

/**
 * Windows UIA Provider implementation for all non-Window widgets.
 *
 * WidgetProvider does not inherit from IRawElementProviderFragmentRoot and thus cannot be queried
 * by external applications for ElementProviderFromPoint(). This should work fine but there could
 * be unforeseen consquences for cases like modal popup dialogs which are not necessarily windows.
 */
class FWindowsUIAWidgetProvider
	: public FWindowsUIABaseProvider
	, public IRawElementProviderSimple
	, public IRawElementProviderFragment
{
	friend class FWindowsUIAManager;
public:
	// IUnknown
	virtual HRESULT STDCALL QueryInterface(REFIID riid, void** ppInterface) override;
	virtual ULONG STDCALL AddRef() override;
	virtual ULONG STDCALL Release() override;
	// ~

	// IRawElementProviderSimple
	virtual HRESULT STDCALL get_ProviderOptions(ProviderOptions* pRetVal) override;
	virtual HRESULT STDCALL GetPatternProvider(PATTERNID patternId, IUnknown** pRetVal) override;
	virtual HRESULT STDCALL GetPropertyValue(PROPERTYID propertyId, VARIANT* pRetVal) override;
	virtual HRESULT STDCALL get_HostRawElementProvider(IRawElementProviderSimple** pRetVal) override;
	// ~

	// IRawElementProviderFragment
	virtual HRESULT STDCALL Navigate(NavigateDirection direction, IRawElementProviderFragment** pRetVal) override;
	virtual HRESULT STDCALL GetRuntimeId(SAFEARRAY** pRetVal) override;
	virtual HRESULT STDCALL get_BoundingRectangle(UiaRect* pRetVal) override;
	virtual HRESULT STDCALL GetEmbeddedFragmentRoots(SAFEARRAY** pRetVal) override;
	virtual HRESULT STDCALL SetFocus() override;
	virtual HRESULT STDCALL get_FragmentRoot(IRawElementProviderFragmentRoot** pRetVal) override;
	// ~

	/**
	 * Check if this Provider implements a specific control pattern. A FWindowsUIAControlProvider can
	 * then be created of the pattern is supported. Since this is a helper function, the caller should
	 * separately ensure IsValid() before doing anything with the result.
	 *
	 * @param PatternId The control pattern to check for
	 * @return true if a control Provider can be created for this Provider for this control pattern
	 */
	bool SupportsInterface(PATTERNID PatternId) const;

protected:
	FWindowsUIAWidgetProvider(FWindowsUIAManager& InManager, TSharedRef<IAccessibleWidget> InWidget);
	virtual ~FWindowsUIAWidgetProvider();

//	void UpdateCachedProperties();
//private:
//	void UpdateCachedProperty(PROPERTYID PropertyId);
//	TMap<int32, FVariant> CachedPropertyValues;
};

/**
 * Windows UIA Provider implementation for all Window widgets. Widget->AsWindow() must return a valid pointer.
 */
class FWindowsUIAWindowProvider
	: public FWindowsUIAWidgetProvider
	, public IRawElementProviderFragmentRoot
{
	friend class FWindowsUIAManager;
public:
	// IUnknown
	virtual HRESULT STDCALL QueryInterface(REFIID riid, void** ppInterface) override;
	virtual ULONG STDCALL AddRef() override;
	virtual ULONG STDCALL Release() override;
	// ~

	// IRawElementProviderSimple
	virtual HRESULT STDCALL get_HostRawElementProvider(IRawElementProviderSimple** pRetVal) override;
	virtual HRESULT STDCALL GetPatternProvider(PATTERNID patternId, IUnknown** pRetVal) override;
	// ~

	// IRawElementProviderFragmentRoot
	virtual HRESULT STDCALL ElementProviderFromPoint(double x, double y, IRawElementProviderFragment** pRetVal) override;
	virtual HRESULT STDCALL GetFocus(IRawElementProviderFragment** pRetVal) override;
	// ~

protected:
	/** Note: InWidget must also be an IAccessibleWindow */
	FWindowsUIAWindowProvider(FWindowsUIAManager& InManager, TSharedRef<IAccessibleWidget> InWidget);
	virtual ~FWindowsUIAWindowProvider();
};

/**
 * Helper class which can be used in conjunction with FWindowsUIAManager::GetWidgetProvider, which will
 * automatically calls Release on the Provider when the variable goes out of scope. Note that GetWidgetProvider
 * already increments the ref count when its called, so the scoped provider does not have to also increase it.
 */
class FScopedWidgetProvider
{
public:
	FScopedWidgetProvider(FWindowsUIAWidgetProvider& InProvider)
		: Provider(InProvider)
	{
	}
	~FScopedWidgetProvider()
	{
		Provider.Release();
	}
	FWindowsUIAWidgetProvider& Provider;
};

#endif
