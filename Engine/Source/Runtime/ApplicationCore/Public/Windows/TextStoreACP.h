// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/MemoryOps.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include <TextStor.h>
#include <msctf.h>
#include "Microsoft/COMPointer.h"

class ITextInputMethodContext;

class FTextStoreACP : public ITextStoreACP, public ITfContextOwnerCompositionSink
{
public:
	FTextStoreACP(const TSharedRef<ITextInputMethodContext>& Context);
	virtual ~FTextStoreACP() = default;

	// IUnknown Interface Begin
	STDMETHODIMP			QueryInterface(REFIID riid, void **ppvObj);
	STDMETHODIMP_(ULONG)	AddRef(void);
	STDMETHODIMP_(ULONG)	Release(void);
	// IUnknown Interface End

	// ITextStoreACP Interface Begin
	STDMETHODIMP AdviseSink(__RPC__in REFIID riid, __RPC__in_opt IUnknown *punk, DWORD dwMask);
	STDMETHODIMP UnadviseSink(__RPC__in_opt IUnknown *punk);
	STDMETHODIMP RequestLock(DWORD dwLockFlags, HRESULT *phrSession);

	STDMETHODIMP GetStatus(__RPC__out TS_STATUS *pdcs);
	STDMETHODIMP GetEndACP(__RPC__out LONG *pacp);

	// Selection Methods
	STDMETHODIMP GetSelection(ULONG ulIndex, ULONG ulCount, __RPC__out_ecount_part(ulCount, *pcFetched) TS_SELECTION_ACP *pSelection, __RPC__out ULONG *pcFetched);
	STDMETHODIMP SetSelection(ULONG ulCount, __RPC__in_ecount_full(ulCount) const TS_SELECTION_ACP *pSelection);

	// Attributes Methods
	STDMETHODIMP RequestSupportedAttrs(DWORD dwFlags, ULONG cFilterAttrs, __RPC__in_ecount_full_opt(cFilterAttrs) const TS_ATTRID *paFilterAttrs);
	STDMETHODIMP RequestAttrsAtPosition(LONG acpPos, ULONG cFilterAttrs, __RPC__in_ecount_full_opt(cFilterAttrs) const TS_ATTRID *paFilterAttrs, DWORD dwFlags);
	STDMETHODIMP RequestAttrsTransitioningAtPosition(LONG acpPos, ULONG cFilterAttrs, __RPC__in_ecount_full_opt(cFilterAttrs) const TS_ATTRID *paFilterAttrs, DWORD dwFlags);
	STDMETHODIMP FindNextAttrTransition(LONG acpStart, LONG acpHalt, ULONG cFilterAttrs, __RPC__in_ecount_full_opt(cFilterAttrs) const TS_ATTRID *paFilterAttrs, DWORD dwFlags, __RPC__out LONG *pacpNext, __RPC__out BOOL *pfFound, __RPC__out LONG *plFoundOffset);
	STDMETHODIMP RetrieveRequestedAttrs(ULONG ulCount, __RPC__out_ecount_part(ulCount, *pcFetched) TS_ATTRVAL *paAttrVals, __RPC__out ULONG *pcFetched);

	// View Methods
	STDMETHODIMP GetActiveView(__RPC__out TsViewCookie *pvcView);
	STDMETHODIMP GetACPFromPoint(TsViewCookie vcView, __RPC__in const POINT *pt, DWORD dwFlags, __RPC__out LONG *pacp);
	STDMETHODIMP GetTextExt(TsViewCookie vcView, LONG acpStart, LONG acpEnd, __RPC__out RECT *prc, __RPC__out BOOL *pfClipped);
	STDMETHODIMP GetScreenExt(TsViewCookie vcView, __RPC__out RECT *prc);
	STDMETHODIMP GetWnd(TsViewCookie vcView, __RPC__deref_out_opt HWND *phwnd);

	// Plain Character Methods
	STDMETHODIMP GetText(LONG acpStart, LONG acpEnd, __RPC__out_ecount_part(cchPlainReq, *pcchPlainOut) WCHAR *pchPlain, ULONG cchPlainReq, __RPC__out ULONG *pcchPlainOut, __RPC__out_ecount_part(ulRunInfoReq, *pulRunInfoOut) TS_RUNINFO *prgRunInfo, ULONG ulRunInfoReq, __RPC__out ULONG *pulRunInfoOut, __RPC__out LONG *pacpNext);
	STDMETHODIMP QueryInsert(LONG acpInsertStart, LONG acpInsertEnd, ULONG cch, __RPC__out LONG *pacpResultStart, __RPC__out LONG *pacpResultEnd);
	STDMETHODIMP InsertTextAtSelection(DWORD dwFlags, __RPC__in_ecount_full(cch) const WCHAR *pchText, ULONG cch, __RPC__out LONG *pacpStart, __RPC__out LONG *pacpEnd, __RPC__out TS_TEXTCHANGE *pChange);
	STDMETHODIMP SetText(DWORD dwFlags, LONG acpStart, LONG acpEnd, __RPC__in_ecount_full(cch) const WCHAR *pchText, ULONG cch, __RPC__out TS_TEXTCHANGE *pChange);

	// Embedded Character Methods
	STDMETHODIMP GetEmbedded(LONG acpPos, __RPC__in REFGUID rguidService, __RPC__in REFIID riid, __RPC__deref_out_opt IUnknown **ppunk);
	STDMETHODIMP GetFormattedText(LONG acpStart, LONG acpEnd, __RPC__deref_out_opt IDataObject **ppDataObject);
	STDMETHODIMP QueryInsertEmbedded(__RPC__in const GUID *pguidService, __RPC__in const FORMATETC *pFormatEtc, __RPC__out BOOL *pfInsertable);
	STDMETHODIMP InsertEmbedded(DWORD dwFlags, LONG acpStart, LONG acpEnd, __RPC__in_opt IDataObject *pDataObject, __RPC__out TS_TEXTCHANGE *pChange);
	STDMETHODIMP InsertEmbeddedAtSelection(DWORD dwFlags, __RPC__in_opt IDataObject *pDataObject, __RPC__out LONG *pacpStart, __RPC__out LONG *pacpEnd, __RPC__out TS_TEXTCHANGE *pChange);
	// ITextStoreACP Interface End

	// ITfContextOwnerCompositionSink Interface Begin
	STDMETHODIMP OnStartComposition(__RPC__in_opt ITfCompositionView *pComposition, __RPC__out BOOL *pfOk);
	STDMETHODIMP OnUpdateComposition(__RPC__in_opt ITfCompositionView *pComposition, __RPC__in_opt ITfRange *pRangeNew);
	STDMETHODIMP OnEndComposition(__RPC__in_opt ITfCompositionView *pComposition);
	// ITfContextOwnerCompositionSink Interface End

private:
	// Reference count for IUnknown Implementation
	ULONG ReferenceCount;

	// Associated text context that genericizes interfacing with text editing widgets.
	const TSharedRef<ITextInputMethodContext> TextContext;

	struct FSupportedAttribute
	{
		FSupportedAttribute(const TS_ATTRID* const InId) : WantsDefault(false), Id(InId)
		{
			VariantInit(&(DefaultValue));
		}

		bool WantsDefault;
		const TS_ATTRID* const Id;
		VARIANT DefaultValue;
	};

	TArray<FSupportedAttribute> SupportedAttributes;

	struct FLockManager
	{
		FLockManager() : LockType(0), IsPendingLockUpgrade(false)
		{}

		DWORD LockType;
		bool IsPendingLockUpgrade;
	} LockManager;

public:
	struct FAdviseSinkObject
	{
		FAdviseSinkObject() : TextStoreACPSink(nullptr), SinkFlags(0) {}

		// Sink object for ITextStoreACP Implementation
		TComPtr<ITextStoreACPSink> TextStoreACPSink;

		TComPtr<ITextStoreACPServices> TextStoreACPServices;

		// Flags defining what events the sink object should be notified of.
		DWORD SinkFlags;
	} AdviseSinkObject;

	struct FComposition
	{
		FComposition() : TSFCompositionView(nullptr)
		{}

		// Composition view object for managing compositions.
		TComPtr<ITfCompositionView> TSFCompositionView;

	} Composition;

public:
	// Document manager object for managing contexts.
	TComPtr<ITfDocumentMgr> TSFDocumentManager;

	// Context object for pushing context on to document manager.
	TComPtr<ITfContext> TSFContext;

	// Context owner composition services object for terminating compositions.
	TComPtr<ITfContextOwnerCompositionServices> TSFContextOwnerCompositionServices;

	TfEditCookie TSFEditCookie;
};

#include "Windows/HideWindowsPlatformTypes.h"
