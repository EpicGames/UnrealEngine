// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Math/Color.h"
#include "HAL/IConsoleManager.h"
#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/IInputInterface.h"
#include "Windows/AllowWindowsPlatformTypes.h"
	#include <Ole2.h>
	#include <oleidl.h>
	#include <ShObjIdl.h>
#include "Windows/HideWindowsPlatformTypes.h"
#include "Windows/WindowsTextInputMethodSystem.h"

class FGenericWindow;
enum class EWindowTransparency;
class ITextInputMethodSystem;
enum class FForceFeedbackChannelType;
struct FForceFeedbackValues;
struct FHapticFeedbackValues;
class IModularFeature;

DECLARE_LOG_CATEGORY_EXTERN(LogWindowsDesktop, Log, All);

class FWindowsWindow;
class FGenericApplicationMessageHandler;

DECLARE_MULTICAST_DELEGATE_OneParam(FWindowsApplication_OnWindowCreated, HWND);

namespace ETaskbarProgressState
{
	enum Type
	{
		//Stops displaying progress and returns the button to its normal state.
		NoProgress = 0x0,

		//The progress indicator does not grow in size, but cycles repeatedly along the 
		//length of the task bar button. This indicates activity without specifying what 
		//proportion of the progress is complete. Progress is taking place, but there is 
		//no prediction as to how long the operation will take.
		Indeterminate = 0x1,

		//The progress indicator grows in size from left to right in proportion to the 
		//estimated amount of the operation completed. This is a determinate progress 
		//indicator; a prediction is being made as to the duration of the operation.
		Normal = 0x2,

		//The progress indicator turns red to show that an error has occurred in one of 
		//the windows that is broadcasting progress. This is a determinate state. If the 
		//progress indicator is in the indeterminate state, it switches to a red determinate 
		//display of a generic percentage not indicative of actual progress.
		Error = 0x4,

		//The progress indicator turns yellow to show that progress is currently stopped in 
		//one of the windows but can be resumed by the user. No error condition exists and 
		//nothing is preventing the progress from continuing. This is a determinate state. 
		//If the progress indicator is in the indeterminate state, it switches to a yellow 
		//determinate display of a generic percentage not indicative of actual progress.
		Paused = 0x8,
	};
}


/**
 * Allows access to task bar lists.
 *
 * This class can be used to change the appearance of a window's entry in the windows task bar,
 * such as setting an overlay icon or showing a progress indicator.
 */
class FTaskbarList
{
public:

	/**
	 * Create and initialize a new task bar list.
	 *
	 * @return The new task bar list.
	 */
	static APPLICATIONCORE_API TSharedRef<FTaskbarList> Create();

	/**
	 * Sets the overlay icon of a task bar entry.
	 *
	 * @param WindowHandle The window handle to change the overlay icon for.
	 * @param Icon The overlay icon to set.
	 * @param Description The overlay icon's description text.
	 */
	APPLICATIONCORE_API void SetOverlayIcon(HWND WindowHandle, HICON Icon, FText Description);

	/**
	 * Sets the progress state of a task bar entry.
	 *
	 * @param WindowHandle The window handle to change the progress state for.
	 * @param State The new progress state.
	 */
	APPLICATIONCORE_API void SetProgressState(HWND WindowHandle, ETaskbarProgressState::Type State);

	/**
	 * Sets the progress value of a task bar entry.
	 *
	 * @param WindowHandle The window handle to change the progress value for.
	 * @param Current The current progress value.
	 * @param Total The total progress value.
	 */
	APPLICATIONCORE_API void SetProgressValue(HWND WindowHandle, uint64 Current, uint64 Total);

	/** Destructor. */
	APPLICATIONCORE_API ~FTaskbarList();

private:

	/** Hidden constructor (use FTaskbarList::Create). */
	FTaskbarList();

	/** Initializes the task bar list instance. */
	void Initialize();

private:

	/** Holds the internal task bar object. */
	ITaskbarList3* TaskBarList3;
};


struct FDeferredWindowsMessage
{
	FDeferredWindowsMessage( const TSharedPtr<FWindowsWindow>& InNativeWindow, HWND InHWnd, uint32 InMessage, WPARAM InWParam, LPARAM InLParam, int32 InX=0, int32 InY=0, uint32 InRawInputFlags = 0 )
		: NativeWindow( InNativeWindow )
		, hWND( InHWnd )
		, Message( InMessage )
		, wParam( InWParam )
		, lParam( InLParam )
		, X( InX )
		, Y( InY )
		, RawInputFlags( InRawInputFlags )
	{ }

	/** Native window that received the message */
	TWeakPtr<FWindowsWindow> NativeWindow;

	/** Window handle */
	HWND hWND;

	/** Message code */
	uint32 Message;

	/** Message data */
	WPARAM wParam;
	LPARAM lParam;

	/** Mouse coordinates */
	int32 X;
	int32 Y;
	uint32 RawInputFlags;
};


namespace EWindowsDragDropOperationType
{
	enum Type
	{
		DragEnter,
		DragOver,
		DragLeave,
		Drop
	};
}


struct FDragDropOLEData
{
	enum EWindowsOLEDataType
	{
		None = 0,
		Text = 1<<0,
		Files = 1<<1,
	};

	FDragDropOLEData()
		: Type(None)
	{ }

	FString OperationText;
	TArray<FString> OperationFilenames;
	uint8 Type;
};


struct FDeferredWindowsDragDropOperation
{
private:

	// Private constructor. Use the factory functions below.
	FDeferredWindowsDragDropOperation()
		: HWnd(NULL)
		, KeyState(0)
	{
		CursorPosition.x = 0;
		CursorPosition.y = 0;
	}

public:

	static FDeferredWindowsDragDropOperation MakeDragEnter(HWND InHwnd, const FDragDropOLEData& InOLEData, ::DWORD InKeyState, POINTL InCursorPosition)
	{
		FDeferredWindowsDragDropOperation NewOperation;
		NewOperation.OperationType = EWindowsDragDropOperationType::DragEnter;
		NewOperation.HWnd = InHwnd;
		NewOperation.OLEData = InOLEData;
		NewOperation.KeyState = InKeyState;
		NewOperation.CursorPosition = InCursorPosition;
		return NewOperation;
	}

	static FDeferredWindowsDragDropOperation MakeDragOver(HWND InHwnd, ::DWORD InKeyState, POINTL InCursorPosition)
	{
		FDeferredWindowsDragDropOperation NewOperation;
		NewOperation.OperationType = EWindowsDragDropOperationType::DragOver;
		NewOperation.HWnd = InHwnd;
		NewOperation.KeyState = InKeyState;
		NewOperation.CursorPosition = InCursorPosition;
		return NewOperation;
	}

	static FDeferredWindowsDragDropOperation MakeDragLeave(HWND InHwnd)
	{
		FDeferredWindowsDragDropOperation NewOperation;
		NewOperation.OperationType = EWindowsDragDropOperationType::DragLeave;
		NewOperation.HWnd = InHwnd;
		return NewOperation;
	}

	static FDeferredWindowsDragDropOperation MakeDrop(HWND InHwnd, const FDragDropOLEData& InOLEData, ::DWORD InKeyState, POINTL InCursorPosition)
	{
		FDeferredWindowsDragDropOperation NewOperation;
		NewOperation.OperationType = EWindowsDragDropOperationType::Drop;
		NewOperation.HWnd = InHwnd;
		NewOperation.OLEData = InOLEData;
		NewOperation.KeyState = InKeyState;
		NewOperation.CursorPosition = InCursorPosition;
		return NewOperation;
	}

	EWindowsDragDropOperationType::Type OperationType;

	HWND HWnd;
	FDragDropOLEData OLEData;
	::DWORD KeyState;
	POINTL CursorPosition;
};

//disable warnings from overriding the deprecated force feedback.  
//calls to the deprecated function will still generate warnings.
PRAGMA_DISABLE_DEPRECATION_WARNINGS

/**
 * Interface for classes that handle Windows events.
 */
class IWindowsMessageHandler
{
public:

	/**
	 * Processes a Windows message.
	 *
	 * @param hwnd Handle to the window that received the message.
	 * @param msg The message.
	 * @param wParam Additional message information.
	 * @param lParam Additional message information.
	 * @param OutResult Will contain the result if the message was handled.
	 * @return true if the message was handled, false otherwise.
	 */
	virtual bool ProcessMessage(HWND hwnd, uint32 msg, WPARAM wParam, LPARAM lParam, int32& OutResult) = 0;
};


/**
 * Windows-specific application implementation.
 */
class FWindowsApplication
	: public GenericApplication
	, public IInputInterface
{
public:

	/**
	 * Static: Creates a new Win32 application
	 *
	 * @param InstanceHandle Win32 instance handle.
	 * @param IconHandle Win32 application icon handle.
	 * @return New application object.
	 */
	static APPLICATIONCORE_API FWindowsApplication* CreateWindowsApplication( const HINSTANCE InstanceHandle, const HICON IconHandle );

	/** Perform operations after pumping messages on global application. */
	static void StaticPostPumpMessages();

	/** Perform operations after pumping messages. */
	void PostPumpMessages();

	struct RawMouseInputResult
	{
		bool bIsAbsoluteInput;
		bool bSimulateRelativeMovement;
		int32 DeltaX;
		int32 DeltaY;
	};

	/** Process raw mouse input. */
	void ProcessRawMouseInput(const RAWMOUSE& Mouse, RawMouseInputResult& OutResult);

	/** Process raw mouse inputs gathered from a helper thread/window. */
	void ProcessWorkerInputs();

	/** Virtual destructor. */
	APPLICATIONCORE_API virtual ~FWindowsApplication();

public:	

	/** Called by a window when an OLE Drag and Drop operation occurred on a non-game thread */
	APPLICATIONCORE_API void DeferDragDropOperation( const FDeferredWindowsDragDropOperation& DeferredDragDropOperation );

	APPLICATIONCORE_API TSharedPtr<FTaskbarList> GetTaskbarList();

	/** Invoked by a window when an OLE Drag and Drop first enters it. */
	APPLICATIONCORE_API HRESULT OnOLEDragEnter( const HWND HWnd, const FDragDropOLEData& OLEData, ::DWORD KeyState, POINTL CursorPosition, ::DWORD *CursorEffect);

	/** Invoked by a window when an OLE Drag and Drop moves over the window. */
	APPLICATIONCORE_API HRESULT OnOLEDragOver( const HWND HWnd, ::DWORD KeyState, POINTL CursorPosition, ::DWORD *CursorEffect);

	/** Invoked by a window when an OLE Drag and Drop exits the window. */
	APPLICATIONCORE_API HRESULT OnOLEDragOut( const HWND HWnd );

	/** Invoked by a window when an OLE Drag and Drop is dropped onto the window. */
	APPLICATIONCORE_API HRESULT OnOLEDrop( const HWND HWnd, const FDragDropOLEData& OLEData, ::DWORD KeyState, POINTL CursorPosition, ::DWORD *CursorEffect);

	/**
	 * Adds a Windows message handler with the application instance.
	 *
	 * @param MessageHandler The message handler to register.
	 * @see RemoveMessageHandler
	 */
	APPLICATIONCORE_API virtual void AddMessageHandler(IWindowsMessageHandler& InMessageHandler);

	/**
	 * Removes a Windows message handler with the application instance.
	 *
	 * @param MessageHandler The message handler to register.
	 * @see AddMessageHandler
	 */
	APPLICATIONCORE_API virtual void RemoveMessageHandler(IWindowsMessageHandler& InMessageHandler);

public:

	// GenericApplication overrides

	APPLICATIONCORE_API virtual void SetMessageHandler( const TSharedRef< class FGenericApplicationMessageHandler >& InMessageHandler ) override;
#if WITH_ACCESSIBILITY
	APPLICATIONCORE_API virtual void SetAccessibleMessageHandler(const TSharedRef<FGenericAccessibleMessageHandler>& InAccessibleMessageHandler) override;
#endif
	APPLICATIONCORE_API virtual void PollGameDeviceState( const float TimeDelta ) override;
	APPLICATIONCORE_API virtual void PumpMessages( const float TimeDelta ) override;
	APPLICATIONCORE_API virtual void ProcessDeferredEvents( const float TimeDelta ) override;
	APPLICATIONCORE_API virtual TSharedRef< FGenericWindow > MakeWindow() override;
	APPLICATIONCORE_API virtual void InitializeWindow( const TSharedRef< FGenericWindow >& Window, const TSharedRef< FGenericWindowDefinition >& InDefinition, const TSharedPtr< FGenericWindow >& InParent, const bool bShowImmediately ) override;
	APPLICATIONCORE_API virtual void SetCapture( const TSharedPtr< FGenericWindow >& InWindow ) override;
	APPLICATIONCORE_API virtual void* GetCapture( void ) const override;
	virtual bool IsMinimized() const override { return bMinimized; }
	APPLICATIONCORE_API virtual void SetHighPrecisionMouseMode( const bool Enable, const TSharedPtr< FGenericWindow >& InWindow ) override;
	void UpdateHighPrecisionMouseMode();
	virtual bool IsUsingHighPrecisionMouseMode() const override { return bUsingHighPrecisionMouseInput; }
	virtual bool IsMouseAttached() const override { return bIsMouseAttached; }
	APPLICATIONCORE_API virtual bool IsGamepadAttached() const override;
	APPLICATIONCORE_API virtual FModifierKeysState GetModifierKeys() const override;
	APPLICATIONCORE_API virtual bool IsCursorDirectlyOverSlateWindow() const override;
	APPLICATIONCORE_API virtual FPlatformRect GetWorkArea( const FPlatformRect& CurrentWindow ) const override;
	APPLICATIONCORE_API virtual void GetInitialDisplayMetrics( FDisplayMetrics& OutDisplayMetrics ) const override;
	APPLICATIONCORE_API virtual EWindowTitleAlignment::Type GetWindowTitleAlignment() const override;
	APPLICATIONCORE_API virtual EWindowTransparency GetWindowTransparencySupport() const override;
	APPLICATIONCORE_API virtual void DestroyApplication() override;

	virtual IInputInterface* GetInputInterface() override
	{
		return this;
	}

	virtual ITextInputMethodSystem *GetTextInputMethodSystem() override
	{
		return TextInputMethodSystem.Get();
	}

	APPLICATIONCORE_API virtual void AddExternalInputDevice(TSharedPtr<class IInputDevice> InputDevice);

	APPLICATIONCORE_API virtual void FinishedInputThisFrame() override;
public:

	// IInputInterface overrides

	APPLICATIONCORE_API virtual void SetForceFeedbackChannelValue (int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) override;
	APPLICATIONCORE_API virtual void SetForceFeedbackChannelValues(int32 ControllerId, const FForceFeedbackValues &Values) override;
	APPLICATIONCORE_API virtual void SetHapticFeedbackValues(int32 ControllerId, int32 Hand, const FHapticFeedbackValues& Values) override;
	APPLICATIONCORE_API virtual void SetLightColor(int32 ControllerId, FColor Color) override;
	APPLICATIONCORE_API virtual void ResetLightColor(int32 ControllerId) override;
	APPLICATIONCORE_API virtual void SetDeviceProperty(int32 ControllerId, const FInputDeviceProperty* Property) override;

	// Touchpad Sensitivity
	/** On Windows 10 22H2 and later where we can set the touchpad sensitivity system wide setting force it to most sensitive so that touch swipes are not ignored while keyboard keys are being depressed.
	 * In Windows 11 24H2 a new api was added which can change this setting only for this app.
	 * 
	 * We reccomend that this feature be exposed as a user setting which is enabled by default and hidden/disabled if SupportsForceMaxTouchpadSensitivity() returns false and that the CVar be used to activate it rather than directly calling 
	 * ApplyForceMaxTouchpadSensitivity/RemoveForceMaxTouchpadSensitivity due to the complexity of handling the focus/exit/crash cases with the SetTouchpadParameters api.
	 * 
	 * See WindowsApplication.cpp namespace UE::WindowsApplication::TouchpadSensitivity for more information about this system.
	 */
	enum class EForceMaxTouchpadSensitivityAsyncBehavior : uint8
	{
		AllowAsynchronous, // This function may defer the system call that actually changes the setting in order to avoid blocking for a long time.
		RequireBlocking // This function will not return until the system call returns.
	};
	APPLICATIONCORE_API bool SupportsForceMaxTouchpadSensitivity();
	APPLICATIONCORE_API void ApplyForceMaxTouchpadSensitivity();
	APPLICATIONCORE_API void RemoveForceMaxTouchpadSensitivity();

	/** If we are using the system wide setting it is possible that we could crash and fail to set it back.  Call this function to
	 * find out if we have a system setting value we would like to restore after a crash.
	 * If this information was passed to a separate process (perhaps by writing it to a file) that process might be able to restore the setting
	 * after a crash.
	 */
	enum class EForceMaxTouchpadSensitivityRestorationValues : int32
	{
		SystemCallFailed = -2, // Indicates we are in a mode where restoration is desirable, but the system call failed.  Most likely callers should do nothing.
		RestorationNotNeeded = -1,  // Indicates restoration is unnecessary (either the system setting was already MostSensitive, we can't change the setting, or the system is fully capable of restoring itself.)
		TOUCHPAD_SENSITIVITY_LEVEL_MOST_SENSITIVE = 0x00000000,  // These values one would restore with ::SystemParametersInfo(SPI_SETTOUCHPADPARAMETERS, ...
		TOUCHPAD_SENSITIVITY_LEVEL_HIGH_SENSITIVITY = 0x00000001,
		TOUCHPAD_SENSITIVITY_LEVEL_MEDIUM_SENSITIVITY = 0x00000002,
		TOUCHPAD_SENSITIVITY_LEVEL_LOW_SENSITIVITY = 0x00000003,
		TOUCHPAD_SENSITIVITY_LEVEL_LEAST_SENSITIVE = 0x00000004,
	};
	APPLICATIONCORE_API EForceMaxTouchpadSensitivityRestorationValues GetForceMaxTouchpadSensitivityRestorationValue();

protected:
	friend LRESULT WindowsApplication_WndProc(HWND hwnd, uint32 msg, WPARAM wParam, LPARAM lParam);

	/** Windows callback for message processing (forwards messages to the FWindowsApplication instance). */
	static APPLICATIONCORE_API LRESULT CALLBACK AppWndProc(HWND hwnd, uint32 msg, WPARAM wParam, LPARAM lParam);

	/** Processes a single Windows message. */
	APPLICATIONCORE_API int32 ProcessMessage( HWND hwnd, uint32 msg, WPARAM wParam, LPARAM lParam );

	/** Processes mouse button. */
	void ProcessMouseButton(double CurrentTime, const TSharedPtr< FGenericWindow >& Window, USHORT ButtonFlags, USHORT DownFlag, USHORT UpFlag, EMouseButtons::Type MouseButton, FVector2D CursorPos);

	/** Processes a deferred Windows message. */
	APPLICATIONCORE_API int32 ProcessDeferredMessage( const FDeferredWindowsMessage& DeferredMessage );

	/** Processes deferred drag and drop operations. */
	APPLICATIONCORE_API void ProcessDeferredDragDropOperation(const FDeferredWindowsDragDropOperation& Op);

	/** Hidden constructor. */
	APPLICATIONCORE_API FWindowsApplication( const HINSTANCE HInstance, const HICON IconHandle );

	APPLICATIONCORE_API void ApplyLowLevelMouseFilter();
	APPLICATIONCORE_API void RemoveLowLevelMouseFilter();

	static APPLICATIONCORE_API LRESULT CALLBACK HandleLowLevelMouseFilterHook(int nCode, WPARAM wParam, LPARAM lParam);

	HHOOK LowLevelMouseFilterHook;
	bool bLowLevelMouseFilterIsApplied = false;

private:

	/** Registers the Windows class for windows and assigns the application instance and icon */
	static APPLICATIONCORE_API bool RegisterClass( const HINSTANCE HInstance, const HICON HIcon );

	/**  @return  True if a windows message is related to user input from the keyboard */
	static APPLICATIONCORE_API bool IsKeyboardInputMessage( uint32 msg );

	/**  @return  True if a windows message is related to user input from the mouse */
	static APPLICATIONCORE_API bool IsMouseInputMessage( uint32 msg );

	/**  @return  True if a windows message is a fake mouse input message generated after a WM_TOUCH event */
	static APPLICATIONCORE_API bool IsFakeMouseInputMessage(uint32 msg);

	/**  @return  True if a windows message is related to user input (mouse, keyboard) */
	static APPLICATIONCORE_API bool IsInputMessage( uint32 msg );

	/** Defers a Windows message for later processing. */
	APPLICATIONCORE_API void DeferMessage( TSharedPtr<FWindowsWindow>& NativeWindow, HWND InHWnd, uint32 InMessage, WPARAM InWParam, LPARAM InLParam, int32 MouseX = 0, int32 MouseY = 0, uint32 RawInputFlags = 0 );

	/** Checks a key code for release of the Shift key. */
	APPLICATIONCORE_API void CheckForShiftUpEvents(const int32 KeyCode);

	/** Shuts down the application (called after an unrecoverable error occurred). */
	APPLICATIONCORE_API void ShutDownAfterError();

	/** Enables or disables Windows accessibility features, such as sticky keys. */
	APPLICATIONCORE_API void AllowAccessibilityShortcutKeys(const bool bAllowKeys);

	/** Queries and caches the number of connected mouse devices. */
	APPLICATIONCORE_API void QueryConnectedMice();

	/** Gets the touch index for a given windows touch ID. */
	APPLICATIONCORE_API uint32 GetTouchIndexForID(int32 TouchID);

	/** Searches for a free touch index. */
	APPLICATIONCORE_API uint32 GetFirstFreeTouchIndex();

	/** Helper function to update the cached states of all modifier keys */
	APPLICATIONCORE_API void UpdateAllModifierKeyStates();

	APPLICATIONCORE_API FPlatformRect GetWorkAreaFromOS(const FPlatformRect& CurrentWindow) const;

	/** Registers for raw input */
	void RegisterHighPrecisionMouseMode();

	/** Unregisters for raw input */
	void UnregisterHighPrecisionMouseMode();

	void OnInputDeviceModuleRegistered(const FName& Type, IModularFeature* ModularFeature);

private:

	static APPLICATIONCORE_API const FIntPoint MinimizedWindowPosition;

	HINSTANCE InstanceHandle;

	bool bMinimized;

	bool bUsingHighPrecisionMouseInput;
	bool bCachedUseWorkerThreadForRawInput;
	HWND HighPrecisionMouseInputWindowHandle;
	HWND CaptureWindowHandle;

	bool bIsMouseAttached;

	bool bForceActivateByMouse;

	bool bForceNoGamepads;

	bool bConsumeAltSpace;

	TArray<FDeferredWindowsMessage> DeferredMessages;

	TArray<FDeferredWindowsDragDropOperation> DeferredDragDropOperations;

	/** Registered Windows message handlers. */
	TArray<IWindowsMessageHandler*> MessageHandlers;

	TArray<TSharedRef<FWindowsWindow>> Windows;

	/** List of input devices implemented in external modules. */
	TArray<TSharedPtr<class IInputDevice>> ExternalInputDevices;
	bool bHasLoadedInputPlugins;

	struct EModifierKey
	{
		enum Type
		{
			LeftShift,		// VK_LSHIFT
			RightShift,		// VK_RSHIFT
			LeftControl,	// VK_LCONTROL
			RightControl,	// VK_RCONTROL
			LeftAlt,		// VK_LMENU
			RightAlt,		// VK_RMENU
			CapsLock,		// VK_CAPITAL
			Count,
		};
	};
	/** Cached state of the modifier keys. True if the modifier key is pressed (or toggled in the case of caps lock), false otherwise */
	bool ModifierKeyState[EModifierKey::Count];

	int32 bAllowedToDeferMessageProcessing;
	
	FAutoConsoleVariableRef CVarDeferMessageProcessing;

	FDisplayMetrics InitialDisplayMetrics;

	TSharedPtr<FWindowsTextInputMethodSystem> TextInputMethodSystem;

	TSharedPtr<FTaskbarList> TaskbarList;

#if WITH_ACCESSIBILITY && UE_WINDOWS_USING_UIA
	/** Handler for WM_GetObject messages that come in */
	TUniquePtr<class FWindowsUIAManager> UIAManager;
#endif

	// Accessibility shortcut keys
	STICKYKEYS							StartupStickyKeys;
	TOGGLEKEYS							StartupToggleKeys;
	FILTERKEYS							StartupFilterKeys;

	struct TouchInfo
	{
		bool HasMoved;
		FVector2D PreviousLocation;
		TOptional<int32> TouchID;

		TouchInfo()
			: HasMoved(false)
			, PreviousLocation(0.f, 0.f)
		{ }
	};
	/** Maps touch information such as TouchID PreviousLocation and HasMoved to windows touch IDs. */
	TArray<TouchInfo> TouchInfoArray;

	bool bSimulatingHighPrecisionMouseInputForRDP;

	FIntPoint LastCursorPoint;
	FIntPoint LastCursorPointPreWrap;
	int32 NumPreWrapMsgsToRespect;
	RECT ClipCursorRect;
	DWORD CachedDragAndDropKeyState;

	double RepeatTime[EMouseButtons::Invalid];
};


PRAGMA_ENABLE_DEPRECATION_WARNINGS

