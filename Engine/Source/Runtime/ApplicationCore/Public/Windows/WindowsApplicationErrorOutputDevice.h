// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Windows/WindowsErrorOutputDevice.h"

/**
 * Windows implementation of console log window, utilizing the Win32 console API
 */
class FWindowsApplicationErrorOutputDevice : public FWindowsErrorOutputDevice
{
protected:
	virtual void HandleErrorRestoreUI() override;
};

