// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef WITH_WINDOWS_CONFIGRULES
#define WITH_WINDOWS_CONFIGRULES 0
#endif // WITH_WINDOWS_CONFIGRULES

#if WITH_WINDOWS_CONFIGRULES

#include "CoreMinimal.h"
#include "GenericPlatform/GenericConfigRules.h"

class FWindowsConfigRules : public FGenericConfigRules
{
	static TMap<FString, FString> PredefinedConfigRuleVars;
	static TMap<FString, FString> ConfigRuleVariablesMap;
	static TArray<uint8> ConfigRulesBytes;

public:

	/** Finds the best config rules file and loads it for parsing */
	static APPLICATIONCORE_API void Init();

	/**
	 * Parse the loaded config rules file and updates the variable map.
	 *
	 * A concrete use case for Reparsing is when some data in PredefinedVars is not available during the first parse and
	 * we'd like to update the RulesMap with new data. E.g. RHI might not be initialized during first Parse but we'd still
	 * like to apply other values before RHIInit. Then during RHIInit call Parse() again to get updated data.
	 *
	 * Can be reparsed until EndOfEngineInit or Release()
	 */
	static APPLICATIONCORE_API void Parse();

	/** Sets a number of standard predefined variables such as build version, CPU information, etc. */
	static APPLICATIONCORE_API void SetStandardPredefinedVars();

	/** 
	 * Sets a number of RHI-specific predefined variables such as adapter name, driver version, etc.
	 * Requires the global GRHIAdapterName to be set by the time this is called.
	*/
	static APPLICATIONCORE_API void SetRHIPredefinedVars(const FString& DynamicRHIModuleName, const FString& RequestedFeatureLevel);

	static APPLICATIONCORE_API void SetAdditionalPredefinedVars(TMap<FString, FString>&& AdditionalPredefinedVars);
	
	/** Frees resources used to parse config rules */
	static APPLICATIONCORE_API void Release();
	
	static const TMap<FString, FString>& GetConfigRulesMap() { return ConfigRuleVariablesMap; }
};

#endif // WITH_WINDOWS_CONFIGRULES
