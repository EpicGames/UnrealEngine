// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/OutputDeviceConsole.h"
#include "UObject/NameTypes.h"

/**
 * Windows implementation of console log window, utilizing the Win32 console API
 */
class FWindowsConsoleOutputDevice2 final : public FOutputDeviceConsole
{
private:
	uint16 TextAttribute;

	/** true if the color is currently set by the caller */
	bool OverrideColorSet;

	class FConsoleWindow;
	FConsoleWindow* Window;
	FRWLock WindowRWLock;
	TArray<FString> CommandHistory;
	int32 CommandHistoryCursor = 0;

	/** maximum allowed log verbosity */
	ELogVerbosity::Type AllowedLogVerbosity = ELogVerbosity::All;

#if !UE_BUILD_SHIPPING
	/** An entry for log category highlighting */
	struct FLogHighlight
	{
		/** The category to highlight */
		FName Category;

		/** The 4 digit color code to highlight with */
		const TCHAR* Color = nullptr;

		bool operator == (FName InCategory) const
		{
			return Category == InCategory;
		}
	};

	/** Log categories to be highlighted */
	TArray<FLogHighlight, TInlineAllocator<8>> LogHighlights;

	/** An entry for log string highlighting */
	struct FLogStringHighlight
	{
		/** The string to search for and highlight */
		TStringBuilder<128> SearchString;

		/** The 4 digit color code to highlight with */
		const TCHAR* Color = nullptr;
	};

	/** Log strings to be highlighted */
	TArray<FLogStringHighlight, TInlineAllocator<8>> LogStringHighlights;
#endif

	/**
	 * Saves the console window's position and size to the game .ini
	 */
	void SaveToINI();

	static const FString& GetConfigFilename();

public:

	/** 
	 * Constructor, setting console control handler.
	 */
	APPLICATIONCORE_API FWindowsConsoleOutputDevice2();
	APPLICATIONCORE_API ~FWindowsConsoleOutputDevice2();

	/**
	 * Shows or hides the console window. 
	 *
	 * @param ShowWindow	Whether to show (true) or hide (false) the console window.
	 */
	APPLICATIONCORE_API virtual void Show( bool ShowWindow );

	/** 
	 * Returns whether console is currently shown or not
	 *
	 * @return true if console is shown, false otherwise
	 */
	APPLICATIONCORE_API virtual bool IsShown();

	APPLICATIONCORE_API virtual bool IsAttached();

	APPLICATIONCORE_API virtual bool CanBeUsedOnAnyThread() const override;

	/**
	 * Displays text on the console and scrolls if necessary.
	 *
	 * @param Data	Text to display
	 * @param Event	Event type, used for filtering/ suppression
	 */
	APPLICATIONCORE_API virtual void Serialize( const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category, const double Time ) override;
	APPLICATIONCORE_API virtual void Serialize( const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category ) override;

	APPLICATIONCORE_API void SetColor( const TCHAR* Color );

	bool GetPreviousCommand(FString& OutCommand);
	bool GetNextCommand(FString& OutCommand);
	bool GetLastCommand(FString& OutCommand);
	void AddCommand(const FString& InCommand);
private:
	FWindowsConsoleOutputDevice2(const FWindowsConsoleOutputDevice2&) = delete;
	FWindowsConsoleOutputDevice2& operator=(const FWindowsConsoleOutputDevice2&) = delete;
};
