// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/PlatformMemory.h"
#include "GenericPlatform/ICursor.h"
#include "Windows/WindowsHWrapper.h"

class FWindowsCursor : public ICursor
{
public:

	FWindowsCursor();

	virtual ~FWindowsCursor();

	virtual void* CreateCursorFromFile(const FString& InPathToCursorWithoutExtension, FVector2D HotSpot) override;

	virtual bool IsCreateCursorFromRGBABufferSupported() const override
	{
		return true;
	}

	virtual void* CreateCursorFromRGBABuffer(const FColor* Pixels, int32 Width, int32 Height, FVector2D InHotSpot) override;

	virtual FVector2D GetPosition() const override;

	virtual void SetPosition( const int32 X, const int32 Y ) override;

	virtual void SetType( const EMouseCursor::Type InNewCursor ) override;

	virtual EMouseCursor::Type GetType() const override
	{
		return CurrentType;
	}

	virtual void GetSize( int32& Width, int32& Height ) const override;

	virtual void Show( bool bShow ) override;

	virtual void Tick() override;
	virtual void Lock( const RECT* const Bounds ) override;

	virtual void SetTypeShape(EMouseCursor::Type InCursorType, void* CursorHandle) override;

private:

	EMouseCursor::Type CurrentType;

	/** Cursors */
	HCURSOR CursorHandles[ EMouseCursor::TotalCursorCount ];

	/** Override Cursors */
	HCURSOR CursorOverrideHandles[EMouseCursor::TotalCursorCount];
};