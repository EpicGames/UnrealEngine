// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/CoreMisc.h"
#include "Misc/FeedbackContext.h"
#include "Misc/OutputDeviceConsole.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/OutputDeviceRedirector.h"

/**
 * Feedback context implementation for windows.
 */
class FWindowsFeedbackContext : public FFeedbackContext
{
	/** Context information for warning and error messages */
	FContextSupplier* Context = nullptr;

public:
	APPLICATIONCORE_API bool YesNof(const FText& Question) override;

	FContextSupplier* GetContext() const { return Context; }
	void SetContext(FContextSupplier* InContext) { Context = InContext; }
};
