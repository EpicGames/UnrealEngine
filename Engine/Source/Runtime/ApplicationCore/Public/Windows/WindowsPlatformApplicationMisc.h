// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformApplicationMisc.h"

struct FWindowsPlatformApplicationMisc : public FGenericPlatformApplicationMisc
{
	static APPLICATIONCORE_API void PreInit();
	static APPLICATIONCORE_API void LoadStartupModules();
	static APPLICATIONCORE_API class FOutputDeviceConsole* CreateConsoleOutputDevice();
	static APPLICATIONCORE_API class FOutputDeviceError* GetErrorOutputDevice();
	static APPLICATIONCORE_API class FFeedbackContext* GetFeedbackContext();
	static APPLICATIONCORE_API class GenericApplication* CreateApplication();
	static APPLICATIONCORE_API void RequestMinimize();
	static APPLICATIONCORE_API bool IsThisApplicationForeground();
	static APPLICATIONCORE_API int32 GetAppIcon();
	static APPLICATIONCORE_API void PumpMessages(bool bFromMainLoop);
	static APPLICATIONCORE_API void PreventScreenSaver();
	static APPLICATIONCORE_API struct FLinearColor GetScreenPixelColor(const FVector2D& InScreenPos, float InGamma = 1.0f);
	static APPLICATIONCORE_API void SetHighDPIMode();
	static APPLICATIONCORE_API bool GetWindowTitleMatchingText(const TCHAR* TitleStartsWith, FString& OutTitle);
	static APPLICATIONCORE_API float GetDPIScaleFactorAtPoint(float X, float Y);
	static APPLICATIONCORE_API void ClipboardCopy(const TCHAR* Str);
	static APPLICATIONCORE_API void ClipboardPaste(class FString& Dest);

	/** Windows platform only */
	/** Function should retrieve the DPI value for the provided monitor information structure */
	static APPLICATIONCORE_API int32 GetMonitorDPI(const FMonitorInfo& MonitorInfo);
	static APPLICATIONCORE_API int32 GetMonitorDPI(void* NativeHandle);

	struct FGPUInfo
	{
		uint32 VendorId = 0;
		uint32 DeviceId = 0;
		uint64 DedicatedVideoMemory = 0;
	};

	static APPLICATIONCORE_API FGPUInfo GetBestGPUInfo();
	/** End Windows platform only */
};

#if defined(WINDOWS_USE_FEATURE_APPLICATIONMISC_CLASS) && WINDOWS_USE_FEATURE_APPLICATIONMISC_CLASS
typedef FWindowsPlatformApplicationMisc FPlatformApplicationMisc;
#endif
