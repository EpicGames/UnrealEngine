// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformSplash.h"

/**
 * Windows splash implementation.
 */
struct FWindowsPlatformSplash
	: public FGenericPlatformSplash
{
	/** Show the splash screen. */
	static APPLICATIONCORE_API void Show();

	/** Hide the splash screen. */
	static APPLICATIONCORE_API void Hide();

	/**
	 * Sets the progress displayed on the application icon (for startup/loading progress).
	 *
	 * @param ProgressPercent Progress value in percent.
	 */
	static APPLICATIONCORE_API void SetProgress(int ProgressPercent);

	/**
	 * Sets the text displayed on the splash screen (for startup/loading progress).
	 *
	 * @param InType Type of text to change.
	 * @param InText Text to display.
	 */
	using FGenericPlatformSplash::SetSplashText;
	static APPLICATIONCORE_API void SetSplashText( const SplashTextType::Type InType, const FText& InText );

	/** Sets the image to be assigned to the specific slot */
	static APPLICATIONCORE_API void SetSplashImage(const SplashImageType::Type InType, const TCHAR* InFilename);

	/** Registers a new licensee text. Only User types are allowed, and only before the splash screen is initialized! */
	static APPLICATIONCORE_API void RegisterUserSplashText(const SplashTextType::Type InType, const FPlatformRect& InRect, ESplashTextSizeType InSize, ESplashTextBrightnessType InBrightness, ESplashTextFlags InFlags = ESplashTextFlags::None);

	/** Registers a new licensee image. Only User types are allowed, and only before the splash screen is initialized! */
	static APPLICATIONCORE_API void RegisterUserSplashImage(const SplashImageType::Type InType, const FIntPoint& InPos, const TCHAR* InFilename);

	/**
	 * Return whether the splash screen is being shown or not
	 */
	static APPLICATIONCORE_API bool IsShown();
};


typedef FWindowsPlatformSplash FPlatformSplash;
