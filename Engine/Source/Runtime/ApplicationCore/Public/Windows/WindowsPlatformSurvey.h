// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformSurvey.h"


/**
 * Windows implementation of FGenericPlatformSurvey.
 */
struct FWindowsPlatformSurvey
	: public FGenericPlatformSurvey
{
	/** Start, or check on, the hardware survey */
	static APPLICATIONCORE_API bool GetSurveyResults( FHardwareSurveyResults& OutResults, bool bWait = false );

private:
	/**
	 * Helper used by SurveyHardware to get the Windows Experience Index score for individual sub-components
	 * @return True if successful
	 */
	static bool GetSubComponentIndex( struct IProvideWinSATResultsInfo* WinSATResults, FHardwareSurveyResults& OutSurveyResults, int32 SubComponent, float& OutSubComponentIndex );

	/**
	 * Find the line that contains Token and return the contents of the line after the token
	 * @return True if the line is found and the function has written the following string to OutString
	 */
	static bool GetLineFollowing(const FString& Token, const TArray<FString>& Lines, FString& OutString, int32 NthHit = 0);

	/**
	 * Look for a section of a file beginning with SectionName
	 */
	static bool GetNamedSection(FString SectionName, const TArray<FString>& InLines, TArray<FString>& OutSectionLines);
};


typedef FWindowsPlatformSurvey FPlatformSurvey;
