// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/OutputDeviceConsole.h"

#if !defined(WITH_REMOTEWIN_CONSOLE)
#define WITH_REMOTEWIN_CONSOLE (!UE_BUILD_SHIPPING) && (!WITH_EDITOR)
#endif

#if WITH_REMOTEWIN_CONSOLE

/**
 * 
 * Implementation of console logging for a remote windows deployment
 * 
 * Likely placeholder until the wdremote tools support process lifetime management & output redirection.
 * Paired with Placeholder_RemoteWinProcessMonitor in UAT
 * 
 * Not enabled in shipping builds by default (see WITH_REMOTEWIN_CONSOLE)
 * 
 */
class FPlaceholder_WindowsRemoteConsoleOutputDevice : public FOutputDeviceConsole
{
public:

	/** Default constructor. */
	APPLICATIONCORE_API FPlaceholder_WindowsRemoteConsoleOutputDevice( const FString& RemoteConsoleHost );

	/** Destructor. */
	APPLICATIONCORE_API ~FPlaceholder_WindowsRemoteConsoleOutputDevice();

	/** Sends the string to the remote logging host */
	APPLICATIONCORE_API void Serialize( const TCHAR* Msg, ELogVerbosity::Type Verbosity, const class FName& Category );

private:
	virtual void Show(bool ShowWindow) {}
	virtual bool IsShown() { return true; }

	class FRemoteConsoleRunnable* Runnable;
	class FRunnableThread* Thread;
};

#endif // WITH_REMOTEWIN_CONSOLE
