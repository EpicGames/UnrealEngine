// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "GenericPlatform/ITextInputMethodSystem.h"
#include "Windows/TextStoreACP.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include <msctf.h>
#include "Microsoft/COMPointer.h"

class FGenericWindow;

class FTextStoreACP;
class FWindowsTextInputMethodSystem;


class FTSFActivationProxy
	: public ITfInputProcessorProfileActivationSink
	, public ITfActiveLanguageProfileNotifySink
{
public:
	FTSFActivationProxy(FWindowsTextInputMethodSystem* InOwner) 
		: TSFProfileCookie(TF_INVALID_COOKIE)
		, TSFLanguageCookie(TF_INVALID_COOKIE)
		, Owner(InOwner)
		, ReferenceCount(1)
	{ }

	virtual ~FTSFActivationProxy() = default;

	// IUnknown Interface Begin
	STDMETHODIMP QueryInterface(REFIID riid, void **ppvObj) override;
	STDMETHODIMP_(ULONG) AddRef() override;
	STDMETHODIMP_(ULONG) Release() override;
	// IUnknown Interface End

	// ITfInputProcessorProfileActivationSink Interface Begin
	STDMETHODIMP OnActivated(DWORD dwProfileType, LANGID langid, __RPC__in REFCLSID clsid, __RPC__in REFGUID catid, __RPC__in REFGUID guidProfile, HKL hkl, DWORD dwFlags) override;
	// ITfInputProcessorProfileActivationSink Interface End

	// ITfActiveLanguageProfileNotifySink Interface Begin
	STDMETHODIMP OnActivated(REFCLSID clsid, REFGUID guidProfile, BOOL fActivated) override;
	// ITfActiveLanguageProfileNotifySink Interface End

public:

	DWORD TSFProfileCookie;
	DWORD TSFLanguageCookie;

private:

	FWindowsTextInputMethodSystem* Owner;

	// Reference count for IUnknown Implementation
	ULONG ReferenceCount;
};


class FWindowsTextInputMethodSystem
	: public ITextInputMethodSystem
{
	friend class FTSFActivationProxy;

public:

	virtual ~FWindowsTextInputMethodSystem() = default;

	bool Initialize();
	void Terminate();

	int32 ProcessMessage(HWND hwnd, uint32 msg, WPARAM wParam, LPARAM lParam);

public:

	// ITextInputMethodSystem interface

	virtual void ApplyDefaults(const TSharedRef<FGenericWindow>& InWindow) override;
	virtual TSharedPtr<ITextInputMethodChangeNotifier> RegisterContext(const TSharedRef<ITextInputMethodContext>& Context) override;
	virtual void UnregisterContext(const TSharedRef<ITextInputMethodContext>& Context) override;
	virtual void ActivateContext(const TSharedRef<ITextInputMethodContext>& Context) override;
	virtual void DeactivateContext(const TSharedRef<ITextInputMethodContext>& Context) override;
	virtual bool IsActiveContext(const TSharedRef<ITextInputMethodContext>& Context) const override;

private:

	void LogActiveInputMethod();

	// IMM Implementation

	bool InitializeIMM();
	void UpdateIMMProperty(HKL KeyboardLatoutHandle);
	bool ShouldDrawIMMCompositionString() const;
	void UpdateIMMWindowPositions(HIMC IMMContext);
	void BeginIMMComposition();
	void EndIMMComposition();
	void CancelIMMComposition();

	// TSF Implementation

	bool InitializeTSF();
	void OnIMEActivationStateChanged(const bool bIsEnabled);
	void ClearStaleWindowHandles();

private:

	enum class EAPI
	{
		Unknown,
		IMM,
		TSF
	} CurrentAPI;

	// TSF Implementation
	TComPtr<ITfInputProcessorProfiles> TSFInputProcessorProfiles;
	TComPtr<ITfInputProcessorProfileMgr> TSFInputProcessorProfileManager;
	TComPtr<ITfThreadMgr> TSFThreadManager;
	TfClientId TSFClientId = 0;
	TComPtr<ITfDocumentMgr> TSFDisabledDocumentManager;
	TComPtr<FTSFActivationProxy> TSFActivationProxy;

	// IMM Implementation
	HIMC IMMContextId = nullptr;
	DWORD IMMProperties = 0;

	struct FInternalContext
	{
		HWND WindowHandle = nullptr;

		TComPtr<FTextStoreACP> TSFContext;

		struct
		{
			bool IsComposing = false;
			bool IsDeactivating = false;
			int32 CompositionBeginIndex = 0;
			uint32 CompositionLength = 0;
		} IMMContext;
	};

	TSharedPtr<ITextInputMethodContext> ActiveContext;

	TMap< TWeakPtr<ITextInputMethodContext>, FInternalContext > ContextToInternalContextMap;

	TSet<TWeakPtr<FGenericWindow>> KnownWindows;
};


#include "Windows/HideWindowsPlatformTypes.h"
