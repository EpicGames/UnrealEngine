// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "GenericPlatform/GenericWindow.h"
#include "Templates/SharedPointer.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include <Ole2.h>
#include <oleidl.h>
#include "Windows/HideWindowsPlatformTypes.h"

class FWindowsApplication;
enum class EWindowTransparency;

/**
 * A platform specific implementation of FNativeWindow.
 *
 * Native Windows provide platform-specific backing for and are always owned by an SWindow.
 */
class FWindowsWindow
	: public FGenericWindow
	, public IDropTarget
{
public:

	/** Win32 requirement: see CreateWindowEx and RegisterClassEx. */
	static APPLICATIONCORE_API const TCHAR AppWindowClass[];

public:

	/** Destructor. */
	APPLICATIONCORE_API ~FWindowsWindow();

	/** Create a new FWin32Window. */
	static APPLICATIONCORE_API TSharedRef<FWindowsWindow> Make();

	/**
	 * Gets the Window's handle.
	 *
	 * @return The window's HWND handle.
	 */
	APPLICATIONCORE_API HWND GetHWnd() const;

	APPLICATIONCORE_API void Initialize( class FWindowsApplication* const Application, const TSharedRef<FGenericWindowDefinition>& InDefinition, HINSTANCE InHInstance, const TSharedPtr<FWindowsWindow>& InParent, const bool bShowImmediately );

	/**
	* Sets the window text (usually the title but can also be text content for things like controls).
	*
	* @param Text The window's title or content text
	*/
	APPLICATIONCORE_API bool IsRegularWindow() const;

	/**
     * Sets the window region to specified dimensions.
	 *
	 * @param Width The width of the window region (in pixels).
	 * @param Height The height of the window region (in pixels).
	 */
	APPLICATIONCORE_API void AdjustWindowRegion(int32 Width, int32 Height);

	/** @return	Gives the native window a chance to adjust our stored window size before we cache it off. */
	APPLICATIONCORE_API virtual void AdjustCachedSize(FVector2D& Size) const override;

	virtual float GetDPIScaleFactor() const override
	{
		return DPIScaleFactor;
	}

	virtual void SetDPIScaleFactor(float Value) override
	{
		DPIScaleFactor = Value;
	}

	/** determines whether or not this window does its own DPI management */
	APPLICATIONCORE_API virtual bool IsManualManageDPIChanges() const override;

	/** call with a true argument if this window need to do its custom size management in response to DPI variations */
	APPLICATIONCORE_API virtual void SetManualManageDPIChanges(const bool bManualDPIChanges) override;

	bool IsInModalSizeLoop() const { return bInModalSizeLoop; }
	void EnterModalSizeLoop() { bInModalSizeLoop = true; }
	void ExitModalSizeLoop() { bInModalSizeLoop = false; }

	/** Called when our parent window is minimized (which will in turn cause us to become minimized). */
	APPLICATIONCORE_API void OnParentWindowMinimized();

	/** Called when our parent window is restored (which will in turn cause us to become restored). */
	APPLICATIONCORE_API void OnParentWindowRestored();

	/** Called by the owning application when the level of transparency support has changed */
	APPLICATIONCORE_API void OnTransparencySupportChanged(EWindowTransparency NewTransparency);

	float GetAspectRatio() const { return AspectRatio; }

	/** @return True if the window is enabled */
	APPLICATIONCORE_API bool IsEnabled();

	// Attempts to retrieve the window's pixels using PrintWindow (https://learn.microsoft.com/en-us/windows/win32/api/winuser/nf-winuser-printwindow).
	// Only the client area's pixels are retrieved, which excludes the title bar, border, etc.
	APPLICATIONCORE_API TArray<FColor> GetWindowPixels(uint32& OutWidth, uint32& OutHeight);

public:

	// FGenericWindow interface

	APPLICATIONCORE_API virtual void ReshapeWindow( int32 X, int32 Y, int32 Width, int32 Height ) override;
	APPLICATIONCORE_API virtual bool GetFullScreenInfo( int32& X, int32& Y, int32& Width, int32& Height ) const override;
	APPLICATIONCORE_API virtual void MoveWindowTo ( int32 X, int32 Y ) override;
	APPLICATIONCORE_API virtual void BringToFront( bool bForce = false ) override;
	APPLICATIONCORE_API virtual void HACK_ForceToFront() override;
	APPLICATIONCORE_API virtual void Destroy() override;
	APPLICATIONCORE_API virtual void Minimize() override;
	APPLICATIONCORE_API virtual void Maximize() override;
	APPLICATIONCORE_API virtual void Restore() override;
	APPLICATIONCORE_API virtual void Show() override;
	APPLICATIONCORE_API virtual void Hide() override;
	APPLICATIONCORE_API virtual void SetWindowMode( EWindowMode::Type NewWindowMode ) override;
	virtual EWindowMode::Type GetWindowMode() const override { return WindowMode; } 
	APPLICATIONCORE_API virtual bool IsMaximized() const override;
	APPLICATIONCORE_API virtual bool IsMinimized() const override;
	APPLICATIONCORE_API virtual bool IsVisible() const override;
	APPLICATIONCORE_API virtual bool GetRestoredDimensions(int32& X, int32& Y, int32& Width, int32& Height) override;
	APPLICATIONCORE_API virtual void SetWindowFocus() override;
	APPLICATIONCORE_API virtual void SetOpacity( const float InOpacity ) override;
	APPLICATIONCORE_API virtual void Enable( bool bEnable ) override;
	APPLICATIONCORE_API virtual bool IsPointInWindow( int32 X, int32 Y ) const override;
	APPLICATIONCORE_API virtual int32 GetWindowBorderSize() const override;
	APPLICATIONCORE_API virtual int32 GetWindowTitleBarSize() const override;
	virtual void* GetOSWindowHandle() const  override { return HWnd; }
	APPLICATIONCORE_API virtual bool IsForegroundWindow() const override;
	APPLICATIONCORE_API virtual bool SupportsDynamicCursorLock() const override;
	APPLICATIONCORE_API virtual bool IsFullscreenSupported() const override;
	APPLICATIONCORE_API virtual void SetText(const TCHAR* const Text) override;
	APPLICATIONCORE_API virtual void DrawAttention(const FWindowDrawAttentionParameters& Parameters) override;

public:

	// IUnknown interface

	APPLICATIONCORE_API HRESULT STDCALL QueryInterface(REFIID iid, void ** ppvObject) override;
	APPLICATIONCORE_API ULONG STDCALL AddRef(void) override;
	APPLICATIONCORE_API ULONG STDCALL Release(void) override;

public:

	// IDropTarget interface

	APPLICATIONCORE_API virtual HRESULT STDCALL DragEnter( __RPC__in_opt IDataObject *DataObjectPointer, ::DWORD KeyState, POINTL CursorPosition, __RPC__inout ::DWORD *CursorEffect) override;
	APPLICATIONCORE_API virtual HRESULT STDCALL DragOver( ::DWORD KeyState, POINTL CursorPosition, __RPC__inout ::DWORD *CursorEffect) override;
	APPLICATIONCORE_API virtual HRESULT STDCALL DragLeave( void ) override;
	APPLICATIONCORE_API virtual HRESULT STDCALL Drop( __RPC__in_opt IDataObject *DataObjectPointer, ::DWORD KeyState, POINTL CursorPosition, __RPC__inout ::DWORD *CursorEffect) override;

private:

	/** Protect the constructor; only TSharedRefs of this class can be made. */
	APPLICATIONCORE_API FWindowsWindow();

	APPLICATIONCORE_API void UpdateVisibility();

	/**
	 * Creates an HRGN for the window's current region.  Remember to delete the returned handle when you're done with it using ::DeleteObject, unless
     * you're passing it to SetWindowRgn(), which will absorb the reference itself.
	 *
	 * @param bForQuery Bypasses situational behavior compensation, so that the same compensation does not also need to be applied to the queried
	 *     location.
	 */
	APPLICATIONCORE_API HRGN MakeWindowRegionObject(bool bForQuery) const;

	APPLICATIONCORE_API void DisableTouchFeedback();

	int32 GetWin32BorderSize() const;

private:

	/** The application that owns this window. */
	FWindowsApplication* OwningApplication;

	/** The window's handle. */
	HWND HWnd;

	/** Store the window region size for querying whether a point lies within the window. */
	int32 RegionWidth;
	int32 RegionHeight;
	
	/** The mode that the window is in (windowed, fullscreen, windowedfullscreen ) */
	EWindowMode::Type WindowMode;

	/** This object's reference count (for the IUnknown interface). */
	int32 OLEReferenceCount;
		
	/** The placement of the window before it entered a fullscreen state. */
	WINDOWPLACEMENT PreFullscreenWindowPlacement;

	/** The placement of the window before it entered a minimized state due to its parent window being minimized. */
	WINDOWPLACEMENT PreParentMinimizedWindowPlacement;

	/** Virtual width and height of the window.  This is only different than the actual width and height for
	    windows which we're trying to optimize because their size changes frequently.  We'll create a larger
		window and have Windows draw it "cropped" so that it appears smaller, rather than actually resizing
		it and incurring a GPU buffer resize performance hit */
	int32 VirtualWidth;
	int32 VirtualHeight;

	/** Current aspect ratio of window's client area */
	float AspectRatio;

	/** Whether the window is currently shown */
	bool bIsVisible : 1;

	/** Whether the window is yet to have its first Show() call. This is set false after first Show(). */
	bool bIsFirstTimeVisible : 1;

	/** We cache the min/max state for any Minimize, Maximize, or Restore calls that were made before the first Show */
	bool bInitiallyMinimized : 1;
	bool bInitiallyMaximized : 1;

	/**
	 * Ratio of pixels to SlateUnits in this window.
	 * E.g. DPIScale of 2.0 means there is a 2x2 pixel square for every 1x1 SlateUnit.
	 */
	float DPIScaleFactor;

	/** when true the window is responsible for its own size adjustments in response to a DPI change */
	bool bHandleManualDPIChanges = false;

	/** when true the window in the middle of a windows modal size loop */
	bool bInModalSizeLoop = false;
};
